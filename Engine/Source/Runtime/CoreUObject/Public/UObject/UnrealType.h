// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealType.h: Unreal engine base type definitions.
=============================================================================*/

#pragma once

#include "Concepts/GetTypeHashable.h"
#include "Concepts/DerivedFrom.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h"
#include "Containers/LinkedListBuilder.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "Containers/ScriptArray.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/NotNull.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/SerializedPropertyScope.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Casts.h"
#include "Templates/IsPODType.h"
#include "Templates/IsUEnumClass.h"
#include "Templates/MemoryOps.h"
#include "Templates/Models.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PropertyTag.h"
#include "UObject/ScriptDelegates.h"
#include "UObject/ScriptInterface.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/SparseDelegate.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include <type_traits>

#define UE_API COREUOBJECT_API

class FBlake3;
class FOutputDevice;
class UPackageMap;
class UPropertyWrapper;
enum ELifetimeCondition : int;
struct CGetTypeHashable;
struct FUObjectSerializeContext;
class FProperty;
class FNumericProperty;
template <typename FuncType> class TFunctionRef;
namespace UE { class FPropertyTypeName; }
namespace UE { class FPropertyTypeNameBuilder; }
namespace UE::GC
{
	class FPropertyStack;
	class FSchemaBuilder;
}
enum class EPropertyVisitorControlFlow : uint8;
struct FPropertyVisitorPath;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogType, Log, All);

namespace UE::CoreUObject::Private
{
	enum class ENonNullableBehavior
	{
		LogWarning                    = 0,
		LogError                      = 1,
		CreateDefaultObjectIfPossible = 2
	};

	COREUOBJECT_API ENonNullableBehavior GetNonNullableBehavior();
}

/*-----------------------------------------------------------------------------
	FProperty.
-----------------------------------------------------------------------------*/

enum EPropertyExportCPPFlags
{
	/** Indicates that there are no special C++ export flags */
	CPPF_None						=	0x00000000,
	/** Indicates that we are exporting this property's CPP text for an optional parameter value */
	CPPF_OptionalValue				=	0x00000001,
	/** Indicates that we are exporting this property's CPP text for an argument or return value */
	CPPF_ArgumentOrReturnValue		=	0x00000002,
	/** Indicates thet we are exporting this property's CPP text for C++ definition of a function. */
	CPPF_Implementation				=	0x00000004,
	/** Indicates that we are exporting this property's CPP text with an custom type name */
	CPPF_CustomTypeName				=	0x00000008,
	/** No 'const' keyword */
	CPPF_NoConst					=	0x00000010,
	/** No reference '&' sign */
	CPPF_NoRef						=	0x00000020,
	/** No static array [%d] */
	CPPF_NoStaticArray				=	0x00000040,
	/** Blueprint compiler generated C++ code */
	CPPF_BlueprintCppBackend		=	0x00000080,
	/** Indicates to not use TObjectPtr but use USomething* instead */
	CPPF_NoTObjectPtr				=	0x00000100,
};

enum class EConvertFromTypeResult
{
	/** No conversion was performed. Use SerializeItem to serialize the property value. */
	UseSerializeItem,
	/** No conversion was performed. The property value was serialized. Skip SerializeItem. */
	Serialized,
	/** No conversion is possible. Skip SerializeItem. */
	CannotConvert,
	/** Conversion of the property value was performed. Skip SerializeItem. */
	Converted,
};

enum class EPropertyMemoryAccess : uint8
{
	// Direct memory access - the associated pointer points to the memory at the reflected item.
	Direct,

	// Container access - the associated pointer points to the outer of the reflected item.
	// Access via containers will use getter and setters, if present.
	InContainer
};

namespace UEProperty_Private { class FProperty_DoNotUse; }


/** Type of pointer provided for property API functions */
enum class EPropertyPointerType
{
	Direct = 0, /** Raw property access */
	Container = 1, /** Property access through its owner container */
};

namespace UE::CoreUObject::Private
{
	// Defined in EnumProperty.cpp and used by both FEnumProperty and FByteProperty.
	// They don't have an API macro because they're not intended to be called outside of CoreUObject.
	const TCHAR* ImportEnumFromBuffer(UEnum* Enum, const FProperty* PropertyToSet, const FNumericProperty* UnderlyingProp, const TCHAR* PropertyClassName, const TCHAR* Buffer, void* ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, FOutputDevice* ErrorText);

	void ExportEnumToBuffer(const UEnum* Enum, const FProperty* Prop, const FNumericProperty* NumericProp, FString& ValueStr, const void* PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope);
}

//
// An UnrealScript variable.
//
class FProperty : public FField
{
	DECLARE_FIELD_API(FProperty, FField, CASTCLASS_FProperty, UE_API)

	// Persistent variables.
	int32			ArrayDim;
	UE_DEPRECATED(5.5, "Use GetElementSize/SetElementSize instead.")
	int32			ElementSize;
public:
	EPropertyFlags	PropertyFlags;
	uint16			RepIndex;

private:
	TEnumAsByte<ELifetimeCondition> BlueprintReplicationCondition;

#if WITH_EDITORONLY_DATA || WITH_METADATA
	union
	{
#if WITH_EDITORONLY_DATA
		/** Index of the property within its owner, inclusive of base properties. Generated during Link(). */
		int32 IndexInOwner = -1;
#endif
#if WITH_METADATA && UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Number of metadata parameters stored in this object at compile time from UHT
		 * This union member is active until initial UObject construction in UObjectProcessRegistrants populates runtime metadata
		 */
		int32 NumMetaDataParams;
#endif
	};
#endif // WITH_EDITORONLY_DATA || WITH_METADATA

	// In memory variables (generated during Link()).
	// When UE_WITH_CONSTINIT_UOBJECT is set, this is set at compile time for compiled-in objects.
	int32		Offset_Internal;

public:
	/** In memory only: Linked list of properties from most-derived to base **/
	FProperty*	PropertyLinkNext = nullptr;

	union
	{
		/** In memory only: Linked list of object reference properties from most-derived to base **/
		FProperty*  NextRef = nullptr;
#if WITH_METADATA && UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Metadata parameters stored in this object at compile time from UHT
		 * This union member is active until initial UObject construction in UObjectProcessRegistrants populates runtime metadata
		 */
		const UE::CodeGen::ConstInit::FMetaData* MetaDataParams;
#endif
	};
	union
	{
		/** In memory only: Linked list of properties requiring destruction. Note this does not include things that will be destroyed by the native destructor **/
		FProperty*	DestructorLinkNext = nullptr;
#if UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * RepNotify function name stored in this object at compile time from UHT
		 * This union member is active until initial UObject construction in UObjectProcessRegistrants 
		 */
		const UTF8CHAR* RepNotifyFuncNameUTF8;
#endif
	};
	/** In memory only: Linked list of properties requiring post constructor initialization.**/
	FProperty*	PostConstructLinkNext = nullptr;

	FName		RepNotifyFunc;

public:
	// Constructors.
	COREUOBJECT_API FProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API FProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API FProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithoutOffset& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);

#if UE_WITH_CONSTINIT_UOBJECT
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	explicit consteval FProperty(UE::CodeGen::ConstInit::FPropertyParams InParams)
		: Super(ConstEval, InParams.Owner, InParams.NextProperty, InParams.NameUTF8)
		, ArrayDim(InParams.ArrayDim)
		, ElementSize(InParams.ElementSize)
		, PropertyFlags(InParams.PropertyFlags)
		, RepIndex(0)
		, BlueprintReplicationCondition()
#if WITH_METADATA
		, NumMetaDataParams(InParams.MetaData.Num())
#endif
		, Offset_Internal(InParams.Offset)
		, PropertyLinkNext(nullptr)
#if WITH_METADATA
		, MetaDataParams(InParams.MetaData.GetData())
#endif
		, RepNotifyFuncNameUTF8(InParams.RepNotifyFuncUTF8)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void InitializeConstInitProperty(UStruct* InStructOwner);
	void InitializeConstInitProperty(FProperty* InPropertyOwner);
#endif

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API explicit FProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// ElementSize accessors to facilitate underlying type change
	COREUOBJECT_API void SetElementSize(int32 NewSize);
	int32 	GetElementSize() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ElementSize;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface

	// FField interface
	COREUOBJECT_API virtual void PostDuplicate(const FField& InField) override;

	/** parses and imports a text definition of a single property's value (if array, may be an individual element)
	 * also includes parsing of special operations for array properties (Add/Remove/RemoveIndex/Empty)
	 * @param Str	the string to parse
	 * @param DestData	base location the parsed property should place its data (DestData + ParsedProperty->Offset)
	 * @param ObjectStruct	the struct containing the valid fields
	 * @param SubobjectOuter	owner of DestData and any subobjects within it
	 * @param PortFlags	property import flags
	 * @param Warn	output device for any error messages
	 * @param DefinedProperties (out)	list of properties/indices that have been parsed by previous calls, so duplicate definitions cause an error
	 * @return pointer to remaining text in the stream (even on failure, but on failure it may not be advanced past the entire key/value pair)
	 */
	static COREUOBJECT_API const TCHAR* ImportSingleProperty( const TCHAR* Str, void* DestData, const UStruct* ObjectStruct, UObject* SubobjectOuter, int32 PortFlags,
											FOutputDevice* Warn, TArray<struct FDefinedProperty>& DefinedProperties );

	/** Gets a redirected property name, will return NAME_None if no redirection was found */
	static COREUOBJECT_API FName FindRedirectedPropertyName(const UStruct* ObjectStruct, FName OldName);

	/**
	 * Returns the C++ name of the property, including the _DEPRECATED suffix if the
	 * property is deprecated.
	 *
	 * @return C++ name of property
	 */
	COREUOBJECT_API FString GetNameCPP() const;

	// UHT interface
	COREUOBJECT_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const;

	/**
	 * Returns the text to use for exporting this property to header file.
	 *
	 * @param	ExtendedTypeText	for property types which use templates, will be filled in with the type
	 * @param	CPPExportFlags		flags for modifying the behavior of the export
	 */
	virtual FString GetCPPType( FString* ExtendedTypeText=NULL, uint32 CPPExportFlags=0 ) const PURE_VIRTUAL(FProperty::GetCPPType,return TEXT(""););
	// End of UHT interface

#if WITH_EDITORONLY_DATA
	/** Gets the wrapper object for this property or creates one if it doesn't exist yet */
	COREUOBJECT_API UPropertyWrapper* GetUPropertyWrapper();
#endif

	/** Checks if this property as a native setter function */
	virtual bool HasSetter() const
	{
		return false;
	}
	/** Checks if this property as a native getter function */
	virtual bool HasGetter() const
	{
		return false;
	}
	/** Checks if this property as a native setter or getter function */
	virtual bool HasSetterOrGetter() const
	{
		return false;
	}
	/** 
	 * Calls the native setter function for this property
	 * @param Container Pointer to the owner of this property (either UObject or struct)
	 * @param InValue Pointer to the new value
	 */
	virtual void CallSetter(void* Container, const void* InValue) const 
	{
		checkf(HasSetter(), TEXT("Calling a setter on %s but it doesn't have one"), *GetFullName());
	}
	/**
	 * Calls the native getter function for this property
	 * @param Container Pointer to the owner of this property (either UObject or struct)
	 * @param OutValue Pointer to the value where the existing property value will be copied to
	 */
	virtual void CallGetter(const void* Container, void* OutValue) const 
	{
		checkf(HasGetter(), TEXT("Calling a getter on %s but it doesn't have one"), *GetFullName());
	}

	UE_DEPRECATED(5.7, "Visit is deprecated, please use Visit with context instead.")
	COREUOBJECT_API EPropertyVisitorControlFlow Visit(const FPropertyVisitorData& Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath& /*Path*/, const FPropertyVisitorData& /*Data*/)> InFunc) const;

	/**
	 * Visits this property and allows recursion into the inner properties
	 * This method allows callers to visit inner properties without knowing about its container type as opposed to TPropertyIterator.
	 * This visit property pattern facilitates the recursion into user defined properties and allows users to add specific visit logic on UStruct via traits. 
	 * @param Data to the property to visit
	 * @param InFunc to call on each visited property, the return value controls what is the next behavior once this property has been visited
	 * @return the new action to take one visited this property
	 */
	COREUOBJECT_API EPropertyVisitorControlFlow Visit(const FPropertyVisitorData& Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const;

	/**
	 * Visits this property and allows recursion into the inner properties
	 * This method allows callers to visit inner properties without knowing about its container type as opposed to TPropertyIterator.
	 * This visit property pattern facilitates the recursion into user defined properties and allows users to add specific visit logic on UStruct via traits. 
	 * @param Context which containes the path that was computed until we reached this property and the sata to the property to visit
	 * @param InFunc to call on each visited property, the return value controls what is the next behavior once this property has been visited
	 * @return the new action to take one visited this property
	 */
	COREUOBJECT_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const;

	/**
	 * Attempt to resolve the given inner path info against this outer property to get the inner property value.
	 * @return The inner property value, or null if the path info is incompatible/missing on this property.
	 */
	COREUOBJECT_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const;

private:
	/** Set the alignment offset for this property 
	 * @return the size of the structure including this newly added property
	*/
	COREUOBJECT_API int32 SetupOffset();

protected:
	friend class FMapProperty;
	friend class UEProperty_Private::FProperty_DoNotUse;

	/** Set the alignment offset for this property - added for FMapProperty */
	COREUOBJECT_API void SetOffset_Internal(int32 NewOffset);

	/**
	 * Initializes internal state.
	 */
	COREUOBJECT_API void Init();

public:
#if WITH_EDITORONLY_DATA
	/** Return the index of the property in its owner. Only valid after Link(). -1 if not linked by a UStruct. */
	UE_INTERNAL UE_FORCEINLINE_HINT int32 GetIndexInOwner() const
	{
		return IndexInOwner;
	}

	/** Set the index of the property in its owner. Set by the owner in UStruct::Link(). */
	UE_INTERNAL UE_FORCEINLINE_HINT void SetIndexInOwner(int32 Index)
	{
		IndexInOwner = Index;
	}
#endif

	/** Return offset of property from container base. */
	UE_FORCEINLINE_HINT int32 GetOffset_ForDebug() const
	{
		return Offset_Internal;
	}
	/** Return offset of property from container base. */
	UE_FORCEINLINE_HINT int32 GetOffset_ForUFunction() const
	{
		return Offset_Internal;
	}
	/** Return offset of property from container base. */
	UE_FORCEINLINE_HINT int32 GetOffset_ForGC() const
	{
		return Offset_Internal;
	}
	/** Return offset of property from container base. */
	UE_FORCEINLINE_HINT int32 GetOffset_ForInternal() const
	{
		return Offset_Internal;
	}
	/** Return offset of property from container base. */
	UE_FORCEINLINE_HINT int32 GetOffset_ReplaceWith_ContainerPtrToValuePtr() const
	{
		return Offset_Internal;
	}

	void LinkWithoutChangingOffset(FArchive& Ar)
	{
		LinkInternal(Ar);
	}

	int32 Link(FArchive& Ar)
	{
		LinkInternal(Ar);
		return SetupOffset();
	}

#if WITH_METADATA
	/** Move metadata from static memory into metadata object on package */
	void InitializeMetaData();
#endif

protected:
	COREUOBJECT_API virtual void LinkInternal(FArchive& Ar);
public:

	/**
	* Allows a property to implement backwards compatibility handling for tagged properties
	* 
	* @param	Tag			property tag of the loading data
	* @param	Ar			the archive the data is being loaded from
	* @param	Data		a pointer to the container to write the loaded data to
	* @param	DefaultsStruct 
	* @param	Defaults	if available, a pointer to the container containing the default value for this property, or null
	*
	* @return	A state which tells the tagged property system how the property dealt with the data.
	*			Converted:        the function handled conversion.
	*			CannotConvert:    the tag is not something that the property can convert.
	*			Serialized:       the function handled serialization without conversion.
	*			UseSerializeItem: no conversion was done on the property - this can mean that the tag is correct and normal serialization applies or that the tag is incompatible.
	*/
	COREUOBJECT_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults);

	/**
	 * Determines whether the property values are identical.
	 * 
	 * @param	A			property data to be compared, already offset
	 * @param	B			property data to be compared, already offset
	 * @param	PortFlags	allows caller more control over how the property values are compared
	 *
	 * @return	true if the property values are identical
	 */
	virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags = 0) const PURE_VIRTUAL(FProperty::Identical, return false;);

	/**
	 * Determines whether the property values are identical.
	 * 
	 * @param	A			property container of data to be compared, NOT offset
	 * @param	B			property container of data to be compared, NOT offset
	 * @param	PortFlags	allows caller more control over how the property values are compared
	 *
	 * @return	true if the property values are identical
	 */
	bool Identical_InContainer(TNotNull<const void*> A, const void* B, int32 ArrayIndex = 0, uint32 PortFlags = 0) const
	{
		return Identical(ContainerPtrToValuePtr<void>(A, ArrayIndex), B ? ContainerPtrToValuePtr<void>(B, ArrayIndex) : nullptr, PortFlags);
	}

	/**
	 * Serializes the property with the struct's data residing in Data.
	 *
	 * @param	Ar				the archive to use for serialization
	 * @param	Data			pointer to the location of the beginning of the struct's property data
	 * @param	ArrayIdx		if not -1 (default), only this array slot will be serialized
	 */
	void SerializeBinProperty( FStructuredArchive::FSlot Slot, TNotNull<void*> Data, int32 ArrayIdx = -1 )
	{
		FStructuredArchive::FStream Stream = Slot.EnterStream();
		if( ShouldSerializeValue(Slot.GetUnderlyingArchive()) )
		{
			const int32 LoopMin = ArrayIdx < 0 ? 0 : ArrayIdx;
			const int32 LoopMax = ArrayIdx < 0 ? ArrayDim : ArrayIdx + 1;
			for (int32 Idx = LoopMin; Idx < LoopMax; Idx++)
			{
				// Keep setting the property in case something inside of SerializeItem changes it
				FSerializedPropertyScope SerializedProperty(Slot.GetUnderlyingArchive(), this);
				SerializeItem(Stream.EnterElement(), ContainerPtrToValuePtr<void>(Data, Idx));
			}
		}
	}
	/**
	 * Serializes the property with the struct's data residing in Data, unless it matches the default
	 *
	 * @param	Ar				the archive to use for serialization
	 * @param	Data			pointer to the location of the beginning of the struct's property data
	 * @param	DefaultData		pointer to the location of the beginning of the data that should be compared against
	 * @param	DefaultStruct	struct corresponding to the block of memory located at DefaultData 
	 */
	void SerializeNonMatchingBinProperty(FStructuredArchive::FSlot Slot, TNotNull<void*> Data, void const* DefaultData, UStruct* DefaultStruct)
	{
		FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
		FStructuredArchive::FStream Stream = Slot.EnterStream();

		if( ShouldSerializeValue(UnderlyingArchive) )
		{
			for (int32 Idx = 0; Idx < ArrayDim; Idx++)
			{
				void* Target = ContainerPtrToValuePtr<void>(Data, Idx);
				void const* Default = ContainerPtrToValuePtrForDefaults<void>(DefaultStruct, DefaultData, Idx);
				if ( !Identical(Target, Default, UnderlyingArchive.GetPortFlags()) )
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, this);
					SerializeItem( Stream.EnterElement(), Target, Default );
				}
			}
		}
	}

	virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults = nullptr) const PURE_VIRTUAL(FProperty::SerializeItem, );
	COREUOBJECT_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const;
	COREUOBJECT_API virtual bool SupportsNetSharedSerialization() const;

	void ExportTextItem_Direct(FString& ValueStr, TNotNull<const void*> PropertyValue, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope = nullptr) const
	{
		ExportText_Internal(ValueStr, PropertyValue, EPropertyPointerType::Direct, DefaultValue, Parent, PortFlags, ExportRootScope);
	}

	void ExportTextItem_InContainer(FString& ValueStr, TNotNull<const void*> Container, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope = nullptr) const
	{
		ExportText_Internal(ValueStr, Container, EPropertyPointerType::Container, DefaultValue, Parent, PortFlags, ExportRootScope);
	}

	/**
	 * Import a text value
	 * @param Buffer		Text representing the property value
	 * @param Container	Pointer to the container that owns this property (either UObject pointer or a struct pointer)
	 * @param OwnerObject	Object that owns the property container (if the container is an UObject then Container is also OwnerObject)
	 * @param PortFlags	Flags controlling the behavior when importing the value
	 * @param ErrorText	Output device for throwing warnings or errors on import
	 * @returns Buffer pointer advanced by the number of characters consumed when reading the text value
	 */
	const TCHAR* ImportText_InContainer(const TCHAR* Buffer, TNotNull<void*> Container, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText = (FOutputDevice*)GWarn) const
	{
		if (!ValidateImportFlags(PortFlags, ErrorText) || Buffer == nullptr)
		{
			return nullptr;
		}
		PortFlags |= EPropertyPortFlags::PPF_UseDeprecatedProperties; // Imports should always process deprecated properties
		return ImportText_Internal(Buffer, Container, EPropertyPointerType::Container, OwnerObject, PortFlags, ErrorText);
	}

	/**
	 * Import a text value
	 * @param Buffer		Text representing the property value
	 * @param PropertyPtr	Pointer to property value
	 * @param OwnerObject	Object that owns the property
	 * @param PortFlags	Flags controlling the behavior when importing the value
	 * @param ErrorText	Output device for throwing warnings or errors on import
	 * @returns Buffer pointer advanced by the number of characters consumed when reading the text value
	 */
	const TCHAR* ImportText_Direct(const TCHAR* Buffer, TNotNull<void*> PropertyPtr, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText = (FOutputDevice*)GWarn) const
	{
		if (!ValidateImportFlags(PortFlags, ErrorText) || Buffer == NULL)
		{
			return NULL;
		}
		PortFlags |= EPropertyPortFlags::PPF_UseDeprecatedProperties; // Imports should always process deprecated properties
		return ImportText_Internal(Buffer, PropertyPtr, EPropertyPointerType::Direct, OwnerObject, PortFlags, ErrorText);
	}

	inline void SetValue_InContainer(void* OutContainer, const void* InValue) const
	{
		if (!HasSetter())
		{
			CopyCompleteValue(ContainerVoidPtrToValuePtrInternal(OutContainer, 0), InValue);
		}
		else
		{
			CallSetter(OutContainer, InValue);
		}
	}
	inline void GetValue_InContainer(void const* InContainer, void* OutValue) const
	{
		if (!HasGetter())
		{
			CopyCompleteValue(OutValue, ContainerVoidPtrToValuePtrInternal((void*)InContainer, 0));
		}
		else
		{
			CallGetter(InContainer, OutValue);
		}
	}

	/**
	* Copies a single value to the property even if the property represents a static array of values
	* @param OutContainer Instance owner of the property
	* @param InValue Pointer to the memory that the value will be copied from. Must be at least ElementSize big
	* @param ArrayIndex Index into the static array to copy the value from. If the property is not a static array it should be 0
	*/
	COREUOBJECT_API void SetSingleValue_InContainer(void* OutContainer, const void* InValue, int32 ArrayIndex) const;

	/**
	* Copies a single value to OutValue even if the property represents a static array of values
	* @param InContainer Instance owner of the property
	* @param OutValue Pointer to the memory that the value will be copied to. Must be at least ElementSize big
	* @param ArrayIndex Index into the static array to copy the value from. If the property is not a static array it should be 0
	*/
	COREUOBJECT_API void GetSingleValue_InContainer(const void* InContainer, void* OutValue, int32 ArrayIndex) const;

	/** Allocates and initializes memory to hold a value this property represents */
	COREUOBJECT_API void* AllocateAndInitializeValue() const;

	/** Destroys and frees memory with a value this property represents */
	COREUOBJECT_API void DestroyAndFreeValue(void* InMemory) const;

	/**
	 * Helper function for setting container / struct property value and performing operation directly on the value memory
	 * @param OutContainer Pointer to the container that owns the property. Can be null but then setters and getters will not be used.
	 * @param DirectPropertyAddress Direct property value address. Can be null only if OutContainer is a valid pointer.
	 * @param DirectValueAccessFunc Function that manipulates directly on property value address. The value address can be different than the passed in DirectPropertyAddress if setters and getters are present and OutContainer pointer is valid.
	 */
	COREUOBJECT_API void PerformOperationWithSetter(void* OutContainer, void* DirectPropertyAddress, TFunctionRef<void(void*)> DirectValueAccessFunc) const;

	/**
	 * Helper function for getting container / struct property value and performing operation directly on the value memory
	 * @param OutContainer Pointer to the container that owns the property. Can be null but then setters and getters will not be used.
	 * @param DirectPropertyAddress Direct property value address. Can be null only if OutContainer is a valid pointer.
	 * @param DirectValueAccessFunc Function that manipulates directly on property value address. The value address can be different than the passed in DirectPropertyAddress if setters and getters are present and OutContainer pointer is valid.
	 */
	COREUOBJECT_API void PerformOperationWithGetter(void* OutContainer, const void* DirectPropertyAddress, TFunctionRef<void(const void*)> DirectValueAccessFunc) const;

	/** 
	 * Gets value address at given index inside of a static array or container
	 * @param InValueAddress address of the value represented by this property
	 * @param Index into the static array or container
	 * @returns address of the value at given index
	 */
	COREUOBJECT_API virtual void* GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, int32 Index) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Updates the given HashBuilder with name and type information of this Property.
	 * Contract: the hashed data is different from any property that serializes differently in Tagged Property Serialization.
	 * If necessary to follow the contract, subclasses should override and add further information after calling
	 * Super::AppendSchemaHash. e.g. FStructProperty needs to append the schema hash of its UStruct.
	 * 
	 * @param HashBuilder The builder to Update with property information
	 * @param bSkipEditorOnly Used by subclasses with sub- properties that may be editor-only.
	 *                        If true, sub- properties that are editor-only should not be appended to the hash.
	 *                        This property's base data is appended without regard for bSkipEditorOnly.
	 */
	COREUOBJECT_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const;
#endif
protected:

	virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope = nullptr) const PURE_VIRTUAL(FProperty::ExportText, );
	virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const PURE_VIRTUAL(FProperty::ImportText, return nullptr;);

public:
	
	COREUOBJECT_API bool ExportText_Direct(FString& ValueStr, TNotNull<const void*> Data, const void* Delta, UObject* Parent, int32 PortFlags, UObject* ExportRootScope = nullptr) const;
	UE_FORCEINLINE_HINT bool ExportText_InContainer(int32 Index, FString& ValueStr, TNotNull<const void*> Data, const void* Delta, UObject* Parent, int32 PortFlags, UObject* ExportRootScope = nullptr) const
	{
		return ExportText_Direct(ValueStr, ContainerPtrToValuePtr<void>(Data, Index), ContainerPtrToValuePtrForDefaults<void>(NULL, Delta, Index), Parent, PortFlags, ExportRootScope);
	}

private:

	inline void* ContainerVoidPtrToValuePtrInternal(void* ContainerPtr, int32 ArrayIndex) const
	{
		checkf((ArrayIndex >= 0) && (ArrayIndex < ArrayDim), TEXT("Array index out of bounds: %i from an array of size %i"), ArrayIndex, ArrayDim);
		check(ContainerPtr);

		if (0)
		{
			// in the future, these checks will be tested if the property is NOT relative to a UClass
			check(!GetOwner<UClass>()); // Check we are _not_ calling this on a direct child property of a UClass, you should pass in a UObject* in that case
		}

		return (uint8*)ContainerPtr + Offset_Internal + static_cast<size_t>(GetElementSize()) * ArrayIndex;
	}

	inline void* ContainerUObjectPtrToValuePtrInternal(UObject* ContainerPtr, int32 ArrayIndex) const
	{
		checkf((ArrayIndex >= 0) && (ArrayIndex < ArrayDim), TEXT("Array index out of bounds: %i from an array of size %i"), ArrayIndex, ArrayDim);
		check(ContainerPtr);

		// in the future, these checks will be tested if the property is supposed be from a UClass
		// need something for networking, since those are NOT live uobjects, just memory blocks
		checkf(((UObject*)ContainerPtr)->IsValidLowLevel(), TEXT("%s does not belong to a valid ContainerPtr"), *GetName()); // Check its a valid UObject that was passed in
		checkf(((UObject*)ContainerPtr)->GetClass() != NULL, TEXT("Expected that %s belongs to a class, and %s is not a class"), *GetName(), *((UObject*)ContainerPtr)->GetName());
		checkf(GetOwner<UClass>(), TEXT("%s does not belong to a class"), *GetName()); // Check that the outer of this property is a UClass (not another property)

		// Check that the object we are accessing is of the class that contains this property
		checkf(((UObject*)ContainerPtr)->IsA(GetOwner<UClass>()), TEXT("'%s' is of class '%s' however property '%s' belongs to class '%s'")
			, *((UObject*)ContainerPtr)->GetName()
			, *((UObject*)ContainerPtr)->GetClass()->GetName()
			, *GetName()
			, *GetOwner<UClass>()->GetName());

		if (0)
		{
			// in the future, these checks will be tested if the property is NOT relative to a UClass
			check(!GetOwner<UClass>()); // Check we are _not_ calling this on a direct child property of a UClass, you should pass in a UObject* in that case
		}

		return (uint8*)ContainerPtr + Offset_Internal + static_cast<size_t>(GetElementSize()) * ArrayIndex;
	}

protected:

	friend const TCHAR* UE::CoreUObject::Private::ImportEnumFromBuffer(UEnum* Enum, const FProperty* PropertyToSet, const FNumericProperty* UnderlyingProp, const TCHAR* PropertyClassName, const TCHAR* Buffer, void* ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, FOutputDevice* ErrorText);
	friend void UE::CoreUObject::Private::ExportEnumToBuffer(const UEnum* Enum, const FProperty* Prop, const FNumericProperty* NumericProp, FString& ValueStr, const void* PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope);

	inline void* PointerToValuePtr(void const* ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, int32 ArrayIndex = 0) const
	{
		if (PropertyPointerType == EPropertyPointerType::Container)
		{
			return (uint8*)ContainerOrPropertyPtr + Offset_Internal + static_cast<size_t>(GetElementSize()) * ArrayIndex;
		}
		else
		{
			return (void*)ContainerOrPropertyPtr;
		}
	}

public:

	/** 
	 *	Get the pointer to property value in a supplied 'container'. 
	 *	You can _only_ call this function on a UObject* or a uint8*. If the property you want is a 'top level' UObject property, you _must_
	 *	call the function passing in a UObject* and not a uint8*. There are checks inside the function to vertify this.
	 *	@param	ContainerPtr			UObject* or uint8* to container of property value
	 *	@param	ArrayIndex				In array case, index of array element we want
	 */
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType* ContainerPtrToValuePtr(UObject* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return (ValueType*)ContainerUObjectPtrToValuePtrInternal(ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType* ContainerPtrToValuePtr(void* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return (ValueType*)ContainerVoidPtrToValuePtrInternal(ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType const* ContainerPtrToValuePtr(UObject const* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return ContainerPtrToValuePtr<ValueType>((UObject*)ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType const* ContainerPtrToValuePtr(void const* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return ContainerPtrToValuePtr<ValueType>((void*)ContainerPtr, ArrayIndex);
	}

	// Default variants, these accept and return nullptr, and also check the property against the size of the container. 
	// If we copy from a baseclass (like for a CDO), then this will give nullptr for any property that doesn't belong to the baseclass.
	template<typename ValueType>
	inline ValueType* ContainerPtrToValuePtrForDefaults(const UStruct* ContainerClass, UObject* ContainerPtr, int32 ArrayIndex = 0) const
	{
		if (ContainerPtr && IsInContainer(ContainerClass))
		{
			return ContainerPtrToValuePtr<ValueType>(ContainerPtr, ArrayIndex);
		}
		return nullptr;
	}
	template<typename ValueType>
	inline ValueType* ContainerPtrToValuePtrForDefaults(const UStruct* ContainerClass, void* ContainerPtr, int32 ArrayIndex = 0) const
	{
		if (ContainerPtr && IsInContainer(ContainerClass))
		{
			return ContainerPtrToValuePtr<ValueType>(ContainerPtr, ArrayIndex);
		}
		return nullptr;
	}
	template<typename ValueType>
	inline const ValueType* ContainerPtrToValuePtrForDefaults(const UStruct* ContainerClass, const UObject* ContainerPtr, int32 ArrayIndex = 0) const
	{
		if (ContainerPtr && IsInContainer(ContainerClass))
		{
			return ContainerPtrToValuePtr<ValueType>(ContainerPtr, ArrayIndex);
		}
		return nullptr;
	}
	template<typename ValueType>
	inline const ValueType* ContainerPtrToValuePtrForDefaults(const UStruct* ContainerClass, const void* ContainerPtr, int32 ArrayIndex = 0) const
	{
		if (ContainerPtr && IsInContainer(ContainerClass))
		{
			return ContainerPtrToValuePtr<ValueType>(ContainerPtr, ArrayIndex);
		}
		return nullptr;
	}
	/** See if the offset of this property is below the supplied container size */
	UE_FORCEINLINE_HINT bool IsInContainer(int32 ContainerSize) const
	{
		return Offset_Internal + GetSize() <= ContainerSize;
	}
	/** See if the offset of this property is below the supplied container size */
	UE_FORCEINLINE_HINT bool IsInContainer(const UStruct* ContainerClass) const
	{
		return Offset_Internal + GetSize() <= (ContainerClass ? ContainerClass->GetPropertiesSize() : MAX_int32);
	}

	/**
	 * Copy the value for a single element of this property.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET + INDEX * SIZE, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 *									INDEX = the index that you want to copy.  for properties which are not arrays, this should always be 0
	 *									SIZE = the ElementSize of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 * @param	InstancingParams	contains information about instancing (if any) to perform
	 */
	inline void CopySingleValue( void* Dest, void const* Src ) const
	{
		if(Dest != Src)
		{
			if (PropertyFlags & CPF_IsPlainOldData)
			{
				FMemory::Memcpy( Dest, Src, GetElementSize() );
			}
			else
			{
				CopyValuesInternal(Dest, Src, 1);
			}
		}
	}

	/**
	 * Returns the hash value for an element of this property.
	 */
	COREUOBJECT_API uint32 GetValueTypeHash(const void* Src) const;

protected:
	COREUOBJECT_API virtual void CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const;
	COREUOBJECT_API virtual uint32 GetValueTypeHashInternal(const void* Src) const;

public:
	/**
	 * Copy the value for all elements of this property.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 * @param	InstancingParams	contains information about instancing (if any) to perform
	 */
	inline void CopyCompleteValue( void* Dest, void const* Src ) const
	{
		if(Dest != Src)
		{
			if (PropertyFlags & CPF_IsPlainOldData)
			{
				FMemory::Memcpy( Dest, Src, static_cast<size_t>(GetElementSize()) * ArrayDim );
			}
			else
			{
				CopyValuesInternal(Dest, Src, ArrayDim);
			}
		}
	}
	UE_FORCEINLINE_HINT void CopyCompleteValue_InContainer( void* Dest, void const* Src ) const
	{
		return CopyCompleteValue(ContainerPtrToValuePtr<void>(Dest), ContainerPtrToValuePtr<void>(Src));
	}

	/**
	 * Copy the value for a single element of this property. To the script VM.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET + INDEX * SIZE, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 *									INDEX = the index that you want to copy.  for properties which are not arrays, this should always be 0
	 *									SIZE = the ElementSize of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 */
	COREUOBJECT_API virtual void CopySingleValueToScriptVM( void* Dest, void const* Src ) const;

	/**
	 * Copy the value for all elements of this property. To the script VM.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 */
	COREUOBJECT_API virtual void CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const;

	/**
	 * Equivalent to the above functions, but using the container and aware of getters/setters when the container has them.
	 */
	COREUOBJECT_API virtual void CopyCompleteValueToScriptVM_InContainer( void* OutValue, void const* InContainer ) const;
	COREUOBJECT_API virtual void CopyCompleteValueFromScriptVM_InContainer( void* OutContainer, void const* InValue ) const;

	/**
	 * Copy the value for a single element of this property. From the script VM.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET + INDEX * SIZE, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 *									INDEX = the index that you want to copy.  for properties which are not arrays, this should always be 0
	 *									SIZE = the ElementSize of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 */
	COREUOBJECT_API virtual void CopySingleValueFromScriptVM( void* Dest, void const* Src ) const;

	/**
	 * Copy the value for all elements of this property. From the script VM.
	 * 
	 * @param	Dest				the address where the value should be copied to.  This should always correspond to the BASE + OFFSET, where
	 *									BASE = (for member properties) the address of the UObject which contains this data, (for locals/parameters) the address of the space allocated for the function's locals
	 *									OFFSET = the Offset of this FProperty
	 * @param	Src					the address of the value to copy from. should be evaluated the same way as Dest
	 */
	COREUOBJECT_API virtual void CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const;

	/**
	 * Zeros the value for this property. The existing data is assumed valid (so for example this calls FString::Empty)
	 * This only does one item and not the entire fixed size array.
	 *
	 * @param	Data		the address of the value for this property that should be cleared.
	 */
	inline void ClearValue( void* Data ) const
	{
		if (HasAllPropertyFlags(CPF_NoDestructor | CPF_ZeroConstructor))
		{
			FMemory::Memzero( Data, GetElementSize() );
		}
		else
		{
			ClearValueInternal((uint8*)Data);
		}
	}
	/**
	 * Zeros the value for this property. The existing data is assumed valid (so for example this calls FString::Empty)
	 * This only does one item and not the entire fixed size array.
	 *
	 * @param	Data		the address of the container of the value for this property that should be cleared.
	 */
	inline void ClearValue_InContainer( void* Data, int32 ArrayIndex = 0 ) const
	{
		if (HasAllPropertyFlags(CPF_NoDestructor | CPF_ZeroConstructor))
		{
			FMemory::Memzero( ContainerPtrToValuePtr<void>(Data, ArrayIndex), GetElementSize() );
		}
		else
		{
			ClearValueInternal(ContainerPtrToValuePtr<uint8>(Data, ArrayIndex));
		}
	}
protected:
	COREUOBJECT_API virtual void ClearValueInternal( void* Data ) const;
public:
	/**
	 * Destroys the value for this property. The existing data is assumed valid (so for example this calls FString::Empty)
	 * This does the entire fixed size array.
	 *
	 * @param	Dest		the address of the value for this property that should be destroyed.
	 */
	inline void DestroyValue( void* Dest ) const
	{
		if (!(PropertyFlags & CPF_NoDestructor))
		{
			DestroyValueInternal(Dest);
		}
	}
	/**
	 * Destroys the value for this property. The existing data is assumed valid (so for example this calls FString::Empty)
	 * This does the entire fixed size array.
	 *
	 * @param	Dest		the address of the container containing the value that should be destroyed.
	 */
	inline void DestroyValue_InContainer( void* Dest ) const
	{
		if (!(PropertyFlags & CPF_NoDestructor))
		{
			DestroyValueInternal(ContainerPtrToValuePtr<void>(Dest));
		}
	}

protected:
	COREUOBJECT_API virtual void DestroyValueInternal( void* Dest ) const;
public:

	/**
	 * Returns true if the property or any of the child properties should be cleared on FinishDestroy.
	 */
	bool ContainsFinishDestroy(TArray<const FStructProperty*>& EncounteredStructProps) const
	{
		// Skip if the property does not need any destroying. 
		if (PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor))
		{
			return false;
		}
		return ContainsClearOnFinishDestroyInternal(EncounteredStructProps);
	}

	/**
	 * Applies appropriate finish destroy actions for the property if needed.
	 * This is used during UObject destruction to e.g. safely clear values which rely on UScriptStructs. 
	 * This does the entire fixed size array.
	 *
	 * @param	Data		the address of the value for this property that should be handled for finish destroy.
	 */
	void FinishDestroy( void* Data ) const
	{
		// Skip if the property does not need any destroying. 
		if (PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor))
		{
			return;
		}
		FinishDestroyInternal(Data);
	}

	/**
	 * Applies appropriate finish destroy actions for the property if needed.
	 * This is used during UObject destruction to e.g. safely clear values which rely on UScriptStructs. 
	 * This does the entire fixed size array.
	 *
	 * @param	Data		the address of the container containing the value that should be handled for finish destroy.
	 */
	void FinishDestroy_InContainer( void* Data ) const
	{
		// Skip if the property does not need any destroying. 
		if (PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor))
		{
			return;
		}
		FinishDestroyInternal(ContainerPtrToValuePtr<void>(Data));
	}
protected:
	COREUOBJECT_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const;
	COREUOBJECT_API virtual void FinishDestroyInternal( void* Data ) const;
public:

	/**
	 * Zeros, copies from the default, or calls the constructor for on the value for this property. 
	 * The existing data is assumed invalid (so for example this might indirectly call FString::FString,
	 * This will do the entire fixed size array.
	 *
	 * @param	Dest		the address of the value for this property that should be cleared.
	 */
	inline void InitializeValue( void* Dest ) const
	{
		if (PropertyFlags & CPF_ZeroConstructor)
		{
			FMemory::Memzero(Dest, static_cast<size_t>(GetElementSize()) * ArrayDim);
		}
		else
		{
			InitializeValueInternal(Dest);
		}
	}
	/**
	 * Zeros, copies from the default, or calls the constructor for on the value for this property. 
	 * The existing data is assumed invalid (so for example this might indirectly call FString::FString,
	 * This will do the entire fixed size array.
	 *
	 * @param	Dest		the address of the container of value for this property that should be cleared.
	 */
	inline void InitializeValue_InContainer( void* Dest ) const
	{
		if (PropertyFlags & CPF_ZeroConstructor)
		{
			FMemory::Memzero(ContainerPtrToValuePtr<void>(Dest), static_cast<size_t>(GetElementSize()) * ArrayDim);
		}
		else
		{
			InitializeValueInternal(ContainerPtrToValuePtr<void>(Dest));
		}
	}
protected:
	COREUOBJECT_API virtual void InitializeValueInternal( void* Dest ) const;
public:

	/**
	 * Verify that modifying this property's value via ImportText is allowed.
	 * 
	 * @param	PortFlags	the flags specified in the call to ImportText
	 * @param	ErrorText	[out] set to the error message that should be displayed if returns false
	 *
	 * @return	true if ImportText should be allowed
	 */
	COREUOBJECT_API bool ValidateImportFlags( uint32 PortFlags, FOutputDevice* ErrorText = NULL ) const;
	COREUOBJECT_API bool ShouldPort( uint32 PortFlags=0 ) const;
	COREUOBJECT_API virtual FName GetID() const;

	/**
	 * Creates new copies of components
	 * 
	 * @param	Data				pointer to the address of the instanced object referenced by this UComponentProperty
	 * @param	DefaultData			pointer to the address of the default value of the instanced object referenced by this UComponentProperty
	 * @param	Owner				the object that contains this property's data
	 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates
	 */
	COREUOBJECT_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph );

	COREUOBJECT_API virtual int32 GetMinAlignment() const;

	/**
	 * Returns true if this property, or in the case of e.g. array or struct properties any sub- property, contains a
	 * UObject reference.
	 * @param	EncounteredStructProps		used to check for recursion in arrays
	 * @param	InReferenceType				type of object reference (strong / weak)
	 *
	 * @return true if property (or sub- properties) contains the specified type of UObject reference, false otherwise
	 */
	COREUOBJECT_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const;

	/**
	 * Returns true if this property, or in the case of e.g. array or struct properties any sub- property, contains a
	 * weak UObject reference.
	 *
	 * @return true if property (or sub- properties) contain a weak UObject reference, false otherwise
	 */
	bool ContainsWeakObjectReference() const
	{
		TArray<const FStructProperty*> EncounteredStructProps;
		return ContainsObjectReference(EncounteredStructProps, EPropertyObjectReferenceType::Weak);
	}

	/**
	 * Returns true if this property, or in the case of e.g. array or struct properties any sub- property, contains a
	 * UObject reference that is marked CPF_NeedCtorLink (i.e. instanced keyword).
	 *
	 * @return true if property (or sub- properties) contain a FObjectProperty that is marked CPF_NeedCtorLink, false otherwise
	 */
	UE_FORCEINLINE_HINT bool ContainsInstancedObjectProperty() const
	{
		return (PropertyFlags&(CPF_ContainsInstancedReference | CPF_InstancedReference)) != 0;
	}

	/**
	 * Emits tokens used by realtime garbage collection code to passed in ReferenceTokenStream. The offset emitted is relative
	 * to the passed in BaseOffset which is used by e.g. arrays of structs.
	 */
	COREUOBJECT_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath);

    // @TODO: Surely this can have an int32 overflow. This should probably return size_t. Just
    // need to audit all callers to make such a change.
	UE_FORCEINLINE_HINT int32 GetSize() const
	{
		return ArrayDim * GetElementSize();
	}
	COREUOBJECT_API bool ShouldSerializeValue( FArchive& Ar ) const;

	COREUOBJECT_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const;

	/**
	 * Determines whether this property value is eligible for copying when duplicating an object
	 * 
	 * @return	true if this property value should be copied into the duplicate object
	 */
	bool ShouldDuplicateValue() const
	{
		return ShouldPort() && GetOwnerClass() != UObject::StaticClass();
	}

	/**
	 * Restores this property and its owned properties from the type name.
	 *
	 * @return true if this property loaded from the type name and is in a valid and usable state.
	 */
	COREUOBJECT_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr);

	/**
	 * Saves the type name of this property and its owned properties.
	 */
	COREUOBJECT_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const;

	/**
	 * Returns true if the type of this property matches the type name.
	 */
	COREUOBJECT_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const;

	/**
	 * Returns the first FProperty in this property's Outer chain that does not have a FProperty for an Outer
	 */
	FProperty* GetOwnerProperty()
	{
		FProperty* Result = this;
		for (FProperty* PropBase = GetOwner<FProperty>(); PropBase; PropBase = PropBase->GetOwner<FProperty>())
		{
			Result = PropBase;
		}
		return Result;
	}

	const FProperty* GetOwnerProperty() const
	{
		const FProperty* Result = this;
		for (const FProperty* PropBase = GetOwner<FProperty>(); PropBase; PropBase = PropBase->GetOwner<FProperty>())
		{
			Result = PropBase;
		}
		return Result;
	}

	/**
	 * Returns this property's propertyflags
	 */
	UE_FORCEINLINE_HINT EPropertyFlags GetPropertyFlags() const
	{
		return PropertyFlags;
	}
	UE_FORCEINLINE_HINT void SetPropertyFlags( EPropertyFlags NewFlags )
	{
		PropertyFlags |= NewFlags;
	}
	UE_FORCEINLINE_HINT void ClearPropertyFlags( EPropertyFlags NewFlags )
	{
		PropertyFlags &= ~NewFlags;
	}
	/**
	 * Used to safely check whether any of the passed in flags are set.
	 * 
	 * @param FlagsToCheck	Object flags to check for.
	 *
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	UE_FORCEINLINE_HINT bool HasAnyPropertyFlags( uint64 FlagsToCheck ) const
	{
		return (PropertyFlags & FlagsToCheck) != 0 || FlagsToCheck == CPF_AllFlags;
	}
	/**
	 * Used to safely check whether all of the passed in flags are set.
	 * 
	 * @param FlagsToCheck	Object flags to check for
	 *
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	UE_FORCEINLINE_HINT bool HasAllPropertyFlags( uint64 FlagsToCheck ) const
	{
		return ((PropertyFlags & FlagsToCheck) == FlagsToCheck);
	}

	/**
	 * Returns the replication owner, which is the property itself, or NULL if this isn't important for replication.
	 * It is relevant if the property is a net relevant and not being run in the editor
	 */
	UE_FORCEINLINE_HINT FProperty* GetRepOwner()
	{
		return (!GIsEditor && ((PropertyFlags & CPF_Net) != 0)) ? this : NULL;
	}

	/**
	 * Editor-only properties are those that only are used with the editor is present or cannot be removed from serialisation.
	 * Editor-only properties include: EditorOnly properties
	 * Properties that cannot be removed from serialisation are:
	 *		Boolean properties (may affect GCC_BITFIELD_MAGIC computation)
	 *		Native properties (native serialisation)
	 */
	UE_FORCEINLINE_HINT bool IsEditorOnlyProperty() const
	{
		return (PropertyFlags & CPF_DevelopmentAssets) != 0;
	}

	/** returns true, if Other is property of exactly the same type */
	COREUOBJECT_API virtual bool SameType(const FProperty* Other) const;

	ELifetimeCondition GetBlueprintReplicationCondition() const { return BlueprintReplicationCondition; }
	void SetBlueprintReplicationCondition(ELifetimeCondition InBlueprintReplicationCondition) { BlueprintReplicationCondition = InBlueprintReplicationCondition; }

	/**
	 * Returns whether this type has a special state for an unset TOptional meaning the size TOptional<T> and T are the same.  
	 * Properties must implement this function explicitly even if they do not have such a state.
	 * @see Optional.h - HasIntrusiveUnsetOptionalState
	 * @see FOptionalProperty
	 */
	virtual bool HasIntrusiveUnsetOptionalState() const PURE_VIRTUAL(FProperty::HasIntrusiveUnsetOptionalState, return false;)

	/**
	 * Initialize the value at the given address to an unset TOptional using an intrusive state rather than a trailing boolean.
	 * @see TOptional::TOptional
	 * @see Constructor taking FIntrusiveUnsetOptionalState
	 */
	COREUOBJECT_API virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const;

	/**
	 * Returns whether an optional value of this inner type is unset. Only valid to call if HasIntrusiveUnsetOptionalState returns true.
	 * Equivalent to TOptional<T>::IsSet()
	 * @see operator==(FIntrusiveUnsetOptionalState)
	 * 
	 * @param Data Address of value to inspect, already offset.
	 * @return true if the value is unset 
	 */
	COREUOBJECT_API virtual bool IsIntrusiveOptionalValueSet(const void* Data) const;

	/**
	 * Set the value to it's special unset state. Equivalent to TOptional<T>::Reset. Only valid to call if HasIntrusiveUnsetOptionalState returns true.
	 * @see operator=(FIntrusiveUnsetOptionalState)
	 * 
	 * @param Data Address of the alue, already offset.
	 */
	COREUOBJECT_API virtual void ClearIntrusiveOptionalValue(void* Data) const;

	/**
	 * For properties returning true from HasIntrusiveUnsetOptionalState which also contain object references, 
	 * emit information for the garbage collector to safely gather the references from the value whether the 
	 * optional value is set or unset.
	 */
	COREUOBJECT_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath);
};


class FPropertyHelpers
{
public:
	// @param Out Appended to
	static COREUOBJECT_API const TCHAR* ReadToken( const TCHAR* Buffer, FString& Out, bool DottedNames = false);
	static COREUOBJECT_API const TCHAR* ReadToken( const TCHAR* Buffer, FStringBuilderBase& Out, bool DottedNames = false);

	/**
	 * Parse a token from the given buffer until we find the given terminating character, or we run out of buffer/hit the given limit.
	 * @note This will also parse and unescape any quoted strings found prior to the terminating character, 
	 *       however a terminating character within a quoted string is not considered as a termination point.
	 * 
	 * @param Buffer		The buffer to parse from.
	 * @param Out			The value to append to on success.
	 * @param Terminator	The token terminator to stop at.
	 * @param MaxTokenLen	Optional limit; will fail if we don't find the terminator within this limit.
	 * 
	 * @return The terminating character location in the buffer, or null if parsing failed.
	 */
	static COREUOBJECT_API const TCHAR* ReadTerminatedToken(const TCHAR* Buffer, FString& Out, const TCHAR Terminator, const int32* MaxTokenLen = nullptr);
	static COREUOBJECT_API const TCHAR* ReadTerminatedToken(const TCHAR* Buffer, FStringBuilderBase& Out, const TCHAR Terminator, const int32* MaxTokenLen = nullptr);
};

namespace UEProperty_Private
{
	/** FProperty methods FOR INTERNAL USE ONLY -- only authorized users should be making use of this. -- DO NOT USE! */
	class FProperty_DoNotUse
	{
	public:
		/** 
		 * To facilitate runtime binding with native C++ data-members, we need 
		 * a way of updating a property's generated offset.
		 * This is needed for pre-generated properties, which are then loaded 
		 * later, and fixed up to point at explicitly mapped C++ data-members.
		 * 
		 * Explicitly exposed for this singular case -- DO NOT USE otherwise.
		 */
		static void Unsafe_AlterOffset(FProperty& Property, const int32 OffsetOverride)
		{
			Property.SetOffset_Internal(OffsetOverride);
		}
	};
}

/** reference to a property and optional array index used in property text import to detect duplicate references */
struct FDefinedProperty
{
    FProperty* Property;
    int32 Index;
    bool operator== (const FDefinedProperty& Other) const
    {
        return (Property == Other.Property && Index == Other.Index);
    }
};

/**
 * Creates a temporary object that represents the default constructed value of a FProperty
 */
class FDefaultConstructedPropertyElement
{
public:
	FDefaultConstructedPropertyElement() = default;
	explicit FDefaultConstructedPropertyElement(const FProperty* InProp)
		: Obj(FMemory::Malloc(InProp->GetSize(), InProp->GetMinAlignment()), [InProp](void* Object)
			{
				InProp->DestroyValue(Object);
				FMemory::Free(Object);
			})
	{
		InProp->InitializeValue(Obj.Get());
	}

	void* GetObjAddress() const
	{
		return Obj.Get();
	}

private:
	TSharedPtr<void> Obj;
};


/*-----------------------------------------------------------------------------
	TProperty.
-----------------------------------------------------------------------------*/


template<typename InTCppType>
class TPropertyTypeFundamentals
{
public:
	/** Type of the CPP property **/
	typedef InTCppType TCppType;
	enum
	{
		CPPSize = sizeof(TCppType),
		CPPAlignment = alignof(TCppType)
	};

	static UE_FORCEINLINE_HINT TCHAR const* GetTypeName()
	{
		return TNameOf<TCppType>::GetName();
	}

	/** Convert the address of a value of the property to the proper type */
	static UE_FORCEINLINE_HINT TCppType const* GetPropertyValuePtr(void const* A)
	{
		return (TCppType const*)A;
	}
	/** Convert the address of a value of the property to the proper type */
	static UE_FORCEINLINE_HINT TCppType* GetPropertyValuePtr(void* A)
	{
		return (TCppType*)A;
	}
	/** Get the value of the property from an address */
	static UE_FORCEINLINE_HINT TCppType const& GetPropertyValue(void const* A)
	{
		return *GetPropertyValuePtr(A);
	}
	/** Get the default value of the cpp type, just the default constructor, which works even for things like in32 */
	static UE_FORCEINLINE_HINT TCppType GetDefaultPropertyValue()
	{
		return TCppType();
	}
	/** Get the value of the property from an address, unless it is NULL, then return the default value */
	static UE_FORCEINLINE_HINT TCppType GetOptionalPropertyValue(void const* B)
	{
		return B ? GetPropertyValue(B) : GetDefaultPropertyValue();
	}
	/** Set the value of a property at an address */
	static UE_FORCEINLINE_HINT void SetPropertyValue(void* A, TCppType const& Value)
	{
		*GetPropertyValuePtr(A) = Value;
	}
	/** Initialize the value of a property at an address, this assumes over uninitialized memory */
	static UE_FORCEINLINE_HINT TCppType* InitializePropertyValue(void* A)
	{
		return new (A) TCppType();
	}
	/** Destroy the value of a property at an address */
	static UE_FORCEINLINE_HINT void DestroyPropertyValue(void* A)
	{
		GetPropertyValuePtr(A)->~TCppType();
	}

	static UE_FORCEINLINE_HINT bool HasIntrusiveUnsetOptionalState()
	{
		return ::HasIntrusiveUnsetOptionalState<TCppType>();
	}

	static UE_FORCEINLINE_HINT void InitializeIntrusiveUnsetOptionalValue(void* Data) 
	{
		new(Data) TOptional<TCppType>();
	}

	static UE_FORCEINLINE_HINT bool IsIntrusiveOptionalValueSet(const void* A)
	{
		return reinterpret_cast<const TOptional<TCppType>*>(A)->IsSet();
	}

	static UE_FORCEINLINE_HINT void ClearIntrusiveOptionalValue(void* A)
	{
		reinterpret_cast<TOptional<TCppType>*>(A)->Reset();
	}

protected:
	/** Get the property flags corresponding to this C++ type, from the C++ type traits system */
	static inline EPropertyFlags GetComputedFlagsPropertyFlags()
	{
		return 
			(TIsPODType<TCppType>::Value ? CPF_IsPlainOldData : CPF_None) 
			| (std::is_trivially_destructible_v<TCppType> ? CPF_NoDestructor : CPF_None) 
			| (TIsZeroConstructType<TCppType>::Value ? CPF_ZeroConstructor : CPF_None)
			| (TModels_V<CGetTypeHashable, TCppType> ? CPF_HasGetValueTypeHash : CPF_None);

	}
};


template<typename InTCppType, class TInPropertyBaseClass>
class TProperty : public TInPropertyBaseClass, public TPropertyTypeFundamentals<InTCppType>
{
public:

	typedef InTCppType TCppType;
	typedef TInPropertyBaseClass Super;
	typedef TPropertyTypeFundamentals<InTCppType> TTypeFundamentals;

	TProperty(EInternal InInernal, FFieldClass* InClass)
		: Super(EC_InternalUseOnlyConstructor, InClass)
	{
	}

	TProperty(FFieldVariant InOwner, const FName& InName)
		: Super(InOwner, InName)
	{
		this->SetElementSize(TTypeFundamentals::CPPSize);
	}
	UE_DEPRECATED(5.8, "TProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	TProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : TProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	template <typename PropertyParamsType>
	TProperty(FFieldVariant InOwner, PropertyParamsType& Prop)
		: Super(InOwner, Prop, TTypeFundamentals::GetComputedFlagsPropertyFlags())
	{
		this->SetElementSize(TTypeFundamentals::CPPSize);
	}

#if UE_WITH_CONSTINIT_UOBJECT
	template<typename... Ts>
	explicit consteval TProperty(
		// Args must be the common parameters to FProperty's constructor followed by properties unique to TInPropertyBaseClass
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		Ts&&... AdditionalArgs
	)
		: Super(InBaseParams.SetElementSize(TTypeFundamentals::CPPSize), Forward<Ts>(AdditionalArgs)...)
	{
	}
#endif

public:

#if WITH_EDITORONLY_DATA
	explicit TProperty(UField* InField)
		: Super(InField)
	{
		this->SetElementSize(TTypeFundamentals::CPPSize);
	}
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	virtual FString GetCPPType( FString* ExtendedTypeText=NULL, uint32 CPPExportFlags=0 ) const override
	{
		return FString(TTypeFundamentals::GetTypeName());
	}
	// End of UHT interface

	// FProperty interface.
	virtual int32 GetMinAlignment() const override
	{
		return TTypeFundamentals::CPPAlignment;
	}
	virtual void LinkInternal(FArchive& Ar) override
	{
		this->SetElementSize(TTypeFundamentals::CPPSize);
		this->PropertyFlags |= TTypeFundamentals::GetComputedFlagsPropertyFlags();

	}
	virtual void CopyValuesInternal( void* Dest, void const* Src, int32 Count ) const override
	{
		for (int32 Index = 0; Index < Count; Index++)
		{
			TTypeFundamentals::GetPropertyValuePtr(Dest)[Index] = TTypeFundamentals::GetPropertyValuePtr(Src)[Index];
		}
	}
	virtual void ClearValueInternal( void* Data ) const override
	{
		TTypeFundamentals::SetPropertyValue(Data, TTypeFundamentals::GetDefaultPropertyValue());
	}
	virtual void InitializeValueInternal( void* Dest ) const override
	{
		for (int32 i = 0; i < this->ArrayDim; ++i)
		{
			TTypeFundamentals::InitializePropertyValue((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize()));
		}
	}
	virtual void DestroyValueInternal( void* Dest ) const override
	{
		for (int32 i = 0; i < this->ArrayDim; ++i)
		{
			TTypeFundamentals::DestroyPropertyValue((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize()));
		}
	}

	/** Convert the address of a container to the address of the property value, in the proper type */
	UE_FORCEINLINE_HINT TCppType const* GetPropertyValuePtr_InContainer(void const* A, int32 ArrayIndex = 0) const
	{
		return TTypeFundamentals::GetPropertyValuePtr(Super::template ContainerPtrToValuePtr<void>(A, ArrayIndex));
	}
	/** Convert the address of a container to the address of the property value, in the proper type */
	UE_FORCEINLINE_HINT TCppType* GetPropertyValuePtr_InContainer(void* A, int32 ArrayIndex = 0) const
	{
		return TTypeFundamentals::GetPropertyValuePtr(Super::template ContainerPtrToValuePtr<void>(A, ArrayIndex));
	}
	/** Get the value of the property from a container address */
	UE_FORCEINLINE_HINT TCppType const& GetPropertyValue_InContainer(void const* A, int32 ArrayIndex = 0) const
	{
		return *GetPropertyValuePtr_InContainer(A, ArrayIndex);
	}
	/** Get the value of the property from a container address, unless it is NULL, then return the default value */
	UE_FORCEINLINE_HINT TCppType GetOptionalPropertyValue_InContainer(void const* B, int32 ArrayIndex = 0) const
	{
		return B ? GetPropertyValue_InContainer(B, ArrayIndex) : TTypeFundamentals::GetDefaultPropertyValue();
	}
	/** Set the value of a property in a container */
	UE_FORCEINLINE_HINT void SetPropertyValue_InContainer(void* A, TCppType const& Value, int32 ArrayIndex = 0) const
	{
		*GetPropertyValuePtr_InContainer(A, ArrayIndex) = Value;
	}

	UE_FORCEINLINE_HINT void SetValue_InContainer(void* OutContainer, const TCppType& InValue) const
	{
		TInPropertyBaseClass::SetValue_InContainer(OutContainer, &InValue);
	}

	UE_FORCEINLINE_HINT void GetValue_InContainer(void const* InContainer, TCppType* OutValue) const
	{
		TInPropertyBaseClass::GetValue_InContainer(InContainer, OutValue);
	}

	virtual bool HasIntrusiveUnsetOptionalState() const override
	{
		return TTypeFundamentals::HasIntrusiveUnsetOptionalState();
	}

	virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override
	{
		TTypeFundamentals::InitializeIntrusiveUnsetOptionalValue(Data);
	}
	
	virtual bool IsIntrusiveOptionalValueSet(const void* Data) const override
	{
		return TTypeFundamentals::IsIntrusiveOptionalValueSet(Data);
	}

	virtual void ClearIntrusiveOptionalValue(void* Data) const override
	{
		TTypeFundamentals::ClearIntrusiveOptionalValue(Data);
	}
	// End of FProperty interface
};

template<typename InTCppType, class TInPropertyBaseClass>
class TProperty_WithEqualityAndSerializer : public TProperty<InTCppType, TInPropertyBaseClass>
{

public:
	typedef TProperty<InTCppType, TInPropertyBaseClass> Super;
	typedef InTCppType TCppType;
	typedef typename Super::TTypeFundamentals TTypeFundamentals;

	TProperty_WithEqualityAndSerializer(EInternal InInernal, FFieldClass* InClass)
		: Super(EC_InternalUseOnlyConstructor, InClass)
	{
	}

	TProperty_WithEqualityAndSerializer(FFieldVariant InOwner, const FName& InName)
		: Super(InOwner, InName)
	{
	}
	UE_DEPRECATED(5.8, "TProperty_WithEqualityAndSerializer constructor with InObjectFlags is deprecated, remove that parameter.")
	TProperty_WithEqualityAndSerializer(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : TProperty_WithEqualityAndSerializer(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	TProperty_WithEqualityAndSerializer(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop)
		: Super(InOwner, Prop)
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	template<typename... Ts>
	explicit consteval TProperty_WithEqualityAndSerializer(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		Ts&&... Args
	)
		: Super(InBaseParams, Forward<Ts>(Args)...)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit TProperty_WithEqualityAndSerializer(UField* InField)
		: Super(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface.
	virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags = 0) const override
	{
		// `B` can be null, but we want to avoid the cost of copying a non-null `B` if we were to just use
		// `TTypeFundamentals::GetOptionalPropertyValue(B)`. It is important that we branch on `B` here
		// rather than using a ternary between the two variants of B (its own value or the default), because
		// `TTypeFundamentals::GetDefaultPropertyValue()` returns a non-ref, and thus doing a ternary would
		// also result in a copy of `B` which we are trying to avoid!
		if (B)
		{
			return TTypeFundamentals::GetPropertyValue(A) == TTypeFundamentals::GetPropertyValue(B);
		}
		else
		{
			return TTypeFundamentals::GetPropertyValue(A) == TTypeFundamentals::GetDefaultPropertyValue();
		}
	}

	virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override
	{
		Slot << *TTypeFundamentals::GetPropertyValuePtr(Value);
	}
	// End of FProperty interface

};

class FNumericProperty : public FProperty
{
	DECLARE_FIELD_API(FNumericProperty, FProperty, CASTCLASS_FNumericProperty, UE_API)

	UE_API FNumericProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FNumericProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FNumericProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FNumericProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FNumericProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FNumericProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FNumericProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// FProperty interface.
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual void ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const override;
	// End of FProperty interface

	// FNumericProperty interface.

	/** Return true if this property is for a floating point number **/
	UE_API virtual bool IsFloatingPoint() const;

	/** Return true if this property is for a integral or enum type **/
	UE_API virtual bool IsInteger() const;

	template <typename T>
	bool CanHoldValue(T Value) const
	{
		if (std::is_floating_point_v<T>)
		{
			//@TODO: FLOATPRECISION: This feels wrong, it might be losing precision before it tests to see if it's going to lose precision...
			return CanHoldDoubleValueInternal((double)Value);
		}
		else if (std::is_signed_v<T>)
		{
			return CanHoldSignedValueInternal(Value);
		}
		else
		{
			return CanHoldUnsignedValueInternal(Value);
		}
	}

	/** Return true if this property is a FByteProperty with a non-null Enum **/
	UE_FORCEINLINE_HINT bool IsEnum() const
	{
		return !!GetIntPropertyEnum();
	}

	/** Return the UEnum if this property is a FByteProperty with a non-null Enum **/
	UE_API virtual UEnum* GetIntPropertyEnum() const;

	/** 
	 * Set the value of an unsigned integral property type
	 * @param Data - pointer to property data to set
	 * @param Value - Value to set data to
	**/
	UE_API virtual void SetIntPropertyValue(void* Data, uint64 Value) const;

	/** 
	 * Set the value of a signed integral property type
	 * @param Data - pointer to property data to set
	 * @param Value - Value to set data to
	**/
	UE_API virtual void SetIntPropertyValue(void* Data, int64 Value) const;

	/** 
	 * Set the value of a floating point property type
	 * @param Data - pointer to property data to set
	 * @param Value - Value to set data to
	**/
	UE_API virtual void SetFloatingPointPropertyValue(void* Data, double Value) const;

	/** 
	 * Set the value of any numeric type from a string point
	 * @param Data - pointer to property data to set
	 * @param Value - Value (as a string) to set 
	 * CAUTION: This routine does not do enum name conversion
	**/
	UE_API virtual void SetNumericPropertyValueFromString(void* Data, TCHAR const* Value) const;
	UE_API virtual void SetNumericPropertyValueFromString_InContainer(void* Container, TCHAR const* Value) const;

	/** 
	 * Gets the value of a signed integral property type
	 * @param Data - pointer to property data to get
	 * @return Data as a signed int
	**/
	UE_API virtual int64 GetSignedIntPropertyValue(void const* Data) const;
	UE_API virtual int64 GetSignedIntPropertyValue_InContainer(void const* Container) const;

	/** 
	 * Gets the value of an unsigned integral property type
	 * @param Data - pointer to property data to get
	 * @return Data as an unsigned int
	**/
	UE_API virtual uint64 GetUnsignedIntPropertyValue(void const* Data) const;
	UE_API virtual uint64 GetUnsignedIntPropertyValue_InContainer(void const* Container) const;

	/** 
	 * Gets the value of an floating point property type
	 * @param Data - pointer to property data to get
	 * @return Data as a double
	**/
	UE_API virtual double GetFloatingPointPropertyValue(void const* Data) const;

	/** 
	 * Get the value of any numeric type and return it as a string
	 * @param Data - pointer to property data to get
	 * @return Data as a string
	 * CAUTION: This routine does not do enum name conversion
	**/
	UE_API virtual FString GetNumericPropertyValueToString(void const* Data) const;
	UE_API virtual FString GetNumericPropertyValueToString_InContainer(void const* Container) const;
	// End of FNumericProperty interface

	UE_API static int64 ReadEnumAsInt64(FStructuredArchive::FSlot Slot, const UStruct* DefaultsStruct, const FPropertyTag& Tag);

private:
	virtual bool CanHoldDoubleValueInternal  (double Value) const PURE_VIRTUAL(FNumericProperty::CanHoldDoubleValueInternal,   return false;);
	virtual bool CanHoldSignedValueInternal  (int64  Value) const PURE_VIRTUAL(FNumericProperty::CanHoldSignedValueInternal,   return false;);
	virtual bool CanHoldUnsignedValueInternal(uint64 Value) const PURE_VIRTUAL(FNumericProperty::CanHoldUnsignedValueInternal, return false;);
};

template<typename InTCppType>
class TProperty_Numeric : public TProperty_WithEqualityAndSerializer<InTCppType, FNumericProperty>
{
public:
	typedef TProperty_WithEqualityAndSerializer<InTCppType, FNumericProperty> Super;
	typedef InTCppType TCppType;
	typedef typename Super::TTypeFundamentals TTypeFundamentals;

	TProperty_Numeric(EInternal InInernal, FFieldClass* InClass)
		: Super(EC_InternalUseOnlyConstructor, InClass)
	{
	}

	TProperty_Numeric(FFieldVariant InOwner, const FName& InName)
		: Super(InOwner, InName)
	{
	}
	UE_DEPRECATED(5.8, "TProperty_Numeric constructor with InObjectFlags is deprecated, remove that parameter.")
	TProperty_Numeric(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : TProperty_Numeric(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	TProperty_Numeric(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop)
		: Super(InOwner, Prop)
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval TProperty_Numeric(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit TProperty_Numeric(UField* InField)
		: Super(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
	virtual uint32 GetValueTypeHashInternal(const void* Src) const override
	{
		return GetTypeHash(*(const InTCppType*)Src);
	}

protected:
	template <typename OldNumericType>
	UE_FORCEINLINE_HINT void ConvertFromArithmeticValue(FStructuredArchive::FSlot Slot, void* Obj, const FPropertyTag& Tag) const
	{
		TConvertAndSet<OldNumericType, TCppType>(*this, Slot, Obj, Tag);
	}

private:
	template <typename FromType, typename ToType>
	struct TConvertAndSet
	{
		TConvertAndSet(const TProperty_Numeric& Property, FStructuredArchive::FSlot Slot, void* Obj, const FPropertyTag& Tag)
		{
			FromType OldValue;
			Slot << OldValue;
			ToType NewValue = (ToType)OldValue;
			Property.SetPropertyValue_InContainer(Obj, NewValue, Tag.ArrayIndex);

			UE_CLOG(
				((std::is_signed_v<FromType> || std::is_floating_point_v<FromType>) && (!std::is_signed_v<ToType> && !std::is_floating_point_v<ToType>) && OldValue < 0) || ((FromType)NewValue != OldValue),
				LogClass,
				Warning,
				TEXT("Potential data loss during conversion of integer property %s of %s - was (%s) now (%s) - for package: %s"),
				*Property.GetName(),
				*Slot.GetUnderlyingArchive().GetArchiveName(),
				*LexToString(OldValue),
				*LexToString(NewValue),
				*Slot.GetUnderlyingArchive().GetArchiveName()
			);
		}
	};

	template <typename SameType>
	struct TConvertAndSet<SameType, SameType>
	{
		inline TConvertAndSet(const TProperty_Numeric& Property, FStructuredArchive::FSlot Slot, void* Obj, const FPropertyTag& Tag)
		{
			SameType Value;
			Slot << Value;
			Property.SetPropertyValue_InContainer(Obj, Value, Tag.ArrayIndex);
		}
	};

public:
	virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override
	{
		if (const EName* TagType = Tag.Type.ToEName(); TagType && Tag.Type.GetNumber() == NAME_NO_NUMBER_INTERNAL)
		{
			PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
			switch (*TagType)
			{
			case NAME_Int8Property:
				ConvertFromArithmeticValue<int8>(Slot, Data, Tag);
				return std::is_same_v<TCppType, int8> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_Int16Property:
				ConvertFromArithmeticValue<int16>(Slot, Data, Tag);
				return std::is_same_v<TCppType, int16> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_IntProperty:
				ConvertFromArithmeticValue<int32>(Slot, Data, Tag);
				return std::is_same_v<TCppType, int32> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_Int64Property:
				ConvertFromArithmeticValue<int64>(Slot, Data, Tag);
				return std::is_same_v<TCppType, int64> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_ByteProperty:
				if (Tag.GetType().GetParameterCount() >= 1)
				{
					int64 PreviousValue = this->ReadEnumAsInt64(Slot, DefaultsStruct, Tag);
					this->SetPropertyValue_InContainer(Data, (TCppType)PreviousValue, Tag.ArrayIndex);
					return EConvertFromTypeResult::Converted;
				}
				ConvertFromArithmeticValue<int8>(Slot, Data, Tag);
				return std::is_same_v<TCppType, uint8> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_EnumProperty:
			{
				int64 PreviousValue = this->ReadEnumAsInt64(Slot, DefaultsStruct, Tag);
				this->SetPropertyValue_InContainer(Data, (TCppType)PreviousValue, Tag.ArrayIndex);
				return EConvertFromTypeResult::Converted;
			}

			case NAME_UInt16Property:
				ConvertFromArithmeticValue<uint16>(Slot, Data, Tag);
				return std::is_same_v<TCppType, uint16> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_UInt32Property:
				ConvertFromArithmeticValue<uint32>(Slot, Data, Tag);
				return std::is_same_v<TCppType, uint32> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_UInt64Property:
				ConvertFromArithmeticValue<uint64>(Slot, Data, Tag);
				return std::is_same_v<TCppType, uint64> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_FloatProperty:
				ConvertFromArithmeticValue<float>(Slot, Data, Tag);
				return std::is_same_v<TCppType, float> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_DoubleProperty:
				ConvertFromArithmeticValue<double>(Slot, Data, Tag);
				return std::is_same_v<TCppType, double> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			case NAME_BoolProperty:
				this->SetPropertyValue_InContainer(Data, (TCppType)Tag.BoolVal, Tag.ArrayIndex);
				return std::is_same_v<TCppType, bool> ? EConvertFromTypeResult::Serialized : EConvertFromTypeResult::Converted;

			default:
				// We didn't convert it
				break;
			}
			PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		}

		return EConvertFromTypeResult::UseSerializeItem;
	}
	// End of FProperty interface

	// FNumericProperty interface.
	virtual bool IsFloatingPoint() const override
	{
		return std::is_floating_point_v<TCppType>;
	}
	virtual bool IsInteger() const override
	{
		return std::is_integral_v<TCppType>;
	}
	virtual void SetIntPropertyValue(void* Data, uint64 Value) const override
	{
		check(std::is_integral_v<TCppType>);
		TTypeFundamentals::SetPropertyValue(Data, (TCppType)Value);
	}
	virtual void SetIntPropertyValue(void* Data, int64 Value) const override
	{
		check(std::is_integral_v<TCppType>);
		TTypeFundamentals::SetPropertyValue(Data, (TCppType)Value);
	}
	virtual void SetFloatingPointPropertyValue(void* Data, double Value) const override
	{
		check(std::is_floating_point_v<TCppType>);
		TTypeFundamentals::SetPropertyValue(Data, (TCppType)Value);
	}
	virtual void SetNumericPropertyValueFromString(void* Data, TCHAR const* Value) const override
	{
		LexFromString(*TTypeFundamentals::GetPropertyValuePtr(Data), Value);
	}
	virtual void SetNumericPropertyValueFromString_InContainer(void* Container, TCHAR const* Value) const override
	{
		TCppType LocalValue{};
		LexFromString(LocalValue, Value);
		FNumericProperty::SetValue_InContainer(Container, &LocalValue);
	}
	virtual FString GetNumericPropertyValueToString(void const* Data) const override
	{
		return LexToString(TTypeFundamentals::GetPropertyValue(Data));
	}
	virtual FString GetNumericPropertyValueToString_InContainer(void const* Container) const override
	{
		TCppType LocalValue{};
		FNumericProperty::GetValue_InContainer(Container, &LocalValue);
		return LexToString(LocalValue);
	}
	virtual int64 GetSignedIntPropertyValue(void const* Data) const override
	{
		check(std::is_integral_v<TCppType>);
		return (int64)TTypeFundamentals::GetPropertyValue(Data);
	}
	virtual int64 GetSignedIntPropertyValue_InContainer(void const* Container) const override
	{
		check(std::is_integral_v<TCppType>);
		TCppType LocalValue{};
		FNumericProperty::GetValue_InContainer(Container, &LocalValue);
		return (int64)LocalValue;
	}
	virtual uint64 GetUnsignedIntPropertyValue(void const* Data) const override
	{
		check(std::is_integral_v<TCppType>);
		return (uint64)TTypeFundamentals::GetPropertyValue(Data);
	}
	virtual uint64 GetUnsignedIntPropertyValue_InContainer(void const* Container) const override
	{
		check(std::is_integral_v<TCppType>);
		TCppType LocalValue{};
		FNumericProperty::GetValue_InContainer(Container, &LocalValue);
		return (uint64)LocalValue;
	}
	virtual double GetFloatingPointPropertyValue(void const* Data) const override
	{
		check(std::is_floating_point_v<TCppType>);
		return (double)TTypeFundamentals::GetPropertyValue(Data);
	}
	// End of FNumericProperty interface

private:
	virtual bool CanHoldDoubleValueInternal(double Value) const
	{
		return (double)(InTCppType)Value == Value;
	}

	virtual bool CanHoldSignedValueInternal(int64 Value) const
	{
		return (int64)(InTCppType)Value == Value;
	}

	virtual bool CanHoldUnsignedValueInternal(uint64 Value) const
	{
		return (uint64)(InTCppType)Value == Value;
	}
};

/*-----------------------------------------------------------------------------
	FByteProperty.
-----------------------------------------------------------------------------*/

//
// Describes an unsigned byte value or 255-value enumeration variable.
//
class FByteProperty : public TProperty_Numeric<uint8>
{
	DECLARE_FIELD_API(FByteProperty, TProperty_Numeric<uint8>, CASTCLASS_FByteProperty, UE_API)

	// Variables.
	TObjectPtr<UEnum> Enum;

	UE_API FByteProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FByteProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FByteProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FByteProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FByteProperty(FFieldVariant InOwner, const UECodeGen_Private::FBytePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FByteProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UEnum> InEnum)
		: Super(InBaseParams)
		, Enum(InEnum)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FByteProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface.
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// UHT interface
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText=NULL, uint32 CPPExportFlags=0 ) const override;
	// End of UHT interface

	// FProperty interface.
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
protected:
	UE_API virtual void ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	// End of FProperty interface

	// FNumericProperty interface.
	UE_API virtual UEnum* GetIntPropertyEnum() const override;
	// End of FNumericProperty interface

	// Returns the number of bits required by NetSerializeItem to encode this property (may be fewer than 8 if this byte represents an enum)
	UE_API uint64 GetMaxNetSerializeBits() const;
};

/*-----------------------------------------------------------------------------
	FInt8Property.
-----------------------------------------------------------------------------*/

//
// Describes a 8-bit signed integer variable.
//
class FInt8Property : public TProperty_Numeric<int8>
{
	DECLARE_FIELD_API(FInt8Property, TProperty_Numeric<int8>, CASTCLASS_FInt8Property, UE_API)

	UE_API FInt8Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FInt8Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FInt8Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FInt8Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FInt8Property(FFieldVariant InOwner, const UECodeGen_Private::FInt8PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FInt8Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FInt8Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};

/*-----------------------------------------------------------------------------
	FInt16Property.
-----------------------------------------------------------------------------*/

//
// Describes a 16-bit signed integer variable.
//
class FInt16Property : public TProperty_Numeric<int16>
{
	DECLARE_FIELD_API(FInt16Property, TProperty_Numeric<int16>, CASTCLASS_FInt16Property, UE_API)

	UE_API FInt16Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FInt16Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FInt16Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FInt16Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FInt16Property(FFieldVariant InOwner, const UECodeGen_Private::FInt16PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FInt16Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FInt16Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};


/*-----------------------------------------------------------------------------
	FIntProperty.
-----------------------------------------------------------------------------*/

//
// Describes a 32-bit signed integer variable.
//
class FIntProperty : public TProperty_Numeric<int32>
{
	DECLARE_FIELD_API(FIntProperty, TProperty_Numeric<int32>, CASTCLASS_FIntProperty, UE_API)

	UE_API FIntProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FIntProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FIntProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FIntProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FIntProperty(FFieldVariant InOwner, const UECodeGen_Private::FIntPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FIntProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FIntProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA
};

/*-----------------------------------------------------------------------------
	FInt64Property.
-----------------------------------------------------------------------------*/

//
// Describes a 64-bit signed integer variable.
//
class FInt64Property : public TProperty_Numeric<int64>
{
	DECLARE_FIELD_API(FInt64Property, TProperty_Numeric<int64>, CASTCLASS_FInt64Property, UE_API)

	UE_API FInt64Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FInt64Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FInt64Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FInt64Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FInt64Property(FFieldVariant InOwner, const UECodeGen_Private::FInt64PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FInt64Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FInt64Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};

/*-----------------------------------------------------------------------------
	FUInt16Property.
-----------------------------------------------------------------------------*/

//
// Describes a 16-bit unsigned integer variable.
//
class FUInt16Property : public TProperty_Numeric<uint16>
{
	DECLARE_FIELD_API(FUInt16Property, TProperty_Numeric<uint16>, CASTCLASS_FUInt16Property, UE_API)

	UE_API FUInt16Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FUInt16Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FUInt16Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FUInt16Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FUInt16Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt16PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FUInt16Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FUInt16Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};

/*-----------------------------------------------------------------------------
	FUInt32Property.
-----------------------------------------------------------------------------*/

//
// Describes a 32-bit unsigned integer variable.
//
class FUInt32Property : public TProperty_Numeric<uint32>
{
	DECLARE_FIELD_API(FUInt32Property, TProperty_Numeric<uint32>, CASTCLASS_FUInt32Property, UE_API)

	UE_API FUInt32Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FUInt32Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FUInt32Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FUInt32Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FUInt32Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt32PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FUInt32Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FUInt32Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};

/*-----------------------------------------------------------------------------
	FUInt64Property.
-----------------------------------------------------------------------------*/

//
// Describes a 64-bit unsigned integer variable.
//
class FUInt64Property : public TProperty_Numeric<uint64>
{
	DECLARE_FIELD_API(FUInt64Property, TProperty_Numeric<uint64>, CASTCLASS_FUInt64Property, UE_API)

	UE_API FUInt64Property(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FUInt64Property constructor with InObjectFlags is deprecated, remove that parameter.")
	FUInt64Property(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FUInt64Property(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FUInt64Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt64PropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FUInt64Property(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FUInt64Property(UField* InField);
#endif // WITH_EDITORONLY_DATA
};


/*-----------------------------------------------------------------------------
	FFloatProperty.
-----------------------------------------------------------------------------*/

//
// Describes an IEEE 32-bit floating point variable.
//
class FFloatProperty : public TProperty_Numeric<float>
{
	DECLARE_FIELD_API(FFloatProperty, TProperty_Numeric<float>, CASTCLASS_FFloatProperty, UE_API)

	UE_API FFloatProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FFloatProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FFloatProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FFloatProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FFloatProperty(FFieldVariant InOwner, const UECodeGen_Private::FFloatPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FFloatProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FFloatProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;

protected:

	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
};

/*-----------------------------------------------------------------------------
	FDoubleProperty.
-----------------------------------------------------------------------------*/

//
// Describes an IEEE 64-bit floating point variable.
//
class FDoubleProperty : public TProperty_Numeric<double>
{
	DECLARE_FIELD_API(FDoubleProperty, TProperty_Numeric<double>, CASTCLASS_FDoubleProperty, UE_API)

	UE_API FDoubleProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FDoubleProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FDoubleProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FDoubleProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FDoubleProperty(FFieldVariant InOwner, const UECodeGen_Private::FDoublePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FDoubleProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FDoubleProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;

protected:

	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
};

using FLargeWorldCoordinatesRealProperty = FDoubleProperty;

/*-----------------------------------------------------------------------------
	FBoolProperty.
-----------------------------------------------------------------------------*/

//
// Describes a single bit flag variable residing in a 32-bit unsigned double word.
//
class FBoolProperty : public FProperty
{
	DECLARE_FIELD_API(FBoolProperty, FProperty, CASTCLASS_FBoolProperty, UE_API)

	// Variables.
private:

	/** Size of the bitfield/bool property. Equal to ElementSize but used to check if the property has been properly initialized (0-8, where 0 means uninitialized). */
	uint8 FieldSize;
	/** Offset from the memeber variable to the byte of the property (0-7). */
	uint8 ByteOffset;
	/** Mask of the byte with the property value. */
	uint8 ByteMask;
	/** Mask of the field with the property value. Either equal to ByteMask or 255 in case of 'bool' type. */
	uint8 FieldMask;

#if UE_WITH_CONSTINIT_UOBJECT
	// CONSTINIT_UOBJECT_TODO: Size increase
	void (*SetBitFunc)(void*) = nullptr;
#endif

public:

	UE_API FBoolProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FBoolProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FBoolProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FBoolProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FBoolProperty(FFieldVariant InOwner, const UECodeGen_Private::FBoolPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FBoolProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		int32 InElementSize,
		bool bIsNativeBool,
		void (*InSetBitFunc)(void*)
	)
		: Super(InBaseParams.SetElementSize(InElementSize))
		, FieldSize(0) 
		, ByteOffset(0)
		, ByteMask(1)
		, FieldMask(bIsNativeBool ? 0xFF : 1)
		, SetBitFunc(InSetBitFunc)
	{
		PropertyFlags |= CPF_HasGetValueTypeHash;
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FBoolProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface.
	UE_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// UHT interface
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const override;
	// End of UHT interface

	// FProperty interface.
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual void CopyValuesInternal( void* Dest, void const* Src, int32 Count ) const override;
	UE_API virtual void ClearValueInternal( void* Data ) const override;
	UE_API virtual void InitializeValueInternal( void* Dest ) const override;
	UE_API virtual int32 GetMinAlignment() const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	// End of FProperty interface

protected:
	void DetermineBitfieldOffsetAndMask(uint32& Offset, uint32& BitMask, void (*SetBit)(void* Obj), SIZE_T SizeOf);
public:
	// Emulate the CPP type API, see TPropertyTypeFundamentals
	// this is incomplete as some operations make no sense for bitfields, for example they don't have a usable address
	typedef bool TCppType;
	inline bool GetPropertyValue(void const* A) const
	{
		check(FieldSize != 0);
		uint8* ByteValue = (uint8*)A + ByteOffset;
		return !!(*ByteValue & FieldMask);
	}
	UE_FORCEINLINE_HINT bool GetPropertyValue_InContainer(void const* A, int32 ArrayIndex = 0) const
	{
		return GetPropertyValue(ContainerPtrToValuePtr<void>(A, ArrayIndex));
	}
	static UE_FORCEINLINE_HINT bool GetDefaultPropertyValue()
	{
		return false;
	}
	UE_FORCEINLINE_HINT bool GetOptionalPropertyValue(void const* B) const
	{
		return B ? GetPropertyValue(B) : GetDefaultPropertyValue();
	}
	UE_FORCEINLINE_HINT bool GetOptionalPropertyValue_InContainer(void const* B, int32 ArrayIndex = 0) const
	{
		return B ? GetPropertyValue_InContainer(B, ArrayIndex) : GetDefaultPropertyValue();
	}
	inline void SetPropertyValue(void* A, bool Value) const
	{
		check(FieldSize != 0);
		uint8* ByteValue = (uint8*)A + ByteOffset;
		*ByteValue = ((*ByteValue) & ~FieldMask) | (Value ? ByteMask : 0);
	}
	UE_FORCEINLINE_HINT void SetPropertyValue_InContainer(void* A, bool Value, int32 ArrayIndex = 0) const
	{
		SetPropertyValue(ContainerPtrToValuePtr<void>(A, ArrayIndex), Value);
	}
	// End of the CPP type API

	/** 
	 * Sets the bitfield/bool type and size. 
	 * This function must be called before FBoolProperty can be used.
	 *
	 * @param InSize size of the bitfield/bool type.
	 * @param bIsNativeBool true if this property represents C++ bool type.
	 */
	UE_API void SetBoolSize( const uint32 InSize, const bool bIsNativeBool = false, const uint32 InBitMask = 0 );

	/**
	 * If the return value is true this FBoolProperty represents C++ bool type.
	 */
	UE_FORCEINLINE_HINT bool IsNativeBool() const
	{
		return FieldMask == 0xff;
	}

	/** Return the the mask that defines the relevant bit for this boolean, or 0xFF if IsNativeBool() is true */
	UE_FORCEINLINE_HINT uint8 GetFieldMask() const
	{
		return FieldMask;
	}

	/** Return the byte offset from this property's storage type to the byte that FieldMask applies to. Only valid if IsNativeBool() is true. */
	UE_FORCEINLINE_HINT uint8 GetByteOffset() const
	{
		return ByteOffset;
	}

	/** Return the FieldSize, for debugging purposes */
	FORCEINLINE uint8 GetBoolFieldSize() const
	{
		return FieldSize;
	}

	/** Return the ByteMask, for debugging purposes */
	FORCEINLINE uint8 GetByteMask() const
	{
		return ByteMask;
	}
	

	UE_API uint32 GetValueTypeHashInternal(const void* Src) const override;

	virtual bool HasIntrusiveUnsetOptionalState() const override 
	{ 
		return false;
	}
};

/*-----------------------------------------------------------------------------
	FObjectPropertyBase.
-----------------------------------------------------------------------------*/

//
// Describes a reference variable to another object which may be nil.
//
class FObjectPropertyBase : public FProperty
{
	DECLARE_FIELD_API(FObjectPropertyBase, FProperty, CASTCLASS_FObjectPropertyBase, UE_API)

public:

	// Variables.
	TObjectPtr<class UClass> PropertyClass;

	UE_API FObjectPropertyBase(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FObjectPropertyBase constructor with InObjectFlags is deprecated, remove that parameter.")
	FObjectPropertyBase(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FObjectPropertyBase(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParams& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);
	UE_API FObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParamsWithoutClass& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);

#if UE_WITH_CONSTINIT_UOBJECT
protected:
	explicit consteval FObjectPropertyBase(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> InPropertyClass)
		: Super(InBaseParams)
		, PropertyClass(InPropertyClass)
	{
	}
public:
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FObjectPropertyBase(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// FProperty interface
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData = nullptr) const override;
	virtual bool SupportsNetSharedSerialization() const override { return false; }
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual FName GetID() const override;
	UE_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph ) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface

	// FObjectPropertyBase interface
public:

	UE_DEPRECATED(5.7, "GetCPPTypeCustom is deprecated, and object properties now implement GetCPPType directly.")
	FString GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& InnerNativeTypeName) const { return {}; }

	/**
	 * Parses a text buffer into an object reference.
	 *
	 * @param	Property			the property that the value is being importing to
	 * @param	OwnerObject			the object that is importing the value; used for determining search scope.
	 * @param	RequiredMetaClass	the meta-class for the object to find; if the object that is resolved is not of this class type, the result is NULL.
	 * @param	PortFlags			bitmask of EPropertyPortFlags that can modify the behavior of the search
	 * @param	Buffer				the text to parse; should point to a textual representation of an object reference.  Can be just the object name (either fully 
	 *								fully qualified or not), or can be formatted as a const object reference (i.e. SomeClass'SomePackage.TheObject')
	 *								When the function returns, Buffer will be pointing to the first character after the object value text in the input stream.
	 * @param	ResolvedValue		receives the object that is resolved from the input text.
	 * @param InSerializeContext	Additional context when called during serialization
	 * @param bAllowAnyPackage		allows ignoring package name to find any object that happens to be loaded with the same name
	 * @return	true if the text is successfully resolved into a valid object reference of the correct type, false otherwise.
	 */
	UE_API static bool ParseObjectPropertyValue( const FProperty* Property, UObject* OwnerObject, UClass* RequiredMetaClass, uint32 PortFlags, const TCHAR*& Buffer, TObjectPtr<UObject>& out_ResolvedValue, FUObjectSerializeContext* InSerializeContext = nullptr, bool bAllowAnyPackage = true );
	UE_API static TObjectPtr<UObject> FindImportedObject( const FProperty* Property, UObject* OwnerObject, UClass* ObjectClass, UClass* RequiredMetaClass, const TCHAR* Text, uint32 PortFlags = 0, FUObjectSerializeContext* InSerializeContext = nullptr, bool bAllowAnyPackage = true );
	
	/**
	 * Returns the qualified export path for a given object, parent, and export root scope
	 * @param Object Object to get the export path for
	 * @param Parent Outer of the Object used as a root object for generating the Objects path name
	 * @param ExportRootScope Similar to Parent but used when exporting from one package or graph to another package or graph
	 * @param PortFlags Property port flags
	 * @return A string representing the export path of an object, usually in the form of /ClassPackage.ClassName'/Package/Path.Object'
	 */
	UE_API static FString GetExportPath(const TObjectPtr<const UObject>& Object, const UObject* Parent = nullptr, const UObject* ExportRootScope = nullptr, const uint32 PortFlags = PPF_None);

	/**
	 * Returns the qualified export path given a class path name and object path name
	 * @param ClassPathName Class path name
	 * @param ObjectPathName of the Object used as a root object for generating the Objects path name
	 * @return A string representing the export path of an object in the form of /ClassPackage.ClassName'/Package/Path.Object'
	 */
	UE_API static FString GetExportPath(FTopLevelAssetPath ClassPathName, const FString& ObjectPathName);

	// Helper method for deep comparing objects
	UE_API static bool StaticIdentical(UObject* A, UObject* B, uint32 PortFlags);

	virtual UObject* LoadObjectPropertyValue(const void* PropertyValueAddress) const
	{
		return GetObjectPropertyValue(PropertyValueAddress);
	}
	UE_FORCEINLINE_HINT UObject* LoadObjectPropertyValue_InContainer(const void* PropertyValueAddress, int32 ArrayIndex = 0) const
	{
		return LoadObjectPropertyValue(ContainerPtrToValuePtr<void>(PropertyValueAddress, ArrayIndex));
	}

protected:
	UE_API virtual void SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const;
	UE_API virtual void SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const;
public:
	UE_API virtual UObject* GetObjectPropertyValue(const void* PropertyValueAddress) const;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue(const void* PropertyValueAddress) const;
	UE_API virtual UObject* GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const;

	UE_API void SetObjectPropertyValue(void* PropertyValueAddress, UObject* Value) const;
	UE_API void SetObjectPtrPropertyValue(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const;
	UE_API void SetObjectPropertyValue_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const;
	UE_API void SetObjectPtrPropertyValue_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const;

	/**
	 * Setter function for this property's PropertyClass member. Favor this 
	 * function whilst loading (since, to handle circular dependencies, we defer 
	 * some class loads and use a placeholder class instead). It properly 
	 * handles deferred loading placeholder classes (so they can properly be 
	 * replaced later).
	 *  
	 * @param  NewPropertyClass    The PropertyClass you want this property set with.
	 */
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	UE_API void SetPropertyClass(UClass* NewPropertyClass);
#else
	UE_FORCEINLINE_HINT void SetPropertyClass(UClass* NewPropertyClass) { PropertyClass = NewPropertyClass; }
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	UE_API virtual void CheckValidObject(void* ValueAddress, TObjectPtr<UObject> OldValue, const void* Defaults = nullptr) const;
	UE_API virtual bool AllowObjectTypeReinterpretationTo(const FObjectPropertyBase* Other) const;

protected:
	UE_API virtual bool AllowCrossLevel() const;
	// End of FObjectPropertyBase interface

	/**
	 * Constructs a new object if the existing one is missing or is not compatible with the property class
	 * Used for making sure non-nullable properties have valid values.
	 * @param ExistingValue Previous object value (can be null)
	 * @param OutFailureReason An optional output string describing the reason why null was returned.
	 * @param Defaults Defaults object to use when constructing the object value (can be null).
	 * @return Pointer that should be assigned to the property value, or null if a default value could not be created.
	 */
	UE_API UObject* ConstructDefaultObjectValueIfNecessary(UObject* ExistingValue, FString* OutFailureReason = nullptr, const void* Defaults = nullptr) const;

	// Disable false positive buffer overrun warning during pgoprofile linking step
	PGO_LINK_DISABLE_WARNINGS
	/* Helper functions for UObject property types that wrap the object pointer in a smart pointer */
	template <typename T, typename OutType>
	void GetWrappedUObjectPtrValues(OutType* OutObjects, const void* SrcAddress, EPropertyMemoryAccess SrcAccess, int32 ArrayIndex, int32 ArrayCount) const
	{
		// Outgoing values are expected to be UObject* or TObjectPtr
		static_assert((std::is_pointer_v<OutType> && std::is_convertible_v<OutType, const UObject*>) || TIsTObjectPtr_V<OutType>);

		// Ensure required range is valid
		checkf(ArrayIndex >= 0 && ArrayCount >= 0 && ArrayIndex <= ArrayDim && ArrayCount <= ArrayDim && ArrayIndex <= ArrayDim - ArrayCount, TEXT("ArrayIndex (%d) and ArrayCount (%d) is invalid for an array of size %d"), ArrayIndex, ArrayCount, ArrayDim);

		if (SrcAccess == EPropertyMemoryAccess::InContainer)
		{
			if (HasGetter())
			{
				if (ArrayCount == 1)
				{
					// Slower but no mallocs. We can copy the value directly to the resulting param
					T Value;
					GetValue_InContainer(SrcAddress, &Value);
					*OutObjects = Value.Get();
				}
				else
				{
					// Malloc a temp value that is the size of the array. Getter will then copy the entire array to the temp value
					T* ValueArray = (T*)AllocateAndInitializeValue();
					FProperty::GetValue_InContainer(SrcAddress, ValueArray);

					// Grab the items we care about and free the temp array
					int32 LocalElementSize = GetElementSize();
					for (int32 OutIndex = 0; OutIndex != ArrayCount; ++OutIndex)
					{
						OutObjects[OutIndex] = ValueArray[ArrayIndex + OutIndex].Get();
					}
					DestroyAndFreeValue(ValueArray);
				}

				return;
			}

			SrcAddress = ContainerPtrToValuePtr<void>(SrcAddress, ArrayIndex);
		}

		// Fast path - direct memory access
		if (ArrayCount == 1)
		{
			*OutObjects = GetObjectPropertyValue(SrcAddress);
		}
		else
		{
			int32 LocalElementSize = GetElementSize();
			for (int32 OutIndex = 0; OutIndex != ArrayCount; ++OutIndex)
			{
				OutObjects[OutIndex] = GetObjectPropertyValue((const uint8*)SrcAddress + OutIndex * LocalElementSize);
			}
		}
	}
	// Enable back buffer overrun warning
	PGO_LINK_ENABLE_WARNINGS

	template <typename T, typename ValueType>
	void SetWrappedUObjectPtrValues(void* DestAddress, EPropertyMemoryAccess DestAccess, ValueType* InValues, int32 ArrayIndex, int32 ArrayCount) const
	{
		// Incoming values are expected to be UObject* or TObjectPtr
		static_assert((std::is_pointer_v<ValueType> && std::is_convertible_v<ValueType, const UObject*>) || TIsTObjectPtr_V<ValueType>);

		// Ensure required range is valid
		checkf(ArrayIndex >= 0 && ArrayCount >= 0 && ArrayIndex <= ArrayDim && ArrayCount <= ArrayDim && ArrayIndex <= ArrayDim - ArrayCount, TEXT("ArrayIndex (%d) and ArrayCount (%d) is invalid for an array of size %d"), ArrayIndex, ArrayCount, ArrayDim);

		if (DestAccess == EPropertyMemoryAccess::InContainer)
		{
			if (HasSetter())
			{
				if (ArrayCount == 1)
				{
					// Slower but no mallocs. We can copy a local wrapped value directly to the resulting param
					T WrappedValue(*InValues);
					SetValue_InContainer(DestAddress, &WrappedValue);
				}
				else
				{
					// Malloc a temp value that is the size of the array. Getter will then copy the entire array to the temp value
					T* ValueArray = (T*)AllocateAndInitializeValue();
					FProperty::GetValue_InContainer(DestAddress, ValueArray);

					// Replace the items we care about
					int32 LocalElementSize = GetElementSize();
					for (int32 OutIndex = 0; OutIndex != ArrayCount; ++OutIndex)
					{
						ValueArray[ArrayIndex + OutIndex] = InValues[OutIndex];
					}

					// Now copy the entire array back to the property using a setter
					SetValue_InContainer(DestAddress, ValueArray);
					DestroyAndFreeValue(ValueArray);
				}

				return;
			}

			DestAddress = ContainerPtrToValuePtr<void>(DestAddress, ArrayIndex);
		}

		// Fast path - direct memory access
		if (ArrayCount == 1)
		{
			SetObjectPropertyValue(DestAddress, *InValues);
		}
		else
		{
			int32 LocalElementSize = GetElementSize();
			for (int32 OutIndex = 0; OutIndex != ArrayCount; ++OutIndex)
			{
				SetObjectPropertyValue((uint8*)DestAddress + OutIndex * LocalElementSize, InValues[OutIndex]);
			}
		}
	}
};

template<typename InTCppType>
class TFObjectPropertyBase : public TProperty<InTCppType, FObjectPropertyBase>
{
public:
	typedef TProperty<InTCppType, FObjectPropertyBase> Super;
	typedef InTCppType TCppType;
	typedef typename Super::TTypeFundamentals TTypeFundamentals;

	TFObjectPropertyBase(EInternal InInernal, FFieldClass* InClass)
		: Super(EC_InternalUseOnlyConstructor, InClass)
	{
	}

	TFObjectPropertyBase(FFieldVariant InOwner, const FName& InName)
		: Super(InOwner, InName)
	{
		this->PropertyClass = nullptr;
	}
	UE_DEPRECATED(5.8, "TFObjectPropertyBase constructor with InObjectFlags is deprecated, remove that parameter.")
	TFObjectPropertyBase(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : TFObjectPropertyBase(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param Prop Pointer to the compiled in structure describing the property
	 **/
	TFObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParams& Prop)
		: Super(InOwner, Prop)
	{
		this->PropertyClass = Prop.ClassFunc ? Prop.ClassFunc(ETypeConstructPhase::Inner) : nullptr;
	}
	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param Prop Pointer to the compiled in structure describing the property
	 * @param InClass Class of the object this property represents
	 **/
	TFObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParamsWithoutClass& Prop, UClass* InClass)
		: Super(InOwner, Prop)
	{
		this->PropertyClass = InClass;
	}

#if UE_WITH_CONSTINIT_UOBJECT
protected:
	explicit consteval TFObjectPropertyBase(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> InPropertyClass)
		: Super(InBaseParams, InPropertyClass)
	{
	}
public:
#endif

#if WITH_EDITORONLY_DATA
	explicit TFObjectPropertyBase(UField* InField)
		: Super(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface.
	virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override
	{
		return (EnumHasAnyFlags(InReferenceType, EPropertyObjectReferenceType::Strong) && !TIsWeakPointerType<InTCppType>::Value)
			|| (EnumHasAnyFlags(InReferenceType, EPropertyObjectReferenceType::Weak) && TIsWeakPointerType<InTCppType>::Value)
			|| (EnumHasAnyFlags(InReferenceType, EPropertyObjectReferenceType::Soft) && TIsSoftObjectPointerType<InTCppType>::Value);
	}
	// End of FProperty interface
};

enum class EObjectPropertyOptions
{
	None = 0,
	AllowNullValuesOnNonNullableProperty = 1
};
ENUM_CLASS_FLAGS(EObjectPropertyOptions);

//
// Describes a reference variable to another object which may be nil.
//
class FObjectProperty : public TFObjectPropertyBase<TObjectPtr<UObject>>
{
	DECLARE_FIELD_API(FObjectProperty, TFObjectPropertyBase<TObjectPtr<UObject>>, CASTCLASS_FObjectProperty, UE_API)

	UE_API FObjectProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FObjectProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FObjectProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FObjectProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FObjectProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> PropertyClass)
		: Super(InBaseParams, PropertyClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FObjectProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override;
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData = nullptr) const override;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	UE_API virtual bool AllowCrossLevel() const override;
private:
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
	UE_API virtual void CopyValuesInternal(void* Dest, void const* Src, int32 Count) const override;
public:
	UE_API virtual void CopySingleValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopySingleValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM_InContainer( void* OutValue, void const* InContainer ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM_InContainer( void* OutContainer, void const* InValue ) const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	// End of FProperty interface

	// FObjectPropertyBase interface
	UE_API virtual UObject* GetObjectPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual UObject* GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
protected:
	UE_API virtual void SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const override;
	UE_API virtual void SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const override;
public:
	// End of FObjectPropertyBase interface
	
	/**
	 * Performs post serialization steps after loading a property value
	 * @param SerializingArchive Archive used for serialization
	 * @param Value Property address
	 * @param CurrentValue Current Object value
	 * @param ObjectValue Deserialized Object value
	 * @param Defaults Defaults used during serialization
	 */
	UE_API void PostSerializeObjectItem(FArchive& SerializingArchive, void* Value, UObject* CurrentValue, UObject* ObjectValue, EObjectPropertyOptions Options = EObjectPropertyOptions::None, const void* Defaults = nullptr) const;

	inline TObjectPtr<UObject>* GetObjectPtrPropertyValuePtr(const void* PropertyValueAddress) const
	{
		return reinterpret_cast<TObjectPtr<UObject>*>(const_cast<void*>(PropertyValueAddress));
	}

	inline TObjectPtr<UObject>& GetObjectPtrPropertyValueRef(const void* PropertyValueAddress) const
	{
		return *reinterpret_cast<TObjectPtr<UObject>*>(const_cast<void*>(PropertyValueAddress));
	}

	virtual bool HasIntrusiveUnsetOptionalState() const 
	{
		// If an object pointer is marked as non-nullable, then nullptr can be used as an intrusive unset state 
		// At present, no C++ properties can be marked with this flag because TOptional<UObject*> and TOptional<TObjectPtr<UObject>> 
		// do not have an intrusive unset state from TOptional's perspective.
		return (PropertyFlags & CPF_NonNullable) != 0;
	}
	
	virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override
	{
		ClearValue(Data);
	}

	virtual bool IsIntrusiveOptionalValueSet(const void* Data) const 
	{
		checkSlow(!IsNative());
		return GetPropertyValue(Data) != nullptr;
	}

	virtual void ClearIntrusiveOptionalValue(void* Data) const 
	{
		checkSlow(!IsNative());
		ClearValue(Data);
	}

	UE_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
};

//
// Describes a reference variable to another object which may be nil, and may turn nil at any point
//
class FWeakObjectProperty : public TFObjectPropertyBase<FWeakObjectPtr>
{
	DECLARE_FIELD_API(FWeakObjectProperty, TFObjectPropertyBase<FWeakObjectPtr>, CASTCLASS_FWeakObjectProperty, UE_API)

	UE_API FWeakObjectProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FWeakObjectProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FWeakObjectProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FWeakObjectProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FWeakObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FWeakObjectPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FWeakObjectProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> PropertyClass)
		: Super(InBaseParams, PropertyClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FWeakObjectProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData = nullptr) const override;
#endif
protected:
	UE_API virtual void LinkInternal(FArchive& Ar) override;
private:
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
public:
	UE_API virtual void CopySingleValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopySingleValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM_InContainer( void* OutValue, void const* InContainer ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM_InContainer( void* OutContainer, void const* InValue ) const override;
	// End of FProperty interface

	// FObjectProperty interface
	UE_API virtual UObject* GetObjectPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual UObject* GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
protected:
	UE_API virtual void SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const override;
	UE_API virtual void SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const override;
public:
	// End of FObjectProperty interface
};

//
// Describes a reference variable to another object which may be nil, and will become valid or invalid at any point
//
class FLazyObjectProperty : public TFObjectPropertyBase<FLazyObjectPtr>
{
	DECLARE_FIELD_API(FLazyObjectProperty, TFObjectPropertyBase<FLazyObjectPtr>, CASTCLASS_FLazyObjectProperty, UE_API)

	UE_API FLazyObjectProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FLazyObjectProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FLazyObjectProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FLazyObjectProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FLazyObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FLazyObjectPropertyParams& Prop);

#if WITH_EDITORONLY_DATA
	UE_API explicit FLazyObjectProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FLazyObjectProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> PropertyClass)
		: Super(InBaseParams, PropertyClass)
	{
	}
#endif

	// UHT interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual FName GetID() const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual void CopySingleValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopySingleValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const override;
	UE_API virtual void CopyCompleteValueToScriptVM_InContainer( void* OutValue, void const* InContainer ) const override;
	UE_API virtual void CopyCompleteValueFromScriptVM_InContainer( void* OutContainer, void const* InValue ) const override;
	// End of FProperty interface

	// FObjectProperty interface
	UE_API virtual UObject* GetObjectPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual UObject* GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
protected:
	UE_API virtual void SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const override;
	UE_API virtual void SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const override;
public:
	UE_API virtual bool AllowCrossLevel() const override;
private:
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
public:
	// End of FObjectProperty interface
};

//
// Describes a reference variable to another object which may be nil, and will become valid or invalid at any point
//
class FSoftObjectProperty : public TFObjectPropertyBase<FSoftObjectPtr>
{
	DECLARE_FIELD_API(FSoftObjectProperty, TFObjectPropertyBase<FSoftObjectPtr>, CASTCLASS_FSoftObjectProperty, UE_API)

	UE_API FSoftObjectProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FSoftObjectProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FSoftObjectProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FSoftObjectProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FSoftObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FSoftObjectPropertyParams& Prop);

	/**
	 * Constructor used for constructing compiled-in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 * @param Class Class of the object this property represents
	 **/
	UE_API FSoftObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParamsWithoutClass& Prop, UClass* InClass);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FSoftObjectProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> PropertyClass)
		: Super(InBaseParams, PropertyClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FSoftObjectProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual FName GetID() const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
protected:
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	// End of FProperty interface

	// FObjectProperty interface
	UE_API virtual UObject* LoadObjectPropertyValue(const void* PropertyValueAddress) const override;

	UE_API virtual UObject* GetObjectPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue(const void* PropertyValueAddress) const override;
	UE_API virtual UObject* GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
	UE_API virtual TObjectPtr<UObject> GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex = 0) const override;
protected:
	UE_API virtual void SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const override;
	UE_API virtual void SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex = 0) const override;
	UE_API virtual void SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex = 0) const override;
public:
	UE_API virtual bool AllowCrossLevel() const override;
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override;

private:
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
public:
	// Note: FSoftObjectProperty does not override the Copy*VM functions, as ScriptVM should store Asset as a FSoftObjectPtr not as a UObject*.

	// End of FObjectProperty interface
};

/*-----------------------------------------------------------------------------
	FClassProperty.
-----------------------------------------------------------------------------*/

//
// Describes a reference variable to another object which may be nil.
//
class FClassProperty : public FObjectProperty
{
	DECLARE_FIELD_API(FClassProperty, FObjectProperty, CASTCLASS_FClassProperty, UE_API)

	// Variables.
	TObjectPtr<class UClass> MetaClass;
public:

	UE_API FClassProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FClassProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FClassProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FClassProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FClassProperty(FFieldVariant InOwner, const UECodeGen_Private::FClassPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FClassProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		UE::CodeGen::ConstInit::TObjectRefParam<UClass> InPropertyClass,
		UE::CodeGen::ConstInit::TObjectRefParam<UClass> InMetaClass
	)
		: Super(InBaseParams, InPropertyClass)
		, MetaClass(InMetaClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FClassProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field Interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// UHT interface
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags)  const override;
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	// End of FProperty interface


	/**
	 * Setter function for this property's MetaClass member. Favor this function 
	 * whilst loading (since, to handle circular dependencies, we defer some 
	 * class loads and use a placeholder class instead). It properly handles 
	 * deferred loading placeholder classes (so they can properly be replaced 
	 * later).
	 * 
	 * @param  NewMetaClass    The MetaClass you want this property set with.
	 */
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	UE_API void SetMetaClass(UClass* NewMetaClass);
#else
	UE_FORCEINLINE_HINT void SetMetaClass(UClass* NewMetaClass) { MetaClass = NewMetaClass; }
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
};


/*-----------------------------------------------------------------------------
	FSoftClassProperty.
-----------------------------------------------------------------------------*/

//
// Describes a reference variable to another class which may be nil, and will become valid or invalid at any point
//
class FSoftClassProperty : public FSoftObjectProperty
{
	DECLARE_FIELD_API(FSoftClassProperty, FSoftObjectProperty, CASTCLASS_FSoftClassProperty, UE_API)

	// Variables.
	TObjectPtr<class UClass> MetaClass;
public:

	UE_API FSoftClassProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FSoftClassProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FSoftClassProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FSoftClassProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FSoftClassProperty(FFieldVariant InOwner, const UECodeGen_Private::FSoftClassPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FSoftClassProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		UE::CodeGen::ConstInit::TObjectRefParam<UClass> ClassClass,
		UE::CodeGen::ConstInit::TObjectRefParam<UClass> InMetaClass
	)
		: Super(InBaseParams, ClassClass)
		, MetaClass(InMetaClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FSoftClassProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override;
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	// End of UHT interface

	// Field Interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// FProperty interface
	UE_API virtual bool SameType(const FProperty* Other) const override;
	// End of FProperty interface


	/**
	 * Setter function for this property's MetaClass member. Favor this function 
	 * whilst loading (since, to handle circular dependencies, we defer some 
	 * class loads and use a placeholder class instead). It properly handles 
	 * deferred loading placeholder classes (so they can properly be replaced 
	 * later).
	 * 
	 * @param  NewMetaClass    The MetaClass you want this property set with.
	 */
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	UE_API void SetMetaClass(UClass* NewMetaClass);
#else
	UE_FORCEINLINE_HINT void SetMetaClass(UClass* NewMetaClass) { MetaClass = NewMetaClass; }
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
};

/*-----------------------------------------------------------------------------
	FInterfaceProperty.
-----------------------------------------------------------------------------*/

/**
 * This variable type provides safe access to a native interface pointer.  The data class for this variable is FScriptInterface, and is exported to auto-generated
 * script header files as a TScriptInterface.
 */

class FInterfaceProperty : public TProperty<FScriptInterface, FProperty>
{
	DECLARE_FIELD_API(FInterfaceProperty, (TProperty<FScriptInterface, FProperty>), CASTCLASS_FInterfaceProperty, UE_API)

	/** The native interface class that this interface property refers to */
	TObjectPtr<class	UClass>		InterfaceClass;
	
public:
	typedef Super::TTypeFundamentals TTypeFundamentals;
	typedef TTypeFundamentals::TCppType TCppType;

	UE_API FInterfaceProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FInterfaceProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FInterfaceProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FInterfaceProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FInterfaceProperty(FFieldVariant InOwner, const UECodeGen_Private::FInterfacePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FInterfaceProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UClass> InInterfaceClass)
		: Super(InBaseParams)
		, InterfaceClass(InInterfaceClass)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FInterfaceProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UHT interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	// End of UHT interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// FProperty interface
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
	virtual bool SupportsNetSharedSerialization() const override { return false; }
protected:
	UE_API virtual void ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	// End of FProperty interface

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	/**
	 * Setter function for this property's InterfaceClass member. Favor this 
	 * function whilst loading (since, to handle circular dependencies, we defer 
	 * some class loads and use a placeholder class instead). It properly 
	 * handles deferred loading placeholder classes (so they can properly be 
	 * replaced later).
	 *  
	 * @param  NewInterfaceClass    The InterfaceClass you want this property set with.
	 */
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	UE_API void SetInterfaceClass(UClass* NewInterfaceClass);
#else
	UE_FORCEINLINE_HINT void SetInterfaceClass(UClass* NewInterfaceClass) { InterfaceClass = NewInterfaceClass; }
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
};

/*-----------------------------------------------------------------------------
	FNameProperty.
-----------------------------------------------------------------------------*/

//
// Describes a name variable pointing into the global name table.
//

class FNameProperty : public TProperty_WithEqualityAndSerializer<FName, FProperty>
{
	DECLARE_FIELD_API(FNameProperty, (TProperty_WithEqualityAndSerializer<FName, FProperty>), CASTCLASS_FNameProperty, UE_API)
public:
	typedef Super::TTypeFundamentals TTypeFundamentals;
	typedef TTypeFundamentals::TCppType TCppType;

	UE_API FNameProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FNameProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FNameProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FNameProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FNameProperty(FFieldVariant InOwner, const UECodeGen_Private::FNamePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FNameProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FNameProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	UE_API uint32 GetValueTypeHashInternal(const void* Src) const override;
	// End of FProperty interface
};

/*-----------------------------------------------------------------------------
	FArrayProperty.
-----------------------------------------------------------------------------*/

//
// Describes a dynamic array.
//

using FFreezableScriptArray = TScriptArray<TMemoryImageAllocator<DEFAULT_ALIGNMENT>>;

static_assert(sizeof(FScriptArray) == sizeof(FFreezableScriptArray) && alignof(FScriptArray) == alignof(FFreezableScriptArray), "FScriptArray and FFreezableScriptArray are expected to be layout-compatible");

class FScriptArrayHelper;

class FArrayProperty : public TProperty<FScriptArray, FProperty>
{
	DECLARE_FIELD_API(FArrayProperty, (TProperty<FScriptArray, FProperty>), CASTCLASS_FArrayProperty, UE_API)

	// Variables.
	EArrayPropertyFlags ArrayFlags;
	FProperty* Inner;

public:
	/** Type of the CPP property **/
	enum
	{
		// These need to be the same as FFreezableScriptArray
		CPPSize = sizeof(FScriptArray),
		CPPAlignment = alignof(FScriptArray)
	};

	using TTypeFundamentals = Super::TTypeFundamentals;
	using TCppType = TTypeFundamentals::TCppType;

	UE_API FArrayProperty(FFieldVariant InOwner, const FName& InName, EArrayPropertyFlags InArrayPropertyFlags=EArrayPropertyFlags::None);
	UE_DEPRECATED(5.8, "FArrayProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FArrayProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags, EArrayPropertyFlags InArrayPropertyFlags=EArrayPropertyFlags::None) : FArrayProperty(InOwner, InName, InArrayPropertyFlags) {}

	/** 
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FArrayProperty(FFieldVariant InOwner, const UECodeGen_Private::FArrayPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FArrayProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams, EArrayPropertyFlags InArrayFlags, FProperty* InInnerProperty
	)
		: Super(InBaseParams)
		, ArrayFlags(InArrayFlags)
		, Inner(InInnerProperty)
	{
	}
#endif

	UE_API virtual ~FArrayProperty();

#if WITH_EDITORONLY_DATA
	UE_API explicit FArrayProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// UField interface
	UE_API virtual void AddCppProperty(FProperty* Property) override;
	UE_API virtual FField* GetInnerFieldByName(const FName& InName) override;
	UE_API virtual void GetInnerFields(TArray<FField*>& OutFields) override;

	// End of UField interface

	// FProperty interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem( FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = NULL ) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const override;
	UE_API virtual void FinishDestroyInternal(void* Data) const override;
public:
	virtual void InitializeValueInternal(void* Dest) const override
	{
		if (EnumHasAnyFlags(ArrayFlags, EArrayPropertyFlags::UsesMemoryImageAllocator))
		{
			for (int32 i = 0; i < this->ArrayDim; ++i)
			{
				new ((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize())) FFreezableScriptArray;
			}
		}
		else
		{
			for (int32 i = 0; i < this->ArrayDim; ++i)
			{
				new ((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize())) FScriptArray;
			}
		}
	}
	UE_API virtual void CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const override;
	UE_API virtual void ClearValueInternal( void* Data ) const override;
	UE_API virtual void DestroyValueInternal( void* Dest ) const override;
	UE_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph ) override;
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;

	virtual int32 GetMinAlignment() const override
	{
		// This is the same as alignof(FFreezableScriptArray)
		return alignof(FScriptArray);
	}

	UE_API virtual void* GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, int32 Index) const override;
	UE_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface

	UE_API FString GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& InnerTypeText, const FString& InInnerExtendedTypeText) const;

	/** Called by ExportText_Internal, but can also be used by a non-ArrayProperty whose ArrayDim is > 1. */
	UE_API static void ExportTextInnerItem(FString& ValueStr, const FProperty* Inner, const void* PropertyValue, int32 NumElements, const void* DefaultValue, int32 DefaultSize, UObject* Parent = nullptr, int32 PortFlags = 0, UObject* ExportRootScope = nullptr);

	/** Called by ImportTextItem, but can also be used by a non-ArrayProperty whose ArrayDim is > 1. ArrayHelper should be supplied by ArrayProperties and nullptr for fixed-size arrays. */
	UE_API static const TCHAR* ImportTextInnerItem(const TCHAR* Buffer, const FProperty* Inner, void* Data, int32 PortFlags, UObject* OwnerObject, FScriptArrayHelper* ArrayHelper = nullptr, FOutputDevice* ErrorText = (FOutputDevice*)GWarn);
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif

	UE_API virtual bool HasIntrusiveUnsetOptionalState() const override;
	UE_API virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override;
	UE_API virtual bool IsIntrusiveOptionalValueSet(const void* Data) const override;
	UE_API virtual void ClearIntrusiveOptionalValue(void* Data) const override;
	UE_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
};

using FFreezableScriptMap = TScriptMap<FMemoryImageSetAllocator>;

//@todo stever sizeof(FScriptMap) is 80 bytes, while sizeof(FFreezableScriptMap) is 56 bytes atm
//static_assert(sizeof(FScriptMap) == sizeof(FFreezableScriptMap) && alignof(FScriptMap) == alignof(FFreezableScriptMap), "FScriptMap and FFreezableScriptMap are expected to be layout-compatible");

class FMapProperty : public TProperty<FScriptMap, FProperty>
{
	DECLARE_FIELD_API(FMapProperty, (TProperty<FScriptMap, FProperty>), CASTCLASS_FMapProperty, UE_API)

	// Properties representing the key type and value type of the contained pairs
	FProperty*       KeyProp;
	FProperty*       ValueProp;
	FScriptMapLayout MapLayout;
	EMapPropertyFlags MapFlags;

	template <typename CallableType>
	auto WithScriptMap(void* InMap, CallableType&& Callable) const
	{
		if (!!(MapFlags & EMapPropertyFlags::UsesMemoryImageAllocator))
		{
			return Callable((FFreezableScriptMap*)InMap);
		}
		else
		{
			return Callable((FScriptMap*)InMap);
		}
	}

public:
	using TTypeFundamentals = Super::TTypeFundamentals;
	using TCppType = TTypeFundamentals::TCppType;

	UE_API FMapProperty(FFieldVariant InOwner, const FName& InName, EMapPropertyFlags InMapFlags=EMapPropertyFlags::None);
	UE_DEPRECATED(5.8, "FMapProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FMapProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags, EMapPropertyFlags InMapFlags=EMapPropertyFlags::None) : FMapProperty(InOwner, InName, InMapFlags) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FMapProperty(FFieldVariant InOwner, const UECodeGen_Private::FMapPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FMapProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		FProperty* InKeyProperty,
		FProperty* InValueProperty,
		EMapPropertyFlags InMapFlags,
		int32 InKeySize,
		int16 InKeyAlignment,
		int32 InValueSize,
		int16 InValueAlignment
	)
		: Super(InBaseParams)
		, KeyProp(InKeyProperty)
		, ValueProp(InValueProperty)
		, MapLayout()
		, MapFlags(InMapFlags)
	{
		MapLayout = FScriptMap::GetScriptLayout(InKeySize, InKeyAlignment, InValueSize, InValueAlignment);
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FMapProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	UE_API virtual ~FMapProperty();

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field Interface
	UE_API virtual void PostDuplicate(const FField& InField) override;
	UE_API virtual FField* GetInnerFieldByName(const FName& InName) override;
	UE_API virtual void GetInnerFields(TArray<FField*>& OutFields) override;

	// UField interface
	UE_API virtual void AddCppProperty(FProperty* Property) override;
	// End of UField interface

	// FProperty interface
	UE_API virtual FString GetCPPMacroType(FString& ExtendedTypeText) const  override;
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override;
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = NULL) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const override;
	UE_API virtual void FinishDestroyInternal(void* Data) const override;
public:
	virtual void InitializeValueInternal(void* Dest) const override
	{
		if (EnumHasAnyFlags(MapFlags, EMapPropertyFlags::UsesMemoryImageAllocator))
		{
			checkf(false, TEXT("FFreezableScriptMap is not supported at the moment"));

			for (int32 i = 0; i < this->ArrayDim; ++i)
			{
				new ((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize())) FFreezableScriptMap;
			}
		}
		else
		{
			for (int32 i = 0; i < this->ArrayDim; ++i)
			{
				new ((uint8*)Dest + i * static_cast<size_t>(this->GetElementSize())) FScriptMap;
			}
		}
	}
	UE_API virtual void CopyValuesInternal(void* Dest, void const* Src, int32 Count) const override;
	UE_API virtual void ClearValueInternal(void* Data) const override;
	UE_API virtual void DestroyValueInternal(void* Dest) const override;
	UE_API virtual void InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph) override;
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	UE_API virtual void* GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, int32 LogicalIndex) const override;
	UE_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface

	UE_API FString GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& KeyTypeText, const FString& InKeyExtendedTypeText, const FString& ValueTypeText, const FString& InValueExtendedTypeText) const;

	/*
	 * Helper function to get the number of key/value pairs inside of a map. 
	 * Used by the garbage collector where for performance reasons the provided map pointer is not guarded
	 */
	int32 GetNum(void* InMap) const
	{
		return WithScriptMap(InMap, [](auto* Map) { return Map->Num(); });
	}

	/*
	 * Helper function to get the sizeof of the map's key/value pair.
	 * Used by the garbage collector.
	 */
	int32 GetPairStride() const
	{
		return MapLayout.SetLayout.Size;
	}

	/*
	 * Helper function to check if the specified index of a key/value pair in the underlying set is valid.
	 * Used by the garbage collector where for performance reasons the provided map pointer is not guarded
	 */
	bool IsValidIndex(void* InMap, int32 InternalIndex) const
	{
		return WithScriptMap(InMap, [InternalIndex](auto* Map) { return Map->IsValidIndex(InternalIndex); });
	}

	/*
	 * Helper function to get the pointer to a key/value pair at the specified index.
	 * Used by the garbage collector where for performance reasons the provided map pointer is not guarded
	 */
	uint8* GetPairPtr(void* InMap, int32 InternalIndex) const
	{
		return WithScriptMap(InMap, [this, InternalIndex](auto* Map) { return (uint8*)Map->GetData(InternalIndex, MapLayout); });
	}

	const FProperty* GetKeyProperty() const
	{
		return KeyProp;
	}

	const FProperty* GetValueProperty() const
	{
		return ValueProp;
	}
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif

	UE_API virtual bool HasIntrusiveUnsetOptionalState() const override;
	UE_API virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override;
	UE_API virtual bool IsIntrusiveOptionalValueSet(const void* Data) const override;
	UE_API virtual void ClearIntrusiveOptionalValue(void* Data) const override;
	UE_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
};

class FSetProperty : public TProperty<FScriptSet, FProperty>
{
	DECLARE_FIELD_API(FSetProperty, (TProperty<FScriptSet, FProperty>), CASTCLASS_FSetProperty, UE_API)

	// Properties representing the key type and value type of the contained pairs
	FProperty*       ElementProp;
	FScriptSetLayout SetLayout;

public:
	using TTypeFundamentals = Super::TTypeFundamentals;
	using TCppType = TTypeFundamentals::TCppType;

	UE_API FSetProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FSetProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FSetProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FSetProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FSetProperty(FFieldVariant InOwner, const UECodeGen_Private::FSetPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FSetProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		FProperty* InElementProperty,
		int32 InElementPropertySize,
		int32 InElementPropertyAlignment
	)
		: Super(InBaseParams)
		, ElementProp(InElementProperty)
		, SetLayout(FScriptSet::GetScriptLayout(InElementPropertySize, InElementPropertyAlignment))
	{
	}
#endif


#if WITH_EDITORONLY_DATA
	UE_API explicit FSetProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	UE_API virtual ~FSetProperty();

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;
	UE_API virtual FField* GetInnerFieldByName(const FName& InName) override;
	UE_API virtual void GetInnerFields(TArray<FField*>& OutFields) override;

	// UField interface
	UE_API virtual void AddCppProperty(FProperty* Property) override;
	// End of UField interface

	// FProperty interface
	UE_API virtual FString GetCPPMacroType(FString& ExtendedTypeText) const  override;
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override;
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = NULL) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const override;
	UE_API virtual void FinishDestroyInternal(void* Data) const override;
public:
	UE_API virtual void CopyValuesInternal(void* Dest, void const* Src, int32 Count) const override;
	UE_API virtual void ClearValueInternal(void* Data) const override;
	UE_API virtual void DestroyValueInternal(void* Dest) const override;
	UE_API virtual void InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph) override;
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	UE_API virtual void* GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, int32 LogicalIndex) const override;
	UE_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface

	UE_API FString GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& ElementTypeText, const FString& InElementExtendedTypeText) const;

	/*
	 * Helper function to get the number of elements inside of a set.
	 * Used by the garbage collector where for performance reasons the provided set pointer is not guarded
	 */
	int32 GetNum(void* InSet) const
	{
		FScriptSet* Set = (FScriptSet*)InSet;
		return Set->Num();
	}

	/*
	 * Helper function to get the size of the set element.
	 * Used by the garbage collector.
	 */
	int32 GetStride() const
	{
		return SetLayout.Size;
	}

	/*
	 * Helper function to check if the specified index of an element is valid.
	 * Used by the garbage collector where for performance reasons the provided set pointer is not guarded
	 */
	bool IsValidIndex(void* InSet, int32 InternalIndex) const
	{
		FScriptSet* Set = (FScriptSet*)InSet;
		return Set->IsValidIndex(InternalIndex);
	}

	/*
	 * Helper function to get the pointer to an element at the specified index.
	 * Used by the garbage collector where for performance reasons the provided set pointer is not guarded
	 */
	uint8* GetElementPtr(void* InSet, int32 InternalIndex) const
	{
		FScriptSet* Set = (FScriptSet*)InSet;
		return (uint8*)Set->GetData(InternalIndex, SetLayout);
	}

	const FProperty* GetElementProperty() const
	{
		return ElementProp;
	}

#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif

	UE_API virtual bool HasIntrusiveUnsetOptionalState() const override;
	UE_API virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override;
	UE_API virtual bool IsIntrusiveOptionalValueSet(const void* Data) const override;
	UE_API virtual void ClearIntrusiveOptionalValue(void* Data) const override;
	UE_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
};

/**
 * FScriptArrayHelper: Pseudo dynamic array. Used to work with array properties in a sensible way.
 **/
class FScriptArrayHelper
{
	enum EInternal { Internal };

	template <typename CallableType>
	auto WithScriptArray(CallableType&& Callable) const
	{
		if (!!(ArrayFlags & EArrayPropertyFlags::UsesMemoryImageAllocator))
		{
			return Callable(FreezableArray);
		}
		else
		{
			return Callable(HeapArray);
		}
	}

public:
	/**
	 *	Constructor, brings together a property and an instance of the property located in memory
	 *	@param	InProperty: the property associated with this memory
	 *	@param	InArray: pointer to raw memory that corresponds to this array. This can be NULL, and sometimes is, but in that case almost all operations will crash.
	**/
	UE_FORCEINLINE_HINT FScriptArrayHelper(const FArrayProperty* InProperty, const void* InArray)
		: FScriptArrayHelper(Internal, InProperty->Inner, InArray, InProperty->Inner->GetElementSize(), InProperty->Inner->GetMinAlignment(), InProperty->ArrayFlags)
	{
	}

	/**
	 *	Index range check
	 *	@param	Index: Index to check
	 *	@return true if accessing this element is legal.
	**/
	UE_FORCEINLINE_HINT bool IsValidIndex( int32 Index ) const
	{
		return Index >= 0 && Index < Num();
	}
	/**
	 *	Return the number of elements in the array.
	 *	@return	The number of elements in the array.
	**/
	inline int32 Num() const
	{
		int32 Result = WithScriptArray([](auto* Array) { return Array->Num(); });
		checkSlow(Result >= 0);
		return Result;
	}
	/**
	 *	Return the number of elements in the array without validating the state of the array.
	 *	Needed to allow reading of the num when the array is 'invalid' during its intrusive unset state.
	 *	@return	The number of elements in the array.
	**/
	inline int32 NumUnchecked() const
	{
		int32 Result = WithScriptArray([](auto* Array) { return Array->NumUnchecked(); });
		return Result;
	}
	/**
	 *	Returns a uint8 pointer to an element in the array
	 *	@param	Index: index of the item to return a pointer to.
	 *	@return	Pointer to this element, or NULL if the array is empty
	**/
	inline uint8* GetRawPtr(int32 Index = 0)
	{
		if (!Num())
		{
			checkSlow(!Index);
			return NULL;
		}
		checkSlow(IsValidIndex(Index)); 
		return (uint8*)WithScriptArray([](auto* Array) { return Array->GetData(); }) + Index * static_cast<size_t>(ElementSize);
	}
	/**
	 *	Returns a uint8 pointer to an element in the array. This call is identical to GetRawPtr and is
	 *  here to provide interface parity with FScriptSetHelper*.
	 *	@param	Index: index of the item to return a pointer to.
	 *	@return	Pointer to this element, or NULL if the array is empty
	**/
	UE_FORCEINLINE_HINT uint8* GetElementPtr(int32 Index = 0)
	{
		return GetRawPtr(Index);
	}
	/**
	*	Empty the array, then add blank, constructed values to a given size.
	*	@param	Count: the number of items the array will have on completion.
	**/
	void EmptyAndAddValues(int32 Count)
	{ 
		check(Count>=0);
		checkSlow(Num() >= 0); 
		EmptyValues(Count);
		AddValues(Count);
	}
	/**
	*	Empty the array, then add uninitialized values to a given size.
	*	@param	Count: the number of items the array will have on completion.
	**/
	void EmptyAndAddUninitializedValues(int32 Count)
	{ 
		check(Count>=0);
		checkSlow(Num() >= 0); 
		EmptyValues(Count);
		AddUninitializedValues(Count);
	}
	/**
	*	Expand the array, if needed, so that the given index is valid
	*	@param	Index: index for the item that we want to ensure is valid
	*	@return true if expansion was necessary
	*	NOTE: This is not a count, it is an INDEX, so the final count will be at least Index+1 this matches the usage.
	**/
	bool ExpandForIndex(int32 Index)
	{ 
		check(Index>=0);
		checkSlow(Num() >= 0); 
		if (Index >= Num())
		{
			AddValues(Index - Num() + 1);
			return true;
		}
		return false;
	}
	/**
	*	Add or remove elements to set the array to a given size.
	*	@param	Count: the number of items the array will have on completion.
	**/
	void Resize(int32 Count)
	{
		if (Count < 0)
		{
			UE::Core::Private::OnInvalidArrayNum(Count);
		}

		int32 OldNum = Num();
		if (Count > OldNum)
		{
			AddValues(Count - OldNum);
		}
		else if (Count < OldNum)
		{
			RemoveValues(Count, OldNum - Count);
		}
	}
	/**
	*	Add blank, constructed values to the end of the array.
	*	@param	Count: the number of items to insert.
	*	@return	the index of the first newly added item.
	**/
	int32 AddValues(int32 Count)
	{ 
		const int32 OldNum = AddUninitializedValues(Count);		
		ConstructItems(OldNum, Count);
		return OldNum;
	}
	/**
	*	Add a blank, constructed values to the end of the array.
	*	@return	the index of the newly added item.
	**/
	UE_FORCEINLINE_HINT int32 AddValue()
	{ 
		return AddValues(1);
	}
	/**
	*	Add uninitialized values to the end of the array.
	*	@param	Count: the number of items to insert.
	*	@return	the index of the first newly added item.
	**/
	int32 AddUninitializedValues(int32 Count)
	{
		check(Count>=0);
		checkSlow(Num() >= 0);
		const int32 OldNum = WithScriptArray([this, Count](auto* Array) { return Array->Add(Count, ElementSize, ElementAlignment); });
		return OldNum;
	}
	/**
	*	Add an uninitialized value to the end of the array.
	*	@return	the index of the newly added item.
	**/
	UE_FORCEINLINE_HINT int32 AddUninitializedValue()
	{
		return AddUninitializedValues(1);
	}
	/**
	 *	Insert blank, constructed values into the array.
	 *	@param	Index: index of the first inserted item after completion
	 *	@param	Count: the number of items to insert.
	**/
	void InsertValues( int32 Index, int32 Count = 1)
	{
		check(Count>=0);
		check(Index>=0 && Index <= Num());
		WithScriptArray([this, Index, Count](auto* Array) { Array->Insert(Index, Count, ElementSize, ElementAlignment); });
		ConstructItems(Index, Count);
	}
	/**
	 *	Remove all values from the array, calling destructors, etc as appropriate.
	 *	@param Slack: used to presize the array for a subsequent add, to avoid reallocation.
	**/
	void EmptyValues(int32 Slack = 0)
	{
		checkSlow(Slack>=0);
		const int32 OldNum = NumUnchecked();
		if (OldNum)
		{
			DestructItems(0, OldNum);
		}
		if (OldNum || Slack)
		{
			WithScriptArray([this, Slack](auto* Array) { Array->Empty(Slack, ElementSize, ElementAlignment); });
		}
	}
	/**
	 *	Remove values from the array, calling destructors, etc as appropriate.
	 *	@param Index: first item to remove.
	 *	@param Count: number of items to remove.
	**/
	void RemoveValues(int32 Index, int32 Count = 1)
	{
		check(Count>=0);
		check(Index>=0 && Index + Count <= Num());
		DestructItems(Index, Count);
		WithScriptArray([this, Index, Count](auto* Array) { Array->Remove(Index, Count, ElementSize, ElementAlignment); });
	}

	/**
	*	Clear values in the array. The meaning of clear is defined by the property system.
	*	@param Index: first item to clear.
	*	@param Count: number of items to clear.
	**/
	void ClearValues(int32 Index, int32 Count = 1)
	{
		check(Count>=0);
		check(Index>=0);
		ClearItems(Index, Count);
	}

	/**
	 *	Swap two elements in the array, does not call constructors and destructors
	 *	@param A index of one item to swap.
	 *	@param B index of the other item to swap.
	**/
	void SwapValues(int32 A, int32 B)
	{
		WithScriptArray([this, A, B](auto* Array) { Array->SwapMemory(A, B, ElementSize); });
	}

	/**
	 *	Move the allocation from another array and make it our own.
	 *	@note The arrays MUST be of the same type, and this function will NOT validate that!
	 *	@param InOtherArray The array to move the allocation from.
	**/
	void MoveAssign(void* InOtherArray)
	{
		checkSlow(InOtherArray);
		// FScriptArray::MoveAssign does not call destructors for our elements, so do that before calling it.
		DestructItems(0, Num());
		WithScriptArray([this, InOtherArray](auto* Array) { Array->MoveAssign(*static_cast<decltype(Array)>(InOtherArray), ElementSize, ElementAlignment); });
	}

	/**
	 *	Used by memory counting archives to accumulate the size of this array.
	 *	@param Ar archive to accumulate sizes
	**/
	void CountBytes( FArchive& Ar  ) const
	{
		WithScriptArray([this, &Ar](auto* Array) { Array->CountBytes(Ar, ElementSize); });
	}

	/**
	 * Destroys the container object - THERE SHOULD BE NO MORE USE OF THIS HELPER AFTER THIS FUNCTION IS CALLED!
	 */
	void DestroyContainer_Unsafe()
	{
		WithScriptArray([](auto* Array) { DestructItem(Array); });
	}

	static FScriptArrayHelper CreateHelperFormInnerProperty(const FProperty* InInnerProperty, const void *InArray, EArrayPropertyFlags InArrayFlags = EArrayPropertyFlags::None)
	{
		return FScriptArrayHelper(Internal, InInnerProperty, InArray, InInnerProperty->GetElementSize(), InInnerProperty->GetMinAlignment(), InArrayFlags);
	}

private:
	FScriptArrayHelper(EInternal, const FProperty* InInnerProperty, const void* InArray, int32 InElementSize, uint32 InElementAlignment, EArrayPropertyFlags InArrayFlags)
		: InnerProperty(InInnerProperty)
		, ElementSize(InElementSize)
		, ElementAlignment(InElementAlignment)
		, ArrayFlags(InArrayFlags)
	{
		//@todo, we are casting away the const here
		if (!!(InArrayFlags & EArrayPropertyFlags::UsesMemoryImageAllocator))
		{
			FreezableArray = (FFreezableScriptArray*)InArray;
		}
		else
		{
			HeapArray = (FScriptArray*)InArray;
		}

		check(ElementSize > 0);
		check(InnerProperty);
	}

	/**
	 *	Internal function to call into the property system to construct / initialize elements.
	 *	@param Index: first item to .
	 *	@param Count: number of items to .
	**/
	void ConstructItems(int32 Index, int32 Count)
	{
		checkSlow(Count >= 0);
		checkSlow(Index >= 0); 
		checkSlow(Index <= Num());
		checkSlow(Index + Count <= Num());
		if (Count > 0)
		{
			uint8* Dest = GetRawPtr(Index);
			if (InnerProperty->PropertyFlags & CPF_ZeroConstructor)
			{
				FMemory::Memzero(Dest, Count * static_cast<size_t>(ElementSize));
			}
			else
			{
				for (int32 LoopIndex = 0; LoopIndex < Count; LoopIndex++, Dest += ElementSize)
				{
					InnerProperty->InitializeValue(Dest);
				}
			}
		}
	}
	/**
	 *	Internal function to call into the property system to destruct elements.
	 *	@param Index: first item to .
	 *	@param Count: number of items to .
	**/
	void DestructItems(int32 Index, int32 Count)
	{
		if (!(InnerProperty->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
		{
			checkSlow(Count >= 0);
			checkSlow(Index >= 0); 
			checkSlow(Index + Count <= Num());
			if (Count > 0)
			{
				uint8* Dest = GetRawPtr(Index);
				for (int32 LoopIndex = 0; LoopIndex < Count; LoopIndex++, Dest += ElementSize)
				{
					InnerProperty->DestroyValue(Dest);
				}
			}
		}
	}
	/**
	 *	Internal function to call into the property system to clear elements.
	 *	@param Index: first item to .
	 *	@param Count: number of items to .
	**/
	void ClearItems(int32 Index, int32 Count)
	{
		checkSlow(Count >= 0);
		checkSlow(Index >= 0); 
		checkSlow(Index < Num());
		checkSlow(Index + Count <= Num());
		if (Count > 0)
		{
			uint8* Dest = GetRawPtr(Index);
			if ((InnerProperty->PropertyFlags & (CPF_ZeroConstructor | CPF_NoDestructor)) == (CPF_ZeroConstructor | CPF_NoDestructor))
			{
				FMemory::Memzero(Dest, Count * static_cast<size_t>(ElementSize));
			}
			else
			{
				for (int32 LoopIndex = 0; LoopIndex < Count; LoopIndex++, Dest += ElementSize)
				{
					InnerProperty->ClearValue(Dest);
				}
			}
		}
	}

	const FProperty* InnerProperty;
	union
	{
		FScriptArray* HeapArray;
		FFreezableScriptArray* FreezableArray;
	};
	int32 ElementSize;
	uint32 ElementAlignment;
	EArrayPropertyFlags ArrayFlags;
};

class FScriptArrayHelper_InContainer : public FScriptArrayHelper
{
public:
	UE_FORCEINLINE_HINT FScriptArrayHelper_InContainer(const FArrayProperty* InProperty, const void* InContainer, int32 FixedArrayIndex=0)
		:FScriptArrayHelper(InProperty, InProperty->ContainerPtrToValuePtr<void>(InContainer, FixedArrayIndex))
	{
	}

	UE_FORCEINLINE_HINT FScriptArrayHelper_InContainer(const FArrayProperty* InProperty, const UObject* InContainer, int32 FixedArrayIndex=0)
		:FScriptArrayHelper(InProperty, InProperty->ContainerPtrToValuePtr<void>(InContainer, FixedArrayIndex))
	{
	}
};

/**
 * Templated iterator to go through script helper containers that may contain invalid entries
 * that are not part of the valid number of elements (i.e. GetMaxIndex() != Num() ).
 * The iterator
 *  - will advance to the first valid entry on creation and when incremented
 *  - can be dereferenced to an internal index to be used with methods like Get<Item>Ptr or Get<Item>PtrWithoutCheck
 *  - can also be used directly with methods like Get<Item>PtrChecked
 *  - can return the associated logical index (number of valid visited entries) by calling GetLogicalIndex()
 */
template<typename ContainerType>
struct TScriptContainerIterator
{
	explicit TScriptContainerIterator(const ContainerType& InContainer) : Container(InContainer)
	{
		Advance();
	}

	explicit TScriptContainerIterator(const ContainerType& InContainer, const int32 InLogicalIndex) : Container(InContainer)
	{
		const int32 MaxIndex = Container.GetMaxIndex();
		if (MaxIndex == Container.Num())
		{
			InternalIndex = InLogicalIndex;
			LogicalIndex = InLogicalIndex;
			return;
		}

		do
		{
			Advance();
		}
		while (LogicalIndex < InLogicalIndex && InternalIndex < MaxIndex);
	}

	TScriptContainerIterator& operator++()
	{
		Advance();
		return *this;
	}

	TScriptContainerIterator operator++(int)
	{
		const TScriptContainerIterator Temp(*this);
		Advance();
		return Temp;
	}

	explicit operator bool() const
	{
		return Container.IsValidIndex(InternalIndex);
	}

	int32 GetInternalIndex() const
	{
		return InternalIndex;
	}

	int32 GetLogicalIndex() const
	{
		return LogicalIndex;
	}

private:
	const ContainerType& Container;
	int32 InternalIndex = INDEX_NONE;
	int32 LogicalIndex = INDEX_NONE;

	void Advance()
	{
		++InternalIndex;
		const int32 MaxIndex = Container.GetMaxIndex();
		while (InternalIndex < MaxIndex && !Container.IsValidIndex(InternalIndex))
		{
			++InternalIndex;
		}

		++LogicalIndex;
	}
};

/**
 * FScriptMapHelper: Pseudo dynamic map. Used to work with map properties in a sensible way.
 * Note that map can contain invalid entries some number of valid entries (i.e. Num() ) can
 * be smaller that the actual number of elements (i.e. GetMaxIndex() ).
 *
 * Internal index naming is used to identify the actual index in the container which can point to
 * an invalid entry. It can be used for methods like Get<Item>Ptr, Get<Item>PtrWithoutCheck or IsValidIndex.
 *
 * Logical index naming is used to identify only valid entries in the container so it can be smaller than the
 * internal index in case we skipped invalid entries to reach the next valid one. This index is used on method
 * like FindNth<Item>Ptr or FindInternalIndex.
 * This is also the type of index we receive from most editor events (e.g. property change events) so it is
 * strongly suggested to rely on FScriptMapHelper::FIterator to iterate or convert to internal index.
 */
class FScriptMapHelper
{
	enum EInternal { Internal };

	friend class FMapProperty;

	template <typename CallableType>
	auto WithScriptMap(CallableType&& Callable) const
	{
		if (!!(MapFlags & EMapPropertyFlags::UsesMemoryImageAllocator))
		{
			return Callable(FreezableMap);
		}
		else
		{
			return Callable(HeapMap);
		}
	}

public:
	/**
	 * Constructor, brings together a property and an instance of the property located in memory
	 *
	 * @param  InProperty  The property associated with this memory
	 * @param  InMap       Pointer to raw memory that corresponds to this map. This can be NULL, and sometimes is, but in that case almost all operations will crash.
	 */
	FScriptMapHelper(const FMapProperty* InProperty, const void* InMap)
	: FScriptMapHelper(Internal, InProperty->KeyProp, InProperty->ValueProp, InMap, InProperty->MapLayout, InProperty->MapFlags)
	{
	}

	FScriptMapHelper(FProperty* InKeyProp, FProperty* InValueProp, const void* InMap, const FScriptMapLayout& InMapLayout, EMapPropertyFlags InMapFlags)
	: FScriptMapHelper(Internal, InKeyProp, InValueProp, InMap, InMapLayout, InMapFlags)
	{
	}

	using FIterator = TScriptContainerIterator<FScriptMapHelper>;

	FIterator CreateIterator() const
	{
		return FIterator(*this);
	}

	FIterator CreateIterator(const int32 InLogicalIndex) const
	{
		return FIterator(*this, InLogicalIndex);
	}
	
	/**
	 * Index range check
	 *
	 * @param InternalIndex Index to check
	 *
	 * @return true if accessing this element is legal.
	 */
	UE_FORCEINLINE_HINT bool IsValidIndex(int32 InternalIndex) const
	{
		return WithScriptMap([InternalIndex](auto* Map) { return Map->IsValidIndex(InternalIndex); });
	}

	/**
	 * Returns the number of elements in the map.
	 *
	 * @return The number of elements in the map.
	 */
	inline int32 Num() const
	{
		int32 Result = WithScriptMap([](auto* Map) { return Map->Num(); });
		checkSlow(Result >= 0); 
		return Result;
	}

	/**
	* Returns the number of elements in the map.
	* Needed to allow reading of the num when the map is 'invalid' during its intrusive unset state.
	* @return The number of elements in the map.
	*/
	inline int32 NumUnchecked() const
	{
		int32 Result = WithScriptMap([](auto* Map) { return Map->NumUnchecked(); });
		checkSlow(Result >= 0); 
		return Result;
	}

	/**
	 * Returns the (non-inclusive) maximum index of elements in the map.
	 *
	 * @return The (non-inclusive) maximum index of elements in the map.
	 */
	inline int32 GetMaxIndex() const
	{
		return WithScriptMap([](auto* Map)
		{
			int32 Result = Map->GetMaxIndex();
			checkSlow(Result >= Map->Num());
			return Result;
		});
	}

	/**
	 * Returns a uint8 pointer to the pair in the map
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	inline uint8* GetPairPtr(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) -> uint8*
		{
			if (Map->Num() == 0)
			{
				checkf(InternalIndex == 0, TEXT("Legacy implementation was only allowing requesting InternalIndex 0 on an empty container."));
				return nullptr;
			}

			checkf(IsValidIndex(InternalIndex), TEXT("Invalid internal index. Use IsValidIndex before calling this method."));
			return (uint8*)Map->GetData(InternalIndex, MapLayout);
		});
	}

	/**
	 * Returns a uint8 pointer to the pair in the map.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT const uint8* GetPairPtr(const int32 InternalIndex) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetPairPtr(InternalIndex);
	}

	/**
	 * Returns a uint8 pointer to the Key (first element) in the map. Currently 
	 * identical to GetPairPtr, but provides clarity of purpose and avoids exposing
	 * implementation details of TMap.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the key, or nullptr if the map is empty.
	 */
	inline uint8* GetKeyPtr(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) -> uint8*
		{
			if (Map->Num() == 0)
			{
				checkf(InternalIndex == 0, TEXT("Legacy implementation was only allowing requesting InternalIndex 0 on an empty container."));
				return nullptr;
			}

			checkf(IsValidIndex(InternalIndex), TEXT("Invalid internal index. Use IsValidIndex before calling this method."));
			return (uint8*)Map->GetData(InternalIndex, MapLayout);
		});
	}

	/**
	 * Returns a uint8 pointer to the Value (second element) in the map.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the value, or nullptr if the map is empty.
	 */
	inline uint8* GetValuePtr(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) -> uint8*
		{
			if (Map->Num() == 0)
			{
				checkf(InternalIndex == 0, TEXT("Legacy implementation was only allowing requesting InternalIndex 0 on an empty container."));
				return nullptr;
			}

			checkf(IsValidIndex(InternalIndex), TEXT("Invalid internal index. Use IsValidIndex before calling this method."));
			return (uint8*)Map->GetData(InternalIndex, MapLayout) + MapLayout.ValueOffset;
		});
	}

	/**
	 * Returns a uint8 pointer to the pair in the map
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or will fail a check if an invalid iterator is provided.
	 */
	inline uint8* GetPairPtr(const FIterator Iterator)
	{
		return WithScriptMap([this, Iterator](auto* Map) -> uint8*
		{
			checkf(Iterator, TEXT("Invalid Iterator. Test Iterator before calling this method."));
			return (uint8*)Map->GetData(Iterator.GetInternalIndex(), MapLayout);
		});
	}

	/**
	 * Returns a uint8 pointer to the pair in the map.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or will fail a check if an invalid iterator is provided.
	 */
	UE_FORCEINLINE_HINT const uint8* GetPairPtr(const FIterator Iterator) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetPairPtr(Iterator);
	}

	/**
	 * Returns a uint8 pointer to the Key (first element) in the map. Currently
	 * identical to GetPairPtr, but provides clarity of purpose and avoids exposing
	 * implementation details of TMap.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the key, or will fail a check if an invalid iterator is provided.
	 */
	inline uint8* GetKeyPtr(const FIterator Iterator)
	{
		return WithScriptMap([this, Iterator](auto* Map) -> uint8*
		{
			checkf(Iterator, TEXT("Invalid Iterator. Test Iterator before calling this method."));
			return (uint8*)Map->GetData(Iterator.GetInternalIndex(), MapLayout);
		});
	}

	/**
	 * Returns a const uint8 pointer to the Key (first element) in the map. Currently
	 * identical to GetPairPtr, but provides clarity of purpose and avoids exposing
	 * implementation details of TMap.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the key, or will fail a check if an invalid iterator is provided.
	 */
	UE_FORCEINLINE_HINT const uint8* GetKeyPtr(const FIterator Iterator) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetKeyPtr(Iterator);
	}

	/**
	 * Returns a uint8 pointer to the Value (second element) in the map.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the value, or will fail a check if an invalid iterator is provided.
	 */
	inline uint8* GetValuePtr(const FIterator Iterator)
	{
		return WithScriptMap([this, Iterator](auto* Map) -> uint8*
		{
			checkf(Iterator, TEXT("Invalid Iterator. Test Iterator before calling this method."));
			return (uint8*)Map->GetData(Iterator.GetInternalIndex(), MapLayout) + MapLayout.ValueOffset;
		});
	}

	/**
	 * Returns a const uint8 pointer to the Value (second element) in the map.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the value, or will fail a check if an invalid iterator is provided.
	 */
	UE_FORCEINLINE_HINT const uint8* GetValuePtr(const FIterator Iterator) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetValuePtr(Iterator);
	}

	/**
	* Returns a uint8 pointer to the the Nth valid pair in the map (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	uint8* FindNthPairPtr(int32 N)
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetPairPtrWithoutCheck(InternalIndex) : nullptr;
	}
	
	/**
	* Returns a uint8 pointer to the the Nth valid key in the map (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	uint8* FindNthKeyPtr(int32 N)
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetKeyPtrWithoutCheck(InternalIndex) : nullptr;
	}
	
	/**
	* Returns a uint8 pointer to the the Nth valid value in the map (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	uint8* FindNthValuePtr(int32 N)
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetValuePtrWithoutCheck(InternalIndex) : nullptr;
	}
	
	/**
	* Returns a uint8 pointer to the the Nth valid pair in the map (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	const uint8* FindNthPairPtr(int32 N) const
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetPairPtrWithoutCheck(InternalIndex) : nullptr;
	}

	/**
	 * Move the allocation from another map and make it our own.
	 * @note The maps MUST be of the same type, and this function will NOT validate that!
	 *
	 * @param InOtherMap The map to move the allocation from.
	 */
	void MoveAssign(void* InOtherMap)
	{
		checkSlow(InOtherMap);
		// FScriptArray::MoveAssign does not call destructors for our elements, so do that before calling it.
		DestructItems(0, Num());
		return WithScriptMap([this, InOtherMap](auto* Map)
		{
			Map->MoveAssign(*(decltype(Map))InOtherMap, MapLayout);
		});
	}

	/**
	 * Add an uninitialized value to the end of the map.
	 *
	 * @return  The index of the added element.
	 */
	inline int32 AddUninitializedValue()
	{
		return WithScriptMap([this](auto* Map)
		{
			checkSlow(Map->Num() >= 0);

			return Map->AddUninitialized(MapLayout);
		});
	}

	/**
	 *	Remove all values from the map, calling destructors, etc as appropriate.
	 *	@param Slack: used to presize the set for a subsequent add, to avoid reallocation.
	**/
	void EmptyValues(int32 Slack = 0)
	{
		checkSlow(Slack >= 0);

		int32 OldNum = NumUnchecked();
		if (OldNum)
		{
			DestructItems(0, OldNum);
		}
		if (OldNum || Slack)
		{
			return WithScriptMap([this, Slack](auto* Map)
			{
				Map->Empty(Slack, MapLayout);
			});
		}
	}

	/**
	 * Adds a blank, constructed value to a given size.
	 * Note that this will create an invalid map because all the keys will be default constructed, and the map needs rehashing.
	 *
	 * @return  The index of the first element added.
	 **/
	int32 AddDefaultValue_Invalid_NeedsRehash()
	{
		return WithScriptMap([this](auto* Map)
		{
			checkSlow(Map->Num() >= 0);

			int32 Result = Map->AddUninitialized(MapLayout);
			ConstructItem(Result);

			return Result;
		});
	}

	/**
	 * Returns the property representing the key of the map pair.
	 *
	 * @return The property representing the key of the map pair.
	 */
	FProperty* GetKeyProperty() const
	{
		return KeyProp;
	}

	/**
	 * Returns the property representing the value of the map pair.
	 *
	 * @return The property representing the value of the map pair.
	 */
	FProperty* GetValueProperty() const
	{
		return ValueProp;
	}

	/**
	 * Removes an element at the specified index, destroying it.
	 *
	 * @param InternalIndex The index of the element to remove.
	 */
	void RemoveAt(int32 InternalIndex, int32 Count = 1)
	{
		return WithScriptMap([this, InternalIndex, Count](auto* Map)
		{
			check(Map->IsValidIndex(InternalIndex));

			// Checking against max as sparse sets may want to remove a range including invalid items, so num is not a correct validation here
			check(InternalIndex + Count <= Map->Max());

#if UE_USE_COMPACT_SET_AS_DEFAULT
			auto GetHashKey = [this](const void* ElementKey)
			{
				return KeyProp->GetValueTypeHash(ElementKey);
			};

			auto DestroyElement = [this](void* Element)
			{
				if (!(KeyProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
				{
					KeyProp->DestroyValue(Element);
				}
				void* ValuePtr = (uint8*)Element + MapLayout.ValueOffset;
				if (!(ValueProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
				{
					ValueProp->DestroyValue(ValuePtr);
				}
			};

			// Last element will get swapped into current spot, so go in reverse so everything moves safely
			for (int32 Index = InternalIndex + Count - 1; Index >= InternalIndex; --Index)
			{
				Map->RemoveAt(Index, MapLayout, GetHashKey, DestroyElement);
			}
#else
			DestructItems(InternalIndex, Count);
			for (int32 LocalCount = Count, LocalIndex = InternalIndex; LocalCount; ++LocalIndex)
			{
				if (Map->IsValidIndex(LocalIndex))
				{
					Map->RemoveAt(LocalIndex, MapLayout);
					--LocalCount;
				}
			}
#endif
		});
	}

	/**
	 * Rehashes the keys in the map.
	 * This function must be called to create a valid map.
	 */
	COREUOBJECT_API void Rehash();

	/** 
	 * Maps have gaps in their indices, so this function translates a logical index (ie. Nth element) 
	 * to an internal index that can be used for the other functions in this class.
	 * NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	 */
	int32 FindInternalIndex(int32 LogicalIdx) const
	{
		return WithScriptMap([this, LogicalIdx](auto* Map) -> int32
		{
			int32 LocalLogicalIdx = LogicalIdx;
			if (LocalLogicalIdx < 0 || LocalLogicalIdx >= Map->Num())
			{
				return INDEX_NONE;
			}

			// if map is compact, use random access
			if (Num() == GetMaxIndex())
			{
				return IsValidIndex(LogicalIdx) ? LogicalIdx : INDEX_NONE;
			}

			int32 MaxIndex = Map->GetMaxIndex();
			for (int32 Actual = 0; Actual < MaxIndex; ++Actual)
			{
				if (Map->IsValidIndex(Actual))
				{
					if (LocalLogicalIdx == 0)
					{
						return Actual;
					}
					--LocalLogicalIdx;
				}
			}
			return INDEX_NONE;
		});
	}

	/** 
	 * Maps have gaps in their indices, so this function translates a internal index
	 * to an logical index (ie. Nth element).
	 * NOTE: This is slow, do not use this for iteration!
	 */
	int32 FindLogicalIndex(int32 InternalIdx) const
	{
		return WithScriptMap([this, InternalIdx](auto* Map) -> int32
		{
			if( !IsValidIndex(InternalIdx) )
			{
				return INDEX_NONE;
			}

			// if map is compact, use random access
			if (GetMaxIndex() == Num())
			{
				return  InternalIdx;
			}

			int32 LogicalIndex = InternalIdx;
			for (int i = 0; i < InternalIdx; ++i)
			{
				if (!IsValidIndex(i))
				{
					LogicalIndex--;
				}
			}

			return LogicalIndex;
		});
	}


	/**
	 * Finds the index of an element in a map which matches the key in another pair.
	 *
	 * @param  PairWithKeyToFind  The address of a map pair which contains the key to search for.
	 * @param  IndexHint          The index to start searching from.
	 *
	 * @return The index of an element found in MapHelper, or -1 if none was found.
	 */
	int32 FindMapIndexWithKey(const void* PairWithKeyToFind, int32 IndexHint = 0) const
	{
		return WithScriptMap([this, PairWithKeyToFind, &IndexHint](auto* Map) -> int32
		{
			int32 MapMax = Map->GetMaxIndex();
			if (MapMax == 0)
			{
				return INDEX_NONE;
			}

			if (IndexHint >= MapMax)
			{
				IndexHint = 0;
			}

			check(IndexHint >= 0);

			FProperty* LocalKeyProp = this->KeyProp; // prevent aliasing in loop below

			int32 InternalIndex = IndexHint;
			for (;;)
			{
				if (Map->IsValidIndex(InternalIndex))
				{
					const void* PairToSearch = Map->GetData(InternalIndex, MapLayout);
					if (LocalKeyProp->Identical(PairWithKeyToFind, PairToSearch))
					{
						return InternalIndex;
					}
				}

				++InternalIndex;
				if (InternalIndex == MapMax)
				{
					InternalIndex = 0;
				}

				if (InternalIndex == IndexHint)
				{
					return INDEX_NONE;
				}
			}
		});
	}

	/**
	 * Finds the pair in a map which matches the key in another pair.
	 *
	 * @param  PairWithKeyToFind  The address of a map pair which contains the key to search for.
	 * @param  IndexHint          The index to start searching from.
	 *
	 * @return A pointer to the found pair, or nullptr if none was found.
	 */
	inline uint8* FindMapPairPtrWithKey(const void* PairWithKeyToFind, int32 IndexHint = 0)
	{
		const int32 InternalIndex = FindMapIndexWithKey(PairWithKeyToFind, IndexHint);
		uint8* Result = (InternalIndex >= 0) ? GetPairPtrWithoutCheck(InternalIndex) : nullptr;
		return Result;
	}

	/** Finds the associated pair from hash, rather than linearly searching */
	int32 FindMapPairIndexFromHash(const void* KeyPtr)
	{
		const int32 InternalIndex = WithScriptMap([this, KeyPtr, LocalKeyPropForCapture = this->KeyProp](auto* Map)
		{
			return Map->FindPairIndex(
				KeyPtr,
				MapLayout,
				[LocalKeyPropForCapture](const void* ElementKey) { return LocalKeyPropForCapture->GetValueTypeHash(ElementKey); },
				[LocalKeyPropForCapture](const void* A, const void* B) { return LocalKeyPropForCapture->Identical(A, B); }
			);
		});
		return InternalIndex;
	}

	/** Finds the associated pair from hash, rather than linearly searching */
	uint8* FindMapPairPtrFromHash(const void* KeyPtr)
	{
		const int32 InternalIndex = FindMapPairIndexFromHash(KeyPtr);
		uint8* Result = (InternalIndex >= 0) ? GetPairPtrWithoutCheck(InternalIndex) : nullptr;
		return Result;
	}

	/** Finds the associated value from hash, rather than linearly searching */
	uint8* FindValueFromHash(const void* KeyPtr)
	{
		return WithScriptMap([this, KeyPtr, LocalKeyPropForCapture = this->KeyProp](auto* Map)
		{
			return Map->FindValue(
				KeyPtr,
				MapLayout,
				[LocalKeyPropForCapture](const void* ElementKey) { return LocalKeyPropForCapture->GetValueTypeHash(ElementKey); },
				[LocalKeyPropForCapture](const void* A, const void* B) { return LocalKeyPropForCapture->Identical(A, B); }
			);
		});
	}

	/** Adds the (key, value) pair to the map, returning true if the element was added, or false if the element was already present and has been overwritten */
	void AddPair(const void* KeyPtr, const void* ValuePtr)
	{
		return WithScriptMap([this, KeyPtr, ValuePtr, LocalKeyPropForCapture = this->KeyProp, LocalValuePropForCapture = this->ValueProp](auto* Map)
		{
			Map->Add(
				KeyPtr,
				ValuePtr,
				MapLayout,
				[LocalKeyPropForCapture](const void* ElementKey) { return LocalKeyPropForCapture->GetValueTypeHash(ElementKey); },
				[LocalKeyPropForCapture](const void* A, const void* B) { return LocalKeyPropForCapture->Identical(A, B); },
				[LocalKeyPropForCapture, KeyPtr](void* NewElementKey)
				{
					if (LocalKeyPropForCapture->PropertyFlags & CPF_ZeroConstructor)
					{
						FMemory::Memzero(NewElementKey, LocalKeyPropForCapture->GetSize());
					}
					else
					{
						LocalKeyPropForCapture->InitializeValue(NewElementKey);
					}

					LocalKeyPropForCapture->CopySingleValueToScriptVM(NewElementKey, KeyPtr);
				},
				[LocalValuePropForCapture, ValuePtr](void* NewElementValue)
				{
					if (LocalValuePropForCapture->PropertyFlags & CPF_ZeroConstructor)
					{
						FMemory::Memzero(NewElementValue, LocalValuePropForCapture->GetSize());
					}
					else
					{
						LocalValuePropForCapture->InitializeValue(NewElementValue);
					}

					LocalValuePropForCapture->CopySingleValueToScriptVM(NewElementValue, ValuePtr);
				},
				[LocalValuePropForCapture, ValuePtr](void* ExistingElementValue)
				{
					LocalValuePropForCapture->CopySingleValueToScriptVM(ExistingElementValue, ValuePtr);
				},
				[LocalKeyPropForCapture](void* ElementKey)
				{
					if (!(LocalKeyPropForCapture->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
					{
						LocalKeyPropForCapture->DestroyValue(ElementKey);
					}
				},
				[LocalValuePropForCapture](void* ElementValue)
				{
					if (!(LocalValuePropForCapture->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
					{
						LocalValuePropForCapture->DestroyValue(ElementValue);
					}
				}
			);
		});
	}


	/**
	 * Finds or adds a new default-constructed value
	 *
	 * No need to rehash after calling. The hash table must be properly hashed before calling.
	 *
	 * @return The address to the value, not the pair
	 **/
	void* FindOrAdd(const void* KeyPtr)
	{
		return WithScriptMap([this, KeyPtr, LocalKeyPropForCapture = this->KeyProp, LocalValuePropForCapture = this->ValueProp](auto* Map)
		{
			return Map->FindOrAdd(
				KeyPtr,
				MapLayout,
				[LocalKeyPropForCapture](const void* ElementKey) { return LocalKeyPropForCapture->GetValueTypeHash(ElementKey); },
				[LocalKeyPropForCapture](const void* A, const void* B) { return LocalKeyPropForCapture->Identical(A, B); },
				[LocalKeyPropForCapture, LocalValuePropForCapture, KeyPtr](void* NewElementKey, void* NewElementValue)
				{
					if (LocalKeyPropForCapture->PropertyFlags & CPF_ZeroConstructor)
					{
						FMemory::Memzero(NewElementKey, LocalKeyPropForCapture->GetSize());
					}
					else
					{
						LocalKeyPropForCapture->InitializeValue(NewElementKey);
					}

					LocalKeyPropForCapture->CopySingleValue(NewElementKey, KeyPtr);

					if (LocalValuePropForCapture->PropertyFlags & CPF_ZeroConstructor)
					{
						FMemory::Memzero(NewElementValue, LocalValuePropForCapture->GetSize());
					}
					else
					{
						LocalValuePropForCapture->InitializeValue(NewElementValue);
					}
				}
			);
		});
	}


	/** Removes the key and its associated value from the map */
	bool RemovePair(const void* KeyPtr)
	{
		return WithScriptMap([this, KeyPtr, LocalKeyPropForCapture = this->KeyProp](auto* Map)
		{
			if (uint8* Entry = Map->FindValue(
				KeyPtr,
				MapLayout,
				[LocalKeyPropForCapture](const void* ElementKey) { return LocalKeyPropForCapture->GetValueTypeHash(ElementKey); },
				[LocalKeyPropForCapture](const void* A, const void* B) { return LocalKeyPropForCapture->Identical(A, B); }
			))
			{
				int32 Idx = (int32)((Entry - (uint8*)Map->GetData(0, MapLayout)) / MapLayout.SetLayout.Size);
				RemoveAt(Idx);
				return true;
			}
			else
			{
				return false;
			}
		});
	}

	static FScriptMapHelper CreateHelperFormInnerProperties(FProperty* InKeyProperty, FProperty* InValProperty, const void *InMap, EMapPropertyFlags InMapFlags = EMapPropertyFlags::None)
	{
		return FScriptMapHelper(
			Internal,
			InKeyProperty,
			InValProperty,
			InMap,
			FScriptMap::GetScriptLayout(InKeyProperty->GetSize(), InKeyProperty->GetMinAlignment(), InValProperty->GetSize(), InValProperty->GetMinAlignment()),
			InMapFlags
		);
	}

private:
	inline FScriptMapHelper(EInternal, FProperty* InKeyProp, FProperty* InValueProp, const void* InMap, const FScriptMapLayout& InMapLayout, EMapPropertyFlags InMapFlags)
		: KeyProp  (InKeyProp)
		, ValueProp(InValueProp)
		, MapLayout(InMapLayout)
		, MapFlags (InMapFlags)
	{
		check(InKeyProp && InValueProp);

		//@todo, we are casting away the const here
		if (!!(InMapFlags & EMapPropertyFlags::UsesMemoryImageAllocator))
		{
			FreezableMap = (FFreezableScriptMap*)InMap;
		}
		else
		{
			HeapMap = (FScriptMap*)InMap;
		}

		check(KeyProp && ValueProp);
	}

	/**
	 * Internal function to call into the property system to construct / initialize elements.
	 *
	 * @param InternalIndex First item to construct.
	 * @param Count Number of items to construct.
	 */
	void ConstructItem(int32 InternalIndex)
	{
		check(IsValidIndex(InternalIndex));

		bool bZeroKey   = !!(KeyProp  ->PropertyFlags & CPF_ZeroConstructor);
		bool bZeroValue = !!(ValueProp->PropertyFlags & CPF_ZeroConstructor);

		void* Dest = WithScriptMap([this, InternalIndex](auto* Map) { return Map->GetData(InternalIndex, MapLayout); });

		if (bZeroKey || bZeroValue)
		{
			// If any nested property needs zeroing, just pre-zero the whole space
			FMemory::Memzero(Dest, MapLayout.SetLayout.Size);
		}

		if (!bZeroKey)
		{
			KeyProp->InitializeValue_InContainer(Dest);
		}

		if (!bZeroValue)
		{
			ValueProp->InitializeValue_InContainer(Dest);
		}
	}

	/**
	 * Internal function to call into the property system to destruct elements.
	 */
	void DestructItems(int32 InternalIndex, int32 Count)
	{
		check(InternalIndex >= 0);
		check(Count >= 0);

		if (Count == 0)
		{
			return;
		}

		bool bDestroyKeys   = !(KeyProp  ->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor));
		bool bDestroyValues = !(ValueProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor));

		if (bDestroyKeys || bDestroyValues)
		{
			uint32 Stride  = MapLayout.SetLayout.Size;
			uint8* PairPtr = WithScriptMap([this, InternalIndex](auto* Map) { return (uint8*)Map->GetData(InternalIndex, MapLayout); });
			if (bDestroyKeys)
			{
				if (bDestroyValues)
				{
					for (; Count; ++InternalIndex)
					{
						if (IsValidIndex(InternalIndex))
						{
							KeyProp  ->DestroyValue_InContainer(PairPtr);
							ValueProp->DestroyValue_InContainer(PairPtr);
							--Count;
						}
						PairPtr += Stride;
					}
				}
				else
				{
					for (; Count; ++InternalIndex)
					{
						if (IsValidIndex(InternalIndex))
						{
							KeyProp->DestroyValue_InContainer(PairPtr);
							--Count;
						}
						PairPtr += Stride;
					}
				}
			}
			else
			{
				for (; Count; ++InternalIndex)
				{
					if (IsValidIndex(InternalIndex))
					{
						ValueProp->DestroyValue_InContainer(PairPtr);
						--Count;
					}
					PairPtr += Stride;
				}
			}
		}
	}

	/**
	 * Returns a uint8 pointer to the pair in the array without checking the index.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT uint8* GetPairPtrWithoutCheck(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) { return (uint8*)Map->GetData(InternalIndex, MapLayout); });
	}

	/**
	 * Returns a uint8 pointer to the pair in the array without checking the index.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT const uint8* GetPairPtrWithoutCheck(int32 InternalIndex) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetPairPtrWithoutCheck(InternalIndex);
	}

	/**
	 * Returns a uint8 pointer to the key in the array without checking the index.
	 *
	 * @param InternalIndex index of the key to return a pointer to.
	 *
	 * @return Pointer to the key, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT uint8* GetKeyPtrWithoutCheck(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) { return (uint8*)Map->GetData(InternalIndex, MapLayout); });
	}

	/**
	 * Returns a const uint8 pointer to the pair in the array without checking the index.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT const uint8* GetKeyPtrWithoutCheck(int32 InternalIndex) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetKeyPtrWithoutCheck(InternalIndex);
	}

	/**
	 * Returns a uint8 pointer to the pair in the array without checking the index.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT uint8* GetValuePtrWithoutCheck(int32 InternalIndex)
	{
		return WithScriptMap([this, InternalIndex](auto* Map) { return (uint8*)Map->GetData(InternalIndex, MapLayout) + MapLayout.ValueOffset; });
	}

	/**
	 * Returns a const uint8 pointer to the pair in the array without checking the index.
	 *
	 * @param InternalIndex index of the item to return a pointer to.
	 *
	 * @return Pointer to the pair, or nullptr if the map is empty.
	 */
	UE_FORCEINLINE_HINT const uint8* GetValuePtrWithoutCheck(int32 InternalIndex) const
	{
		return const_cast<FScriptMapHelper*>(this)->GetValuePtrWithoutCheck(InternalIndex);
	}

public:
	FProperty*        KeyProp;
	FProperty*        ValueProp;
	union
	{
		FScriptMap*          HeapMap;
		FFreezableScriptMap* FreezableMap;
	};
	FScriptMapLayout  MapLayout;
	EMapPropertyFlags MapFlags;
};

class FScriptMapHelper_InContainer : public FScriptMapHelper
{
public:
	UE_FORCEINLINE_HINT FScriptMapHelper_InContainer(const FMapProperty* InProperty, const void* InArray, int32 FixedArrayIndex=0)
		:FScriptMapHelper(InProperty, InProperty->ContainerPtrToValuePtr<void>(InArray, FixedArrayIndex))
	{
	}
};

/**
 * FScriptSetHelper: Pseudo dynamic Set. Used to work with Set properties in a sensible way.
 * Note that the set can contain invalid entries some number of valid entries (i.e. Num() ) can
 * be smaller that the actual number of elements (i.e. GetMaxIndex() ).
 *
 * Internal index naming is used to identify the actual index in the container which can point to
 * an invalid entry. It can be used for methods like Get<Item>Ptr, Get<Item>PtrWithoutCheck or IsValidIndex.
 *
 * Logical index naming is used to identify only valid entries in the container so it can be smaller than the
 * internal index in case we skipped invalid entries to reach the next valid one. This index is used on method
 * like FindNth<Item>Ptr or FindInternalIndex.
 * This is also the type of index we receive from most editor events (e.g. property change events) so it is
 * strongly suggested to rely on FScriptSetHelper::FIterator to iterate or convert to internal index.
 */
class FScriptSetHelper
{
	friend class FSetProperty;

public:

	using FIterator = TScriptContainerIterator<FScriptSetHelper>;

	FIterator CreateIterator() const
	{
		return FIterator(*this);
	}

	FIterator CreateIterator(const int32 InLogicalIndex) const
	{
		return FIterator(*this, InLogicalIndex);
	}

	/**
	* Constructor, brings together a property and an instance of the property located in memory
	*
	* @param  InProperty  The property associated with this memory
	* @param  InSet       Pointer to raw memory that corresponds to this Set. This can be NULL, and sometimes is, but in that case almost all operations will crash.
	*/
	FScriptSetHelper(const FSetProperty* InProperty, const void* InSet)
	: FScriptSetHelper(InProperty->ElementProp, InSet, InProperty->SetLayout)
	{
	}

	FScriptSetHelper(FProperty* InElementProp, const void* InSet, const FScriptSetLayout& InLayout)
	: ElementProp(InElementProp)
	, Set((FScriptSet*)InSet)  //@todo, we are casting away the const here
	, SetLayout(InLayout)
	{
		check(ElementProp);
	}

	/**
	* Index range check
	*
	* @param InternalIndex Index to check
	*
	* @return true if accessing this element is legal.
	*/
	UE_FORCEINLINE_HINT bool IsValidIndex(int32 InternalIndex) const
	{
		return Set->IsValidIndex(InternalIndex);
	}

	/**
	* Returns the number of elements in the set.
	*
	* @return The number of elements in the set.
	*/
	inline int32 Num() const
	{
		const int32 Result = Set->Num();
		checkSlow(Result >= 0); 
		return Result;
	}

	/**
	* Returns the number of elements in the set.
	*	Needed to allow reading of the num when the set is 'invalid' during its intrusive unset state.
	* @return The number of elements in the set.
	*/
	inline int32 NumUnchecked() const
	{
		const int32 Result = Set->NumUnchecked();
		return Result;
	}

	/**
	* Returns the (non-inclusive) maximum index of elements in the set.
	*
	* @return The (non-inclusive) maximum index of elements in the set.
	*/
	inline int32 GetMaxIndex() const
	{
		const int32 Result = Set->GetMaxIndex();
		checkSlow(Result >= Num());
		return Result;
	}

	/**
	* Static version of Num() used when you don't need to bother to construct a FScriptSetHelper. Returns the number of elements in the set.
	*
	* @param  Target  Pointer to the raw memory associated with a FScriptSet
	*
	* @return The number of elements in the set.
	*/
	static inline int32 Num(const void* Target)
	{
		const int32 Result = ((const FScriptSet*)Target)->Num();
		checkSlow(Result >= 0); 
		return Result;
	}

	/**
	* Returns a uint8 pointer to the element in the set.
	*
	* @param InternalIndex index of the item to return a pointer to.
	*
	* @return Pointer to the element, or nullptr if the set is empty.
	*/
	inline uint8* GetElementPtr(int32 InternalIndex)
	{
		if (Num() == 0)
		{
			checkf(InternalIndex == 0, TEXT("Legacy implementation was only allowing requesting InternalIndex 0 on an empty container."));
			return nullptr;
		}

		checkf(IsValidIndex(InternalIndex), TEXT("Invalid internal index. Use IsValidIndex before calling this method."));
		return (uint8*)Set->GetData(InternalIndex, SetLayout);
	}

	/**
	* Returns a uint8 pointer to the element in the set.
	*
	* @param InternalIndex index of the item to return a pointer to.
	*
	* @return Pointer to the element, or nullptr if the set is empty.
	*/
	UE_FORCEINLINE_HINT const uint8* GetElementPtr(int32 InternalIndex) const
	{
		return const_cast<FScriptSetHelper*>(this)->GetElementPtr(InternalIndex);
	}

	/**
	 * Returns a uint8 pointer to the element in the set.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the element, or will fail a check if an invalid iterator is provided.
	 */
	inline uint8* GetElementPtr(const FIterator Iterator)
	{
		checkf(Iterator, TEXT("Invalid Iterator. Test Iterator before calling this method."));
		return (uint8*)Set->GetData(Iterator.GetInternalIndex(), SetLayout);
	}

	/**
	 * Returns a uint8 pointer to the element in the set.
	 *
	 * @param Iterator A valid iterator of the item to return a pointer to.
	 *
	 * @return Pointer to the element, or will fail a check if an invalid iterator is provided.
	 */
	UE_FORCEINLINE_HINT const uint8* GetElementPtr(const FIterator Iterator) const
	{
		return const_cast<FScriptSetHelper*>(this)->GetElementPtr(Iterator);
	}

	/**
	* Returns a uint8 pointer to the the Nth valid element in the set (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	uint8* FindNthElementPtr(int32 N)
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetElementPtrWithoutCheck(InternalIndex) : nullptr;
	}

	/**
	* Returns a uint8 pointer to the the Nth valid element in the set (skipping invalid entries).
	* NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	*
	* @return Pointer to the element, or nullptr if the index is invalid.
	*/
	const uint8* FindNthElementPtr(int32 N) const
	{
		const int32 InternalIndex = FindInternalIndex(N);
		return (InternalIndex != INDEX_NONE) ? GetElementPtrWithoutCheck(InternalIndex) : nullptr;
	}

	/**
	* Move the allocation from another set and make it our own.
	* @note The sets MUST be of the same type, and this function will NOT validate that!
	*
	* @param InOtherSet The set to move the allocation from.
	*/
	void MoveAssign(void* InOtherSet)
	{
		FScriptSet* OtherSet = (FScriptSet*)InOtherSet;
		checkSlow(OtherSet);
		Set->MoveAssign(*OtherSet, SetLayout);
	}

	/**
	* Add an uninitialized value to the end of the set.
	*
	* @return  The index of the added element.
	*/
	inline int32 AddUninitializedValue()
	{
		checkSlow(Num() >= 0);

		return Set->AddUninitialized(SetLayout);
	}

	/**
	*	Remove all values from the set, calling destructors, etc as appropriate.
	*	@param Slack: used to presize the set for a subsequent add, to avoid reallocation.
	**/
	void EmptyElements(int32 Slack = 0)
	{
		checkSlow(Slack >= 0);

		int32 OldNum = NumUnchecked();
		if (OldNum)
		{
			DestructItems(0, OldNum);
		}
		if (OldNum || Slack)
		{
			Set->Empty(Slack, SetLayout);
		}
	}

	/**
	* Adds a blank, constructed value to a given size.
	* Note that this will create an invalid Set because all the keys will be default constructed, and the set needs rehashing.
	*
	* @return  The index of the first element added.
	**/
	int32 AddDefaultValue_Invalid_NeedsRehash()
	{
		checkSlow(Num() >= 0);

		int32 Result = AddUninitializedValue();
		ConstructItem(Result);

		return Result;
	}

	/**
	* Returns the property representing the element of the set
	*/
	FProperty* GetElementProperty() const
	{
		return ElementProp;
	}

	/**
	* Removes an element at the specified index, destroying it.
	*
	* @param InternalIndex The index of the element to remove.
	*/
	void RemoveAt(int32 InternalIndex, int32 Count = 1)
	{
		check(IsValidIndex(InternalIndex));

		// Checking against max as sparse sets may want to remove a range including invalid items, so num is not a correct validation here
		check(InternalIndex + Count <= Set->Max());

#if UE_USE_COMPACT_SET_AS_DEFAULT
		auto GetHashKey = [this](const void* ElementKey)
		{
			return ElementProp->GetValueTypeHash(ElementKey);
		};

		auto DestroyElement = [this](void* Element)
		{
			if (!(ElementProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
			{
				ElementProp->DestroyValue(Element);
			}
		};

		// Last element will get swapped into current spot, so go in reverse so everything moves safely
		for (int32 Index = InternalIndex + Count - 1; Index >= InternalIndex; --Index)
		{
			Set->RemoveAt(Index, SetLayout, GetHashKey, DestroyElement);
		}
#else
		DestructItems(InternalIndex, Count);
		for (; Count; ++InternalIndex)
		{
			if (IsValidIndex(InternalIndex))
			{
				Set->RemoveAt(InternalIndex, SetLayout);
				--Count;
			}
		}
#endif
	}

	/**
	* Rehashes the keys in the set.
	* This function must be called to create a valid set.
	*/
	COREUOBJECT_API void Rehash();

	/**
	 * Sets have gaps in their indices, so this function translates a logical index (ie. Nth element)
	 * to an internal index that can be used for the other functions in this class.
	 * NOTE: This is slow, do not use this for iteration! Use CreateIterator() instead.
	 */
	int32 FindInternalIndex(int32 LogicalIdx) const
	{
		if (LogicalIdx < 0 || LogicalIdx >= Num())
		{
			return INDEX_NONE;
		}

		// if set is compact, use random access
		if (Num() == GetMaxIndex())
		{
			return IsValidIndex(LogicalIdx) ? LogicalIdx : INDEX_NONE;
		}

		int32 MaxIndex = GetMaxIndex();
		for (int32 Actual = 0; Actual < MaxIndex; ++Actual)
		{
			if (IsValidIndex(Actual))
			{
				if (LogicalIdx == 0)
				{
					return Actual;
				}
				--LogicalIdx;
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Sets have gaps in their indices, so this function translates a internal index
	 * to an logical index (ie. Nth element).
	 * NOTE: This is slow, do not use this for iteration!
	 */
	int32 FindLogicalIndex(int32 InternalIdx) const
	{
		if (!IsValidIndex(InternalIdx))
		{
			return INDEX_NONE;
		}

		// if set is compact, use random access
		if (GetMaxIndex() == Num())
		{
			return  InternalIdx;
		}

		int32 LogicalIndex = InternalIdx;
		for (int i = 0; i < InternalIdx; ++i)
		{
			if (!IsValidIndex(i))
			{
				LogicalIndex--;
			}
		}

		return LogicalIndex;
	}

	/**
	* Finds the index of an element in a set
	*
	* @param  ElementToFind		The address of an element to search for.
	* @param  IndexHint         The index to start searching from.
	*
	* @return The index of an element found in SetHelper, or -1 if none was found.
	*/
	int32 FindElementIndex(const void* ElementToFind, int32 IndexHint = 0) const
	{
		const int32 SetMax = GetMaxIndex();
		if (SetMax == 0)
		{
			return INDEX_NONE;
		}

		if (IndexHint >= SetMax)
		{
			IndexHint = 0;
		}

		check(IndexHint >= 0);

		FProperty* LocalKeyProp = this->ElementProp; // prevent aliasing in loop below

		int32 InternalIndex = IndexHint;
		for (;;)
		{
			if (IsValidIndex(InternalIndex))
			{
				const void* ElementToCheck = GetElementPtrWithoutCheck(InternalIndex);
				if (LocalKeyProp->Identical(ElementToFind, ElementToCheck))
				{
					return InternalIndex;
				}
			}

			++InternalIndex;
			if (InternalIndex == SetMax)
			{
				InternalIndex = 0;
			}

			if (InternalIndex == IndexHint)
			{
				return INDEX_NONE;
			}
		}
	}

	/**
	* Finds the pair in a map which matches the key in another pair.
	*
	* @param  PairWithKeyToFind  The address of a map pair which contains the key to search for.
	* @param  IndexHint          The index to start searching from.
	*
	* @return A pointer to the found pair, or nullptr if none was found.
	*/
	inline uint8* FindElementPtr(const void* ElementToFind, int32 IndexHint = 0)
	{
		const int32 InternalIndex = FindElementIndex(ElementToFind, IndexHint);
		uint8* Result = (InternalIndex >= 0 ? GetElementPtrWithoutCheck(InternalIndex) : nullptr);
		return Result;
	}

	/** Finds element index from hash, rather than linearly searching */
	inline int32 FindElementIndexFromHash(const void* ElementToFind) const
	{
		FProperty* LocalElementPropForCapture = ElementProp;
		return Set->FindIndex(
			ElementToFind,
			SetLayout,
			[LocalElementPropForCapture](const void* Element) { return LocalElementPropForCapture->GetValueTypeHash(Element); },
			[LocalElementPropForCapture](const void* A, const void* B) { return LocalElementPropForCapture->Identical(A, B); }
		);
	}

	/** Finds element pointer from hash, rather than linearly searching */
	inline uint8* FindElementPtrFromHash(const void* ElementToFind)
	{
		const int32 InternalIndex = FindElementIndexFromHash(ElementToFind);
		uint8* Result = (InternalIndex >= 0 ? GetElementPtrWithoutCheck(InternalIndex) : nullptr);
		return Result;
	}

	/** Adds the element to the set, returning true if the element was added, or false if the element was already present */
	void AddElement(const void* ElementToAdd)
	{
		FProperty* LocalElementPropForCapture = ElementProp;
		FScriptSetLayout& LocalSetLayoutForCapture = SetLayout;
		Set->Add(
			ElementToAdd,
			SetLayout,
			[LocalElementPropForCapture](const void* Element) { return LocalElementPropForCapture->GetValueTypeHash(Element); },
			[LocalElementPropForCapture](const void* A, const void* B) { return LocalElementPropForCapture->Identical(A, B); },
			[LocalElementPropForCapture, ElementToAdd, LocalSetLayoutForCapture](void* NewElement)
			{
				if (LocalElementPropForCapture->PropertyFlags & CPF_ZeroConstructor)
				{
					FMemory::Memzero(NewElement, LocalElementPropForCapture->GetSize());
				}
				else
				{
					LocalElementPropForCapture->InitializeValue(NewElement);
				}

				LocalElementPropForCapture->CopySingleValueToScriptVM(NewElement, ElementToAdd);
			},
			[LocalElementPropForCapture](void* Element)
			{
				if (!(LocalElementPropForCapture->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)))
				{
					LocalElementPropForCapture->DestroyValue(Element);
				}
			}
		);
	}

	/** Removes the element from the set */
	bool RemoveElement(const void* ElementToRemove)
	{
		FProperty* LocalElementPropForCapture = ElementProp;
		int32 FoundIndex = Set->FindIndex(
			ElementToRemove,
			SetLayout,
			[LocalElementPropForCapture](const void* Element) { return LocalElementPropForCapture->GetValueTypeHash(Element); },
			[LocalElementPropForCapture](const void* A, const void* B) { return LocalElementPropForCapture->Identical(A, B); }
		);
		if (FoundIndex != INDEX_NONE)
		{
			RemoveAt(FoundIndex);
			return true;
		}
		else
		{
			return false;
		}
	}

	static FScriptSetHelper CreateHelperFormElementProperty(FProperty* InElementProperty, const void *InSet)
	{
		check(InElementProperty);

		FScriptSetHelper ScriptSetHelper;
		ScriptSetHelper.ElementProp = InElementProperty;
		ScriptSetHelper.Set = (FScriptSet*)InSet;

		const int32 ElementPropSize = InElementProperty->GetSize();
		const int32 ElementPropAlignment = InElementProperty->GetMinAlignment();
		ScriptSetHelper.SetLayout = FScriptSet::GetScriptLayout(ElementPropSize, ElementPropAlignment);

		return ScriptSetHelper;
	}

private: 
	FScriptSetHelper()
		: ElementProp(nullptr)
		, Set(nullptr)
		, SetLayout(FScriptSet::GetScriptLayout(0, 1))
	{}

	/**
	* Internal function to call into the property system to construct / initialize elements.
	*
	* @param InternalIndex First item to construct.
	* @param Count Number of items to construct.
	*/
	void ConstructItem(int32 InternalIndex)
	{
		check(IsValidIndex(InternalIndex));

		bool bZeroElement = !!(ElementProp->PropertyFlags & CPF_ZeroConstructor);
		uint8* Dest = GetElementPtrWithoutCheck(InternalIndex);

		if (bZeroElement)
		{
			// If any nested property needs zeroing, just pre-zero the whole space
			FMemory::Memzero(Dest, SetLayout.Size);
		}

		if (!bZeroElement)
		{
			ElementProp->InitializeValue_InContainer(Dest);
		}
	}

	/**
	* Internal function to call into the property system to destruct elements.
	*/
	void DestructItems(int32 InternalIndex, int32 Count)
	{
		check(InternalIndex >= 0);
		check(Count >= 0);

		if (Count == 0)
		{
			return;
		}

		bool bDestroyElements = !(ElementProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor));

		if (bDestroyElements)
		{
			uint32 Stride = SetLayout.Size;
			uint8* ElementPtr = GetElementPtrWithoutCheck(InternalIndex);

			for (; Count; ++InternalIndex)
			{
				if (IsValidIndex(InternalIndex))
				{
					ElementProp->DestroyValue_InContainer(ElementPtr);
					--Count;
				}
				ElementPtr += Stride;
			}
		}
	}

	/**
	* Returns a uint8 pointer to the element in the array without checking the index.
	*
	* @param InternalIndex index of the item to return a pointer to.
	*
	* @return Pointer to the element, or nullptr if the array is empty.
	*/
	UE_FORCEINLINE_HINT uint8* GetElementPtrWithoutCheck(int32 InternalIndex)
	{
		return (uint8*)Set->GetData(InternalIndex, SetLayout);
	}

	/**
	* Returns a uint8 pointer to the element in the array without checking the index.
	*
	* @param InternalIndex index of the item to return a pointer to.
	*
	* @return Pointer to the pair, or nullptr if the array is empty.
	*/
	UE_FORCEINLINE_HINT const uint8* GetElementPtrWithoutCheck(int32 InternalIndex) const
	{
		return const_cast<FScriptSetHelper*>(this)->GetElementPtrWithoutCheck(InternalIndex);
	}

public:
	FProperty*       ElementProp;
	FScriptSet*      Set;
	FScriptSetLayout SetLayout;
};

class FScriptSetHelper_InContainer : public FScriptSetHelper
{
public:
	UE_FORCEINLINE_HINT FScriptSetHelper_InContainer(const FSetProperty* InProperty, const void* InArray, int32 FixedArrayIndex=0)
		:FScriptSetHelper(InProperty, InProperty->ContainerPtrToValuePtr<void>(InArray, FixedArrayIndex))
	{
	}
};

/*-----------------------------------------------------------------------------
	FStructProperty.
-----------------------------------------------------------------------------*/

//
// Describes a structure variable embedded in (as opposed to referenced by) 
// an object.
//
class FStructProperty : public FProperty
{
	DECLARE_FIELD_API(FStructProperty, FProperty, CASTCLASS_FStructProperty, UE_API)

	// Variables.
	TObjectPtr<class UScriptStruct> Struct;
public:
	UE_API FStructProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FStructProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FStructProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FStructProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FStructProperty(FFieldVariant InOwner, const UECodeGen_Private::FStructPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FStructProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		int32 StructSize,
		UE::CodeGen::ConstInit::TObjectRefParam<UScriptStruct> InScriptStruct
	)
		: Super(InBaseParams.SetElementSize(StructSize))
		, Struct(InScriptStruct)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FStructProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// FProperty interface
	UE_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
	UE_API virtual bool SupportsNetSharedSerialization() const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const override;
	UE_API virtual void FinishDestroyInternal(void* Data) const override;
public:
	UE_API virtual void CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const override;
	UE_API virtual void ClearValueInternal( void* Data ) const override;
	UE_API virtual void DestroyValueInternal( void* Dest ) const override;
	UE_API virtual void InitializeValueInternal( void* Dest ) const override;
	UE_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph ) override;
	UE_API virtual int32 GetMinAlignment() const override;
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	UE_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface

	UE_API bool FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData) const;

	UE_API virtual bool HasIntrusiveUnsetOptionalState() const override;
	UE_API virtual void InitializeIntrusiveUnsetOptionalValue(void* Data) const override;
	UE_API virtual bool IsIntrusiveOptionalValueSet(const void* Data) const override;
	UE_API virtual void ClearIntrusiveOptionalValue(void* Data) const override;
	UE_API virtual void EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;

private:
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const;
};

/*-----------------------------------------------------------------------------
	FDelegateProperty.
-----------------------------------------------------------------------------*/

/**
 * Describes a pointer to a function bound to an Object.
 */
class FDelegateProperty : public TProperty<FScriptDelegate, FProperty>
{
	DECLARE_FIELD_API(FDelegateProperty, (TProperty<FScriptDelegate, FProperty>), CASTCLASS_FDelegateProperty, UE_API)

	/** Points to the source delegate function (the function declared with the delegate keyword) used in the declaration of this delegate property. */
	TObjectPtr<UFunction> SignatureFunction;
public:

	using TTypeFundamentals = Super::TTypeFundamentals;
	using TCppType = TTypeFundamentals::TCppType;

	UE_API FDelegateProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FDelegateProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FDelegateProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FDelegateProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FDelegatePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FDelegateProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UFunction> InSignatureFunction)
		: Super(InBaseParams)
		, SignatureFunction(InSignatureFunction)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FDelegateProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// FProperty interface
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph ) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
#if WITH_EDITORONLY_DATA
	UE_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	// End of FProperty interface
};


/*-----------------------------------------------------------------------------
	FMulticastDelegateProperty.
-----------------------------------------------------------------------------*/

/**
 * Describes a list of functions bound to an Object.
 */
class FMulticastDelegateProperty : public FProperty
{
	DECLARE_FIELD_API(FMulticastDelegateProperty, FProperty, CASTCLASS_FMulticastDelegateProperty, UE_API)

	/** Points to the source delegate function (the function declared with the delegate keyword) used in the declaration of this delegate property. */
	TObjectPtr<UFunction> SignatureFunction;

public:

	UE_API FMulticastDelegateProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FMulticastDelegateProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FMulticastDelegateProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FMulticastDelegateProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FMulticastDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop, EPropertyFlags AdditionalPropertyFlags = CPF_None);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FMulticastDelegateProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UFunction> InSignatureFunction)
		: Super(InBaseParams)
		, SignatureFunction(InSignatureFunction)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	UE_API explicit FMulticastDelegateProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// UObject interface
	UE_API virtual void Serialize( FArchive& Ar ) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	UE_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;

	// FProperty interface
	UE_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
protected:
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
public:
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph ) override;
	UE_API virtual bool SameType(const FProperty* Other) const override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	// End of FProperty interface

	virtual const FMulticastScriptDelegate* GetMulticastDelegate(const void* PropertyValue) const PURE_VIRTUAL(FMulticastDelegateProperty::GetMulticastDelegate, return nullptr;);
	virtual void SetMulticastDelegate(void* PropertyValue, FMulticastScriptDelegate ScriptDelegate) const PURE_VIRTUAL(FMulticastDelegateProperty::SetMulticastDelegate, );

	virtual void AddDelegate(FScriptDelegate ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const PURE_VIRTUAL(FMulticastDelegateProperty::AddDelegate, );
	virtual void RemoveDelegate(const FScriptDelegate& ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const PURE_VIRTUAL(FMulticastDelegateProperty::RemoveDelegate, );
	virtual void ClearDelegate(UObject* Parent = nullptr, void* PropertyValue = nullptr)  const PURE_VIRTUAL(FMulticastDelegateProperty::ClearDelegate, );

protected:
	friend class FProperty;

	UE_API static FMulticastScriptDelegate EmptyDelegate;
	virtual FMulticastScriptDelegate& GetMulticastScriptDelegate(const void* PropertyValue, int32 Index) const PURE_VIRTUAL(FMulticastDelegateProperty::GetMulticastScriptDelegate, return EmptyDelegate;);


	UE_API const TCHAR* ImportText_Add( const TCHAR* Buffer, void* PropertyValue, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText ) const;
	UE_API const TCHAR* ImportText_Remove( const TCHAR* Buffer, void* PropertyValue, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText ) const;

	UE_API const TCHAR* ImportDelegateFromText(FMulticastScriptDelegate& MulticastDelegate, const TCHAR* Buffer, UObject* OwnerObject, FOutputDevice* ErrorText) const;
};

template<class InTCppType>
class TProperty_MulticastDelegate : public TProperty<InTCppType, FMulticastDelegateProperty>
{
public:
	typedef TProperty<InTCppType, FMulticastDelegateProperty> Super;
	typedef InTCppType TCppType;
	typedef typename Super::TTypeFundamentals TTypeFundamentals;
	TProperty_MulticastDelegate(FFieldVariant InOwner, const FName& InName, UFunction* InSignatureFunction)
		: Super(InOwner, InName)
	{
		this->SignatureFunction = InSignatureFunction;
	}

	TProperty_MulticastDelegate(FFieldVariant InOwner, const FName& InName)
		: Super(InOwner, InName)
	{
		this->SignatureFunction = nullptr;
	}
	UE_DEPRECATED(5.8, "TProperty_MulticastDelegate constructor with InObjectFlags is deprecated, remove that parameter.")
	TProperty_MulticastDelegate(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : TProperty_MulticastDelegate(InOwner, InName) {}

	TProperty_MulticastDelegate(EInternal InInernal, FFieldClass* InClass)
		: Super(EC_InternalUseOnlyConstructor, InClass)
	{
	}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	TProperty_MulticastDelegate(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop)
		: Super(InOwner, Prop)
	{
		this->SignatureFunction = Prop.SignatureFunctionFunc ? Prop.SignatureFunctionFunc(ETypeConstructPhase::Inner) : nullptr;
	}

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval TProperty_MulticastDelegate(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, UE::CodeGen::ConstInit::TObjectRefParam<UFunction> InSignatureFunction)
		: Super(InBaseParams, InSignatureFunction)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit TProperty_MulticastDelegate(UField* InField)
		: Super(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface.
	virtual FString GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const override
	{
		return FMulticastDelegateProperty::GetCPPType(ExtendedTypeText, CPPExportFlags);
	}
	// End of FProperty interface
};

class FMulticastInlineDelegateProperty : public TProperty_MulticastDelegate<FMulticastScriptDelegate>
{
	DECLARE_FIELD_API(FMulticastInlineDelegateProperty, TProperty_MulticastDelegate<FMulticastScriptDelegate>, CASTCLASS_FMulticastInlineDelegateProperty, UE_API)

public:

	FMulticastInlineDelegateProperty(FFieldVariant InOwner, const FName& InName)
		: TProperty_MulticastDelegate(InOwner, InName)
	{
	}
	UE_DEPRECATED(5.8, "FMulticastInlineDelegateProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FMulticastInlineDelegateProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FMulticastInlineDelegateProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FMulticastInlineDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FMulticastInlineDelegateProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		UE::CodeGen::ConstInit::TObjectRefParam<UFunction> InSignatureFunction
	)
		: Super(InBaseParams, InSignatureFunction)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit FMulticastInlineDelegateProperty(UField* InField)
		: TProperty_MulticastDelegate(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	// End of FProperty interface

	// FMulticastDelegateProperty interface
	UE_API virtual const FMulticastScriptDelegate* GetMulticastDelegate(const void* PropertyValue) const override;
	UE_API virtual void SetMulticastDelegate(void* PropertyValue, FMulticastScriptDelegate ScriptDelegate) const override;

	UE_API virtual void AddDelegate(FScriptDelegate ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;
	UE_API virtual void RemoveDelegate(const FScriptDelegate& ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;
	UE_API virtual void ClearDelegate(UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;

protected:
	UE_API virtual FMulticastScriptDelegate& GetMulticastScriptDelegate(const void* PropertyValue, int32 Index) const;
	// End of FMulticastDelegateProperty interface
};

class FMulticastSparseDelegateProperty : public TProperty_MulticastDelegate<FSparseDelegate> 
{
	DECLARE_FIELD_API(FMulticastSparseDelegateProperty, TProperty_MulticastDelegate<FSparseDelegate>, CASTCLASS_FMulticastSparseDelegateProperty, UE_API)

public:

	FMulticastSparseDelegateProperty(FFieldVariant InOwner, const FName& InName)
		: TProperty_MulticastDelegate(InOwner, InName)
	{
	}
	UE_DEPRECATED(5.8, "FMulticastSparseDelegateProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FMulticastSparseDelegateProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FMulticastSparseDelegateProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	UE_API FMulticastSparseDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FMulticastSparseDelegateProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		UE::CodeGen::ConstInit::TObjectRefParam<UFunction> InSignatureFunction
	)
		: Super(InBaseParams, InSignatureFunction)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit FMulticastSparseDelegateProperty(UField* InField)
		: TProperty_MulticastDelegate(InField)
	{
	}
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	// End of FProperty interface

	// FMulticastDelegateProperty interface
	UE_API virtual const FMulticastScriptDelegate* GetMulticastDelegate(const void* PropertyValue) const override;
	UE_API virtual void SetMulticastDelegate(void* PropertyValue, FMulticastScriptDelegate ScriptDelegate) const override;

	UE_API virtual void AddDelegate(FScriptDelegate ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;
	UE_API virtual void RemoveDelegate(const FScriptDelegate& ScriptDelegate, UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;
	UE_API virtual void ClearDelegate(UObject* Parent = nullptr, void* PropertyValue = nullptr) const override;

protected:
	UE_API virtual FMulticastScriptDelegate& GetMulticastScriptDelegate(const void* PropertyValue, int32 Index) const;
	// End of FMulticastDelegateProperty interface

private:
	UE_API virtual void SerializeItemInternal(FArchive& Ar, void* Value, void const* Defaults) const;
};

/** Describes a single node in a custom property list. */
struct FCustomPropertyListNode
{
	/** The property that's being referenced at this node. */
	FProperty* Property;

	/** Used to identify which array index is specifically being referenced if this is an array property. Defaults to 0. */
	int32 ArrayIndex;

	/** If this node represents a struct property, this may contain a "sub" property list for the struct itself. */
	struct FCustomPropertyListNode* SubPropertyList;

	/** Points to the next node in the list. */
	struct FCustomPropertyListNode* PropertyListNext;

	/** Default constructor. */
	FCustomPropertyListNode(FProperty* InProperty = nullptr, int32 InArrayIndex = 0)
		:Property(InProperty)
		, ArrayIndex(InArrayIndex)
		, SubPropertyList(nullptr)
		, PropertyListNext(nullptr)
	{
	}

	/** Convenience method to return the next property in the list and advance the given ptr. */
	inline static FProperty* GetNextPropertyAndAdvance(const FCustomPropertyListNode*& Node)
	{
		if (Node)
		{
			Node = Node->PropertyListNext;
		}

		return Node ? Node->Property : nullptr;
	}
};


/**
 * This class represents the chain of member properties leading to an internal struct property.  It is used
 * for tracking which member property corresponds to the UScriptStruct that owns a particular property.
 */
class FEditPropertyChain : public TDoubleLinkedList<FProperty*>
{

public:
	/** Constructors */
	FEditPropertyChain() : ActivePropertyNode(NULL), ActiveMemberPropertyNode(NULL), bFilterAffectedInstances(false) {}

	/**
	 * Sets the ActivePropertyNode to the node associated with the property specified.
	 *
	 * @param	NewActiveProperty	the FProperty that is currently being evaluated by Pre/PostEditChange
	 *
	 * @return	true if the ActivePropertyNode was successfully changed to the node associated with the property
	 *			specified.  false if there was no node corresponding to that property.
	 */
	UE_API bool SetActivePropertyNode( FProperty* NewActiveProperty );

	/**
	 * Sets the ActiveMemberPropertyNode to the node associated with the property specified.
	 *
	 * @param	NewActiveMemberProperty		the member FProperty which contains the property currently being evaluated
	 *										by Pre/PostEditChange
	 *
	 * @return	true if the ActiveMemberPropertyNode was successfully changed to the node associated with the
	 *			property specified.  false if there was no node corresponding to that property.
	 */
	UE_API bool SetActiveMemberPropertyNode( FProperty* NewActiveMemberProperty );

	/**
	 * Specify the set of archetype instances that will be affected by the property change.
	 */
	template<typename T>
	void SetAffectedArchetypeInstances( T&& InAffectedInstances )
	{
		bFilterAffectedInstances = true;
		AffectedInstances = Forward<T>(InAffectedInstances);
	}
	
	/**
	 * Returns whether the specified archetype instance will be affected by the property change.
	 */
	UE_API bool IsArchetypeInstanceAffected( UObject* InInstance ) const;

	/**
	 * Returns the node corresponding to the currently active property.
	 */
	UE_API TDoubleLinkedListNode* GetActiveNode() const;

	/**
	 * Returns the node corresponding to the currently active property, or if the currently active property
	 * is not a member variable (i.e. inside of a struct/array), the node corresponding to the member variable
	 * which contains the currently active property.
	 */
	UE_API TDoubleLinkedListNode* GetActiveMemberNode() const;

protected:
	/**
	 * In a hierarchy of properties being edited, corresponds to the property that is currently
	 * being processed by Pre/PostEditChange
	 */
	TDoubleLinkedListNode* ActivePropertyNode;

	/**
	 * In a hierarchy of properties being edited, corresponds to the class member property which
	 * contains the property that is currently being processed by Pre/PostEditChange.  This will
	 * only be different from the ActivePropertyNode if the active property is contained within a struct,
	 * dynamic array, or static array.
	 */
	TDoubleLinkedListNode* ActiveMemberPropertyNode;

	/**
	 * Archetype instances that will be affected by the property change.
	 */
	TSet<UObject*> AffectedInstances;

	/**
	 * Assume all archetype instances are affected unless a set of affected instances is provided.
	 */
	bool bFilterAffectedInstances;

	/** TDoubleLinkedList interface */
	/**
	 * Updates the size reported by Num().  Child classes can use this function to conveniently
	 * hook into list additions/removals.
	 *
	 * This version ensures that the ActivePropertyNode and ActiveMemberPropertyNode point to a valid nodes or NULL if this list is empty.
	 *
	 * @param	NewListSize		the new size for this list
	 */
	UE_API virtual void SetListSize( int32 NewListSize );
};


//-----------------------------------------------------------------------------
//EPropertyNodeFlags - Flags used internally by property editors
//-----------------------------------------------------------------------------
namespace EPropertyChangeType
{
	typedef uint32 Type;

	//default value.  Add new enums to add new functionality.
	inline const Type Unspecified		= 1 << 0;
	//Array Add
	inline const Type ArrayAdd			= 1 << 1;
	//Array Remove
	inline const Type ArrayRemove		= 1 << 2;
	//Array Clear
	inline const Type ArrayClear		= 1 << 3;
	//Value Set
	inline const Type ValueSet			= 1 << 4;
	//Duplicate
	inline const Type Duplicate			= 1 << 5;
	//Interactive, e.g. dragging a slider. Will be followed by a ValueSet when finished.
	inline const Type Interactive		= 1 << 6;
	//Redirected.  Used when property references are updated due to content hot-reloading, or an asset being replaced during asset deletion (aka, asset consolidation).
	inline const Type Redirected		= 1 << 7;
	// Array Item Moved Within the Array
	inline const Type ArrayMove			= 1 << 8;
	// Edit Condition State has changed
	inline const Type ToggleEditable	= 1 << 9;
	//
	inline const Type ResetToDefault    = 1 << 10;
};

/**
 * Structure for passing pre and post edit change events
 */
struct FPropertyChangedEvent
{
	FPropertyChangedEvent(FProperty* InProperty, EPropertyChangeType::Type InChangeType = EPropertyChangeType::Unspecified, TArrayView<const UObject* const> InTopLevelObjects = TArrayView<const UObject* const>())
		: Property(InProperty)
		, MemberProperty(InProperty)
		, ChangeType(InChangeType)
		, ObjectIteratorIndex(INDEX_NONE)
		, bFilterChangedInstances(false)
		, TopLevelObjects(InTopLevelObjects)
	{
	}

	void SetActiveMemberProperty( FProperty* InActiveMemberProperty )
	{
		MemberProperty = InActiveMemberProperty;
	}

	/**
	 * Saves off map of array indices per object being set.
	 */
	void SetArrayIndexPerObject(TArrayView<const TMap<FString, int32>> InArrayIndices)
	{
		ArrayIndicesPerObject = InArrayIndices; 
	}

	/**
	 * Specify the set of archetype instances that were modified by the property change.
	 */
	template<typename T>
	void SetInstancesChanged(T&& InInstancesChanged)
	{
		bFilterChangedInstances = true;
		InstancesChanged = Forward<T>(InInstancesChanged);
	}

	bool GetArrayIndicesPerObject(int32 InObjectIteratorIndex, TMap<FString, int32>& OutArrayIndicesPerObject) const
	{
		if (!(ArrayIndicesPerObject.IsValidIndex(InObjectIteratorIndex)))
		{
			return false;
		}

		OutArrayIndicesPerObject = ArrayIndicesPerObject[InObjectIteratorIndex];
		return true;
	}
	
	/**
	 * Gets the Array Index of the "current object" based on a particular name
	 * InName - Name of the property to find the array index for
	 */
	int32 GetArrayIndex(const FString& InName) const
	{
		//default to unknown index
		int32 Retval = -1;
		if (ArrayIndicesPerObject.IsValidIndex(ObjectIteratorIndex))
		{
			const int32* ValuePtr = ArrayIndicesPerObject[ObjectIteratorIndex].Find(InName);
			if (ValuePtr)
			{
				Retval = *ValuePtr;
			}
		}
		return Retval;
	}

	/**
	 * Test whether an archetype instance was modified.
	 * InInstance - The instance we want to know the status.
	 */
	bool HasArchetypeInstanceChanged(UObject* InInstance) const
	{
		return !bFilterChangedInstances || InstancesChanged.Contains(InInstance);
	}

	/**
	 * @return The number of objects being edited during this change event
	 */
	int32 GetNumObjectsBeingEdited() const { return TopLevelObjects.Num(); }

	/**
	 * Gets an object being edited by this change event.  Multiple objects could be edited at once
	 *
	 * @param Index	The index of the object being edited. Assumes index is valid.  Call GetNumObjectsBeingEdited first to check if there are valid objects
	 * @return The object being edited or nullptr if no object was found
	 */
	const UObject* GetObjectBeingEdited(int32 Index) const { return TopLevelObjects[Index]; }

	/**
	 * Simple utility to get the name of the property and takes care of the possible null property.
	 */
	FName GetPropertyName() const
	{
		return (Property != nullptr) ? Property->GetFName() : NAME_None;
	}

	/**
	 * Simple utility to get the name of the object's member property and takes care of the possible null property.
	 */
	FName GetMemberPropertyName() const
	{
		return (MemberProperty != nullptr) ? MemberProperty->GetFName() : NAME_None;
	}

	/**
	 * The actual property that changed
	 */
	FProperty* Property;

	/**
	 * The member property of the object that PostEditChange is being called on.  
	 * For example if the property that changed is inside a struct on the object, this property is the struct property
	 */
	FProperty* MemberProperty;

	// The kind of change event that occurred
	EPropertyChangeType::Type ChangeType;

	// Used by the param system to say which object is receiving the event in the case of multi-select
	int32 ObjectIteratorIndex;
private:
	//In the property window, multiple objects can be selected at once.  In the case of adding/inserting to an array, each object COULD have different indices for the new entries in the array
	TArrayView<const TMap<FString, int32>> ArrayIndicesPerObject;
	
	//In the property window, multiple objects can be selected at once. In this case we want to know if an instance was updated for this operation (used in array/set/map context)
	TSet<UObject*> InstancesChanged;

	//Assume all archetype instances were changed unless a set of changed instances is provided.
	bool bFilterChangedInstances;

	/** List of top level objects being changed */
	TArrayView<const UObject* const> TopLevelObjects;
};

/**
 * Structure for passing pre and post edit change events
 */
struct FPropertyChangedChainEvent : public FPropertyChangedEvent
{
	FPropertyChangedChainEvent(FEditPropertyChain& InPropertyChain, const FPropertyChangedEvent& SrcChangeEvent) :
		FPropertyChangedEvent(SrcChangeEvent),
		PropertyChain(InPropertyChain)
	{
	}
	FEditPropertyChain& PropertyChain;
};

namespace UEProperty_Private
{
	using FPropertyListBuilderPropertyLink = TLinkedListBuilder<FProperty, TLinkedListBuilderNextLinkMemberVar<FProperty, &FProperty::PropertyLinkNext>>;
	using FPropertyListBuilderRefLink = TLinkedListBuilder<FProperty, TLinkedListBuilderNextLinkMemberVar<FProperty, &FProperty::NextRef>>;
	using FPropertyListBuilderDestructorLink = TLinkedListBuilder<FProperty, TLinkedListBuilderNextLinkMemberVar<FProperty, &FProperty::DestructorLinkNext>>;
	using FPropertyListBuilderPostConstructLink = TLinkedListBuilder<FProperty, TLinkedListBuilderNextLinkMemberVar<FProperty, &FProperty::PostConstructLinkNext>>;
}

/*-----------------------------------------------------------------------------
TFieldIterator.
-----------------------------------------------------------------------------*/

/** TFieldIterator construction flags */
enum class EFieldIterationFlags : uint8
{
	None = 0,
	IncludeSuper = 1<<0,		// Include super class
	IncludeDeprecated = 1<<1,	// Include deprecated properties
	IncludeInterfaces = 1<<2,	// Include interfaces

	IncludeAll = IncludeSuper | IncludeDeprecated | IncludeInterfaces,

	Default = IncludeSuper | IncludeDeprecated,
};
ENUM_CLASS_FLAGS(EFieldIterationFlags);

/** Old-style TFieldIterator construction flags */
namespace EFieldIteratorFlags
{
	enum SuperClassFlags
	{
		ExcludeSuper = (uint8)EFieldIterationFlags::None,
		IncludeSuper = (uint8)EFieldIterationFlags::IncludeSuper,
	};

	enum DeprecatedPropertyFlags
	{
		ExcludeDeprecated = (uint8)EFieldIterationFlags::None,
		IncludeDeprecated = (uint8)EFieldIterationFlags::IncludeDeprecated,
	};

	enum InterfaceClassFlags
	{
		ExcludeInterfaces = (uint8)EFieldIterationFlags::None,
		IncludeInterfaces = (uint8)EFieldIterationFlags::IncludeInterfaces,
	};
}

template <class FieldType>
FieldType* GetChildFieldsFromStruct(const UStruct* Owner)
{
	check(false);
	return nullptr;
}

template <>
inline UField* GetChildFieldsFromStruct(const UStruct* Owner)
{
	return Owner->Children;
}

template <>
inline FField* GetChildFieldsFromStruct(const UStruct* Owner)
{
	return Owner->ChildProperties;
}


//
// For iterating through a linked list of fields.
//
template <class T>
class TFieldIterator
{
private:
	/** The object being searched for the specified field */
	const UStruct* Struct;
	/** The current location in the list of fields being iterated */
	typename T::BaseFieldClass* Field;
	/** The index of the current interface being iterated */
	int32 InterfaceIndex;
	/** Whether to include the super class or not */
	const bool bIncludeSuper;
	/** Whether to include deprecated fields or not */
	const bool bIncludeDeprecated;
	/** Whether to include interface fields or not */
	const bool bIncludeInterface;

public:
	TFieldIterator(const UStruct* InStruct, EFieldIterationFlags InIterationFlags = EFieldIterationFlags::Default)
		: Struct            ( InStruct )
		, Field             ( InStruct ? GetChildFieldsFromStruct<typename T::BaseFieldClass>(InStruct) : NULL )
		, InterfaceIndex    ( -1 )
		, bIncludeSuper     ( EnumHasAnyFlags(InIterationFlags, EFieldIterationFlags::IncludeSuper) )
		, bIncludeDeprecated( EnumHasAnyFlags(InIterationFlags, EFieldIterationFlags::IncludeDeprecated) )
		, bIncludeInterface ( EnumHasAnyFlags(InIterationFlags, EFieldIterationFlags::IncludeInterfaces) && InStruct && InStruct->IsA(UClass::StaticClass()) )
	{
		IterateToNext();
	}

	/** Legacy version taking the flags as 3 separate values */
	TFieldIterator(const UStruct*                               InStruct,
	               EFieldIteratorFlags::SuperClassFlags         InSuperClassFlags,
	               EFieldIteratorFlags::DeprecatedPropertyFlags InDeprecatedFieldFlags = EFieldIteratorFlags::IncludeDeprecated,
	               EFieldIteratorFlags::InterfaceClassFlags     InInterfaceFieldFlags  = EFieldIteratorFlags::ExcludeInterfaces)
		: TFieldIterator(InStruct, (EFieldIterationFlags)(InSuperClassFlags | InDeprecatedFieldFlags | InInterfaceFieldFlags))
	{
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return Field != NULL; 
	}
	/** inverse of the "bool" operator */
	UE_FORCEINLINE_HINT bool operator !() const 
	{
		return !(bool)*this;
	}

	inline bool operator==(const TFieldIterator<T>& Rhs) const { return Field == Rhs.Field; }
	inline bool operator!=(const TFieldIterator<T>& Rhs) const { return Field != Rhs.Field; }

	inline void operator++()
	{
		checkSlow(Field);
		Field = Field->Next;
		IterateToNext();
	}
	inline T* operator*()
	{
		checkSlow(Field);
		return (T*)Field;
	}
	inline const T* operator*() const
	{
		checkSlow(Field);
		return (const T*)Field;
	}
	inline T* operator->()
	{
		checkSlow(Field);
		return (T*)Field;
	}
	inline const UStruct* GetStruct()
	{
		return Struct;
	}
protected:
	inline void IterateToNext()
	{
		typename T::BaseFieldClass* CurrentField  = Field;
		const UStruct* CurrentStruct = Struct;

		while (CurrentStruct)
		{
			for (; CurrentField; CurrentField = CurrentField->Next)
			{
				typename T::FieldTypeClass* FieldClass = CurrentField->GetClass();

				if (!FieldClass->HasAllCastFlags(T::StaticClassCastFlags()))
				{
					continue;
				}

				if (FieldClass->HasAllCastFlags(CASTCLASS_FProperty))
				{
					FProperty* Prop = (FProperty*)CurrentField;
					if (Prop->HasAllPropertyFlags(CPF_Deprecated) && !bIncludeDeprecated)
					{
						continue;
					}
				}

				Struct = CurrentStruct;
				Field = CurrentField;
				return;
			}

			if (bIncludeInterface)
			{
				// We shouldn't be able to get here for non-classes
				UClass* CurrentClass = (UClass*)CurrentStruct;
				++InterfaceIndex;
				if (InterfaceIndex < CurrentClass->Interfaces.Num())
				{
					FImplementedInterface& Interface = CurrentClass->Interfaces[InterfaceIndex];
					CurrentField = Interface.Class ? GetChildFieldsFromStruct<typename T::BaseFieldClass>(Interface.Class) : nullptr;
					continue;
				}
			}

			if (bIncludeSuper)
			{
				CurrentStruct = CurrentStruct->GetInheritanceSuper();
				if (CurrentStruct)
				{
					CurrentField   = GetChildFieldsFromStruct<typename T::BaseFieldClass>(CurrentStruct);
					InterfaceIndex = -1;
					continue;
				}
			}

			break;
		}

		Struct = CurrentStruct;
		Field  = CurrentField;
	}
};

template <typename T>
struct TFieldRange
{
	TFieldRange(const UStruct* InStruct, EFieldIterationFlags InIterationFlags = EFieldIterationFlags::Default)
		: Begin(InStruct, InIterationFlags)
	{
	}

	/** Legacy version taking the flags as 3 separate values */
	TFieldRange(const UStruct*                               InStruct,
	            EFieldIteratorFlags::SuperClassFlags         InSuperClassFlags,
	            EFieldIteratorFlags::DeprecatedPropertyFlags InDeprecatedFieldFlags = EFieldIteratorFlags::IncludeDeprecated,
	            EFieldIteratorFlags::InterfaceClassFlags     InInterfaceFieldFlags  = EFieldIteratorFlags::ExcludeInterfaces)
		: TFieldRange(InStruct, (EFieldIterationFlags)(InSuperClassFlags | InDeprecatedFieldFlags | InInterfaceFieldFlags))
	{
	}

	friend TFieldIterator<T> begin(const TFieldRange& Range) { return Range.Begin; }
	friend TFieldIterator<T> end  (const TFieldRange& Range) { return TFieldIterator<T>(NULL); }

	TFieldIterator<T> Begin;
};

/*-----------------------------------------------------------------------------
	Field templates.
-----------------------------------------------------------------------------*/

template <UE::CDerivedFrom<UField> T>
T* FindUField(const UStruct* Owner, FName FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	static_assert(sizeof(T) > 0, "T must not be an incomplete type");

	// We know that a "none" field won't exist in this Struct
	if (FieldName.IsNone())
	{
		return nullptr;
	}

	// Search by comparing FNames (INTs), not strings
	for (TFieldIterator<T>It(Owner, IterationFlags); It; ++It)
	{
		if (It->GetFName() == FieldName)
		{
			return *It;
		}
	}

	// If we didn't find it, return no field
	return nullptr;
}

template <UE::CDerivedFrom<UField> T>
T* FindUField(const UStruct* Owner, const TCHAR* FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	static_assert(sizeof(T) > 0, "T must not be an incomplete type");

	// lookup the string name in the Name hash
	FName Name(FieldName, FNAME_Find);
	return FindUField<T>(Owner, Name, IterationFlags);
}

template <UE::CDerivedFrom<FField> T>
T* FindFProperty(const UStruct* Owner, FName FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	static_assert(sizeof(T) > 0, "T must not be an incomplete type");

	// We know that a "none" field won't exist in this Struct
	if (FieldName.IsNone())
	{
		return nullptr;
	}

	// Search by comparing FNames (INTs), not strings
	for (TFieldIterator<T>It(Owner, IterationFlags); It; ++It)
	{
		if (It->GetFName() == FieldName)
		{
			return *It;
		}
	}

	// If we didn't find it, return no field
	return nullptr;
}

template <UE::CDerivedFrom<FField> T>
T* FindFProperty(const UStruct* Owner, const TCHAR* FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	static_assert(sizeof(T) > 0, "T must not be an incomplete type");

	// lookup the string name in the Name hash
	FName Name(FieldName, FNAME_Find);
	return FindFProperty<T>(Owner, Name, IterationFlags);
}

/** Finds FProperties or UFunctions and UEnums */
inline FFieldVariant FindUFieldOrFProperty(const UStruct* Owner, FName FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	// Look for properties first as they're most often the runtime thing higher level code wants to find
	FFieldVariant Result = FindFProperty<FProperty>(Owner, FieldName, IterationFlags);
	if (!Result)
	{
		Result = FindUField<UField>(Owner, FieldName, IterationFlags);
	}
	return Result;
}

/** Finds FProperties or UFunctions and UEnums */
inline FFieldVariant FindUFieldOrFProperty(const UStruct* Owner, const TCHAR* FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	// lookup the string name in the Name hash
	FName Name(FieldName, FNAME_Find);
	return FindUFieldOrFProperty(Owner, Name, IterationFlags);
}

template <UE::CDerivedFrom<UField> T>
T* FindUFieldOrFProperty(const UStruct* Owner, FName FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	return FindUField<T>(Owner, FieldName, IterationFlags);
}

template <UE::CDerivedFrom<UField> T>
T* FindUFieldOrFProperty(const UStruct* Owner, const TCHAR* FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	return FindUField<T>(Owner, FieldName, IterationFlags);
}

template <UE::CDerivedFrom<FField> T>
T* FindUFieldOrFProperty(const UStruct* Owner, FName FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	return FindFProperty<T>(Owner, FieldName, IterationFlags);
}

template <UE::CDerivedFrom<FField> T>
T* FindUFieldOrFProperty(const UStruct* Owner, const TCHAR* FieldName, EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default)
{
	return FindFProperty<T>(Owner, FieldName, IterationFlags);
}

/**
 * Search for the named field within the specified scope, including any Outer classes; assert on failure.
 *
 * @param	Scope		the scope to search for the field in
 * @param	FieldName	the name of the field to search for.
 */
template<typename T>
T* FindFieldChecked( const UStruct* Scope, FName FieldName )
{
	if ( FieldName != NAME_None && Scope != NULL )
	{
		const UStruct* InitialScope = Scope;
		for ( ; Scope != NULL; Scope = dynamic_cast<const UStruct*>(Scope->GetOuter()) )
		{
			for ( TFieldIterator<T> It(Scope); It; ++It )
			{
				if ( It->GetFName() == FieldName )
				{
					return *It;
				}
			}
		}
	
		UE_LOGF( LogType, Fatal, "Failed to find %ls %ls in %ls", *T::StaticClass()->GetName(), *FieldName.ToString(), *InitialScope->GetFullName() );
	}

	return NULL;
}

/*-----------------------------------------------------------------------------*
 * PropertyValueIterator
 *-----------------------------------------------------------------------------*/

/** FPropertyValueIterator construction flags */
enum class EPropertyValueIteratorFlags : uint8
{
	NoRecursion = 0,	// Don't recurse at all, only do top level properties
	FullRecursion = 1,	// Recurse into containers and structs
};

/** For recursively iterating over a UStruct to find nested FProperty pointers and values */
class FPropertyValueIterator
{
public:
	using BasePairType = TPair<const FProperty*, const void*>;

	/** 
	 * Construct an iterator using a struct and struct value
	 *
	 * @param InPropertyClass	The UClass of the FProperty type you are looking for
	 * @param InStruct			The UClass or UScriptStruct containing properties to search for
	 * @param InStructValue		Address in memory of struct to search for property values
	 * @param InRecursionFlags	Rather to recurse into container and struct properties
	 * @param InDeprecatedPropertyFlags	Rather to iterate over deprecated properties
	 */
	COREUOBJECT_API explicit FPropertyValueIterator(FFieldClass* InPropertyClass, const UStruct* InStruct, const void* InStructValue,
		EPropertyValueIteratorFlags						InRecursionFlags = EPropertyValueIteratorFlags::FullRecursion,
		EFieldIteratorFlags::DeprecatedPropertyFlags	InDeprecatedPropertyFlags = EFieldIteratorFlags::IncludeDeprecated);

	/** Invalid iterator, start with empty stack */
	FPropertyValueIterator()
		: PropertyClass(nullptr)
		, RecursionFlags(EPropertyValueIteratorFlags::FullRecursion)
		, DeprecatedPropertyFlags(EFieldIteratorFlags::IncludeDeprecated)
		, bSkipRecursionOnce(false)
	{
	}

	/** Conversion to "bool" returning true if the iterator is valid */
	inline explicit operator bool() const
	{
		// If nothing left in the stack, iteration is complete
		if (PropertyIteratorStack.Num() > 0)
		{
			return true;
		}
		return false;
	}

	UE_FORCEINLINE_HINT bool operator==(const FPropertyValueIterator& Rhs) const
	{
		return PropertyIteratorStack == Rhs.PropertyIteratorStack;
	}
	
	UE_FORCEINLINE_HINT bool operator!=(const FPropertyValueIterator& Rhs) const
	{
		return !(PropertyIteratorStack == Rhs.PropertyIteratorStack);
	}

	/** Returns a TPair containing Property/Value currently being iterated */
	inline const BasePairType& operator*() const
	{
		const FPropertyValueStackEntry& Entry = PropertyIteratorStack.Last();
		return Entry.GetPropertyValue();
	}

	inline const BasePairType* operator->() const
	{
		const FPropertyValueStackEntry& Entry = PropertyIteratorStack.Last();
		return &Entry.GetPropertyValue();
	}

	/** Returns Property currently being iterated */
	UE_FORCEINLINE_HINT const FProperty* Key() const 
	{
		return (*this)->Key; 
	}
	
	/** Returns memory address currently being iterated */
	UE_FORCEINLINE_HINT const void* Value() const
	{
		return (*this)->Value;
	}

	/** Increments iterator */
	UE_FORCEINLINE_HINT void operator++()
	{
		IterateToNext();
	}

	/** Call when iterating a recursive property such as Array or Struct to stop it from iterating into that property */
	UE_FORCEINLINE_HINT void SkipRecursiveProperty()
	{
		bSkipRecursionOnce = true;
	}

	/** 
	 * Returns the full stack of properties for the property currently being iterated. This includes struct and container properties
	 *
	 * @param PropertyChain	Filled in with ordered list of Properties, with currently active property first and top parent last
	 */
	COREUOBJECT_API void GetPropertyChain(TArray<const FProperty*>& PropertyChain) const;

	/**
	 * Returns a string of the property chain in a form that makes reading it easy.  The format may change over time,
	 * do not depend on this being in a specific Unreal path format.  It's primary intention is to aid in debugging
	 * and in reporting paths to the end developer in an editor environment.
	 *
	 * Now for the implementation details, normally paths involving an array, will do things like print the array
	 * name twice, since an array in the path is actually 2 properties, one for the array and another for the index.
	 * Maps are similar.  The end result is stuff like seeing, ActionsArray.ActionsArray.ActionIfo.ActionPower.
	 *
	 * What this function does is tries to make this stuff a lot more human readable, by printing,
	 * ActionsArray[3].ActionIfo.ActionPower, so that you know the index it comes from.  Similarly Maps would print
	 * ActionMap["Action Name"].ActionIfo.ActionPower, so that you know the index it comes from.  Similarly Maps would print
	 */
	COREUOBJECT_API FString GetPropertyPathDebugString() const;

private:
	enum class EPropertyValueFlags : uint8
	{
		None = 0x0,
		IsMatch = 0x01,

		IsOptional = 0x08,
		IsArray = 0x10,
		IsMap = 0x20,
		IsSet = 0x40,
		IsStruct = 0x80,

		// When adding a new 'container' (needs to be recursed into) flag here, add it to the EPropertyValueFlags_ContainerMask macro too.
	};
	FRIEND_ENUM_CLASS_FLAGS(EPropertyValueFlags)

	struct FPropertyValueStackEntry
	{
		/** Address of owning UStruct or FProperty container */
		const void* Owner = nullptr;
		
		/** List of current root property+value pairs for the current top level FProperty */
		typedef TPair<BasePairType, EPropertyValueFlags> BasePairAndFlags;
		typedef TArray<BasePairAndFlags, TInlineAllocator<8>> FValueArrayType;
		FValueArrayType ValueArray;

		/** Current position inside ValueArray */
		int32 ValueIndex = -1;

		/** Next position inside ValueArray */
		int32 NextValueIndex = 0;

		FPropertyValueStackEntry(const void* InValue)
			: Owner(InValue)
		{}

		FPropertyValueStackEntry(const UStruct* InStruct, const void* InValue, EFieldIteratorFlags::DeprecatedPropertyFlags InDeprecatedPropertyFlags)
			: Owner(InValue)
		{}

		UE_FORCEINLINE_HINT bool operator==(const FPropertyValueStackEntry& Rhs) const
		{
			return Owner == Rhs.Owner && ValueIndex == Rhs.ValueIndex;
		}

		UE_FORCEINLINE_HINT const BasePairType& GetPropertyValue() const
		{
			// Index has to be valid to get this far
			return ValueArray[ValueIndex].Key;
		}
	};

	/** Internal stack, one per continer/struct */
	TArray<FPropertyValueStackEntry, TInlineAllocator<8>> PropertyIteratorStack;

	/** Property type that is explicitly checked for */
	FFieldClass* PropertyClass = nullptr;

	/** Whether to recurse into containers/structs */
	const EPropertyValueIteratorFlags RecursionFlags;

	/** Inherits to child field iterator */
	const EFieldIteratorFlags::DeprecatedPropertyFlags DeprecatedPropertyFlags;

	/** If true, next iteration will skip recursing into containers/structs */
	bool bSkipRecursionOnce = false;

	/** If true, all properties will be matched without checking IsA(PropertyClass) */
	bool bMatchAll = false;

	/** Returns EPropertyValueFlags to describe if this Property is a match and/or a container/struct */
	EPropertyValueFlags GetPropertyValueFlags(const FProperty* Property) const;

	/** Fills the Entry.ValueArray with all relevant properties found in Struct */
	void FillStructProperties(const UStruct* Struct, FPropertyValueStackEntry& Entry);

	/**
	 * Goes to the next Property/value pair.
	 * Returns false on a match or out of properties, true when iteration should continue
	 */
	bool NextValue(EPropertyValueIteratorFlags RecursionFlags);

	/** Iterates to next property being checked for or until reaching the end of the structure */
	COREUOBJECT_API void IterateToNext();
};

/** Templated version, will verify the property type is correct and will skip any properties that are not */
template <class T>
class TPropertyValueIterator : public FPropertyValueIterator
{
public:
	using PairType = TPair<const T*, const void*>;
	
	/** 
	 * Construct an iterator using a struct and struct value
	 *
	 * @param InStruct			The UClass or UScriptStruct containing properties to search for
	 * @param InStructValue		Address in memory of struct to search for property values
	 * @param InRecursionFlags	Rather to recurse into container and struct properties
	 * @param InDeprecatedPropertyFlags	Rather to iterate over deprecated properties
	 */
	explicit TPropertyValueIterator(const UStruct* InStruct, const void* InStructValue,
		EPropertyValueIteratorFlags						InRecursionFlags = EPropertyValueIteratorFlags::FullRecursion,
		EFieldIteratorFlags::DeprecatedPropertyFlags	InDeprecatedPropertyFlags = EFieldIteratorFlags::IncludeDeprecated)
		: FPropertyValueIterator(T::StaticClass(), InStruct, InStructValue, InRecursionFlags, InDeprecatedPropertyFlags)
	{
	}

	/** Invalid iterator, start with empty stack */
	TPropertyValueIterator() = default;

	/** Returns a TPair containing Property/Value currently being iterated */
	UE_FORCEINLINE_HINT const PairType& operator*() const
	{
		return (const PairType&)FPropertyValueIterator::operator*();
	}

	UE_FORCEINLINE_HINT const PairType* operator->() const
	{
		return (const PairType*)FPropertyValueIterator::operator->();
	}

	/** Returns Property currently being iterated */
	UE_FORCEINLINE_HINT const T* Key() const
	{
		return (*this)->Key;
	}
};

/** Templated range to allow ranged-for syntax */
template <class T>
struct TPropertyValueRange
{
	/** 
	 * Construct a range using a struct and struct value
	 *
	 * @param InStruct			The UClass or UScriptStruct containing properties to search for
	 * @param InStructValue		Address in memory of struct to search for property values
	 * @param InRecursionFlags	Rather to recurse into container and struct properties
	 * @param InDeprecatedPropertyFlags	Rather to iterate over deprecated properties
	 */
	TPropertyValueRange(const UStruct* InStruct, const void* InStructValue,
		EPropertyValueIteratorFlags						InRecursionFlags = EPropertyValueIteratorFlags::FullRecursion,
		EFieldIteratorFlags::DeprecatedPropertyFlags	InDeprecatedPropertyFlags = EFieldIteratorFlags::IncludeDeprecated)
		: Begin(InStruct, InStructValue, InRecursionFlags, InDeprecatedPropertyFlags)
	{
	}

	friend TPropertyValueIterator<T> begin(const TPropertyValueRange& Range) { return Range.Begin; }
	friend TPropertyValueIterator<T> end(const TPropertyValueRange& Range) { return TPropertyValueIterator<T>(); }

	TPropertyValueIterator<T> Begin;
};

/**
 * Determine if this object has SomeObject in its archetype chain.
 */
inline bool UObject::IsBasedOnArchetype(  const UObject* const SomeObject ) const
{
	checkfSlow(this, TEXT("IsBasedOnArchetype() is called on a null pointer. Fix the call site."));
	if ( SomeObject != this )
	{
		for ( UObject* Template = GetArchetype(); Template; Template = Template->GetArchetype() )
		{
			if ( SomeObject == Template )
			{
				return true;
			}
		}
	}

	return false;
}


/*-----------------------------------------------------------------------------
	C++ property macros.
-----------------------------------------------------------------------------*/

static_assert(sizeof(bool) == sizeof(uint8), "Bool is not one byte.");

/** helper to calculate an array's dimensions **/
#define CPP_ARRAY_DIM(ArrayName, ClassName) \
	(sizeof(((ClassName*)0)->ArrayName) / sizeof(((ClassName*)0)->ArrayName[0]))

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "HAL/PlatformMath.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/EnableIf.h"
#include "Templates/Greater.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsSigned.h"
#include "UObject/StrProperty.h"
#endif

// Types extracted from UnrealType.h that need to be in their own header because of the rule that .generated.h must be the last included file 
#include "PropertyWrapper.h"
