// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealType.h: Unreal engine base type definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UnrealType.h"

#include "UnrealTypePrivate.generated.h"

#define USE_UPROPERTY_LOAD_DEFERRING (USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING && WITH_EDITORONLY_DATA)

UCLASS(Abstract, Config=Engine)
class UProperty : public UField
{
	GENERATED_BODY()

public:

	// Persistent variables.
	int32			ArrayDim;
	int32			ElementSize;
	EPropertyFlags	PropertyFlags;
	uint16			RepIndex;

	TEnumAsByte<ELifetimeCondition> BlueprintReplicationCondition;

	// In memory variables (generated during Link()).
	int32		Offset_Internal;

	FName		RepNotifyFunc;

	/** In memory only: Linked list of properties from most-derived to base **/
	UProperty*	PropertyLinkNext;
	/** In memory only: Linked list of object reference properties from most-derived to base **/
	UProperty*  NextRef;
	/** In memory only: Linked list of properties requiring destruction. Note this does not include things that will be destroyed by the native destructor **/
	UProperty*	DestructorLinkNext;
	/** In memory only: Linked list of properties requiring post constructor initialization.**/
	UProperty*	PostConstructLinkNext;


	// Constructors.
	COREUOBJECT_API UProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	COREUOBJECT_API UProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags);
	COREUOBJECT_API UProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags );

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface

	/**
	 * Returns the first UProperty in this property's Outer chain that does not have a UProperty for an Outer
	 */
	UProperty* GetOwnerProperty()
	{
		UProperty* Result=this;
		for (UProperty* PropBase = dynamic_cast<UProperty*>(GetOuter()); PropBase; PropBase = dynamic_cast<UProperty*>(PropBase->GetOuter()))
		{
			Result = PropBase;
		}
		return Result;
	}

	const UProperty* GetOwnerProperty() const
	{
		const UProperty* Result = this;
		for (UProperty* PropBase = dynamic_cast<UProperty*>(GetOuter()); PropBase; PropBase = dynamic_cast<UProperty*>(PropBase->GetOuter()))
		{
			Result = PropBase;
		}
		return Result;
	}

	UE_FORCEINLINE_HINT bool HasAnyPropertyFlags( uint64 FlagsToCheck ) const
	{
		return (PropertyFlags & FlagsToCheck) != 0 || FlagsToCheck == CPF_AllFlags;
	}
	/**
	 * Used to safely check whether all of the passed in flags are set. This is required
	 * as PropertyFlags currently is a 64 bit data type and bool is a 32 bit data type so
	 * simply using PropertyFlags&CPF_MyFlagBiggerThanMaxInt won't work correctly when
	 * assigned directly to an bool.
	 *
	 * @param FlagsToCheck	Object flags to check for
	 *
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	UE_FORCEINLINE_HINT bool HasAllPropertyFlags( uint64 FlagsToCheck ) const
	{
		return ((PropertyFlags & FlagsToCheck) == FlagsToCheck);
	}

	/**
	 * Editor-only properties are those that only are used with the editor is present or cannot be removed from serialisation.
	 * Editor-only properties include: EditorOnly properties
	 * Properties that cannot be removed from serialisation are:
	 *		Boolean properties (may affect GCC_BITFIELD_MAGIC computation)
	 *		Native properties (native serialisation)
	 */
	UE_FORCEINLINE_HINT bool IsEditorOnlyProperty() const
	{
		return (PropertyFlags & CPF_DevelopmentAssets) != 0;
	}

private:

	inline void* ContainerVoidPtrToValuePtrInternal(void* ContainerPtr, int32 ArrayIndex) const
	{
		check(ArrayIndex < ArrayDim);
		check(ContainerPtr);

		if (0)
		{
			// in the future, these checks will be tested if the property is NOT relative to a UClass
			check(!Cast<UClass>(GetOuter())); // Check we are _not_ calling this on a direct child property of a UClass, you should pass in a UObject* in that case
		}

		return (uint8*)ContainerPtr + Offset_Internal + ElementSize * ArrayIndex;
	}

	inline void* ContainerUObjectPtrToValuePtrInternal(UObject* ContainerPtr, int32 ArrayIndex) const
	{
		check(ArrayIndex < ArrayDim);
		check(ContainerPtr);

		// in the future, these checks will be tested if the property is supposed be from a UClass
		// need something for networking, since those are NOT live uobjects, just memory blocks
		check(((UObject*)ContainerPtr)->IsValidLowLevel()); // Check its a valid UObject that was passed in
		check(((UObject*)ContainerPtr)->GetClass() != NULL);
		check(GetOuter()->IsA(UClass::StaticClass())); // Check that the outer of this property is a UClass (not another property)

		// Check that the object we are accessing is of the class that contains this property
		checkf(((UObject*)ContainerPtr)->IsA((UClass*)GetOuter()), TEXT("'%s' is of class '%s' however property '%s' belongs to class '%s'")
			, *((UObject*)ContainerPtr)->GetName()
			, *((UObject*)ContainerPtr)->GetClass()->GetName()
			, *GetName()
			, *((UClass*)GetOuter())->GetName());

		if (0)
		{
			// in the future, these checks will be tested if the property is NOT relative to a UClass
			check(!GetOuter()->IsA(UClass::StaticClass())); // Check we are _not_ calling this on a direct child property of a UClass, you should pass in a UObject* in that case
		}

		return (uint8*)ContainerPtr + Offset_Internal + ElementSize * ArrayIndex;
	}

public:

	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType* ContainerPtrToValuePtr(UObject* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return (ValueType*)ContainerUObjectPtrToValuePtrInternal(ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType* ContainerPtrToValuePtr(void* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return (ValueType*)ContainerVoidPtrToValuePtrInternal(ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType const* ContainerPtrToValuePtr(UObject const* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return ContainerPtrToValuePtr<ValueType>((UObject*)ContainerPtr, ArrayIndex);
	}
	template<typename ValueType>
	UE_FORCEINLINE_HINT ValueType const* ContainerPtrToValuePtr(void const* ContainerPtr, int32 ArrayIndex = 0) const
	{
		return ContainerPtrToValuePtr<ValueType>((void*)ContainerPtr, ArrayIndex);
	}

#if WITH_EDITORONLY_DATA

	FField* AssociatedField;

	COREUOBJECT_API virtual FField* GetAssociatedFField() override;
	COREUOBJECT_API virtual void SetAssociatedFField(FField* InField) override;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(Abstract, Config = Engine)
class UNumericProperty : public UProperty
{
	GENERATED_BODY()

public:
	UNumericProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UNumericProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
		: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{}

	UNumericProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
		:	UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{}
};

UCLASS(Config = Engine)
class UByteProperty : public UNumericProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<UEnum> Enum;

	UByteProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UByteProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UEnum* InEnum = nullptr)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	, Enum(InEnum)
	{
	}

	UByteProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UEnum* InEnum = nullptr)
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	, Enum( InEnum )
	{
	}

	// UObject interface.
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class UInt8Property : public UNumericProperty
{
	GENERATED_BODY()

public:
 	UInt8Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UInt8Property(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
		: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UInt8Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
		: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};

UCLASS(Config = Engine)
class UInt16Property : public UNumericProperty
{
	GENERATED_BODY()

public:
 	UInt16Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UInt16Property(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UInt16Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};

UCLASS(Config = Engine)
class UIntProperty : public UNumericProperty
{
	GENERATED_BODY()

public:
 	UIntProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UIntProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UIntProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};

UCLASS(Config = Engine)
class UInt64Property : public UNumericProperty
{
	GENERATED_BODY()

public:
 	UInt64Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UInt64Property(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UInt64Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config = Engine)
class UUInt16Property : public UNumericProperty
{
	GENERATED_BODY()

public:
 	UUInt16Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UUInt16Property(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UUInt16Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config = Engine)
class UUInt32Property : public UNumericProperty
{
	GENERATED_BODY()

public:
	UUInt32Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UUInt32Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config = Engine)
class UUInt64Property : public UNumericProperty
{
	GENERATED_BODY()

public:
	UUInt64Property(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UUInt64Property(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UUInt64Property( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config = Engine)
class UFloatProperty : public UNumericProperty
{
	GENERATED_BODY()

public:
	UFloatProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UFloatProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UFloatProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UNumericProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config=Engine)
class UDoubleProperty : public UNumericProperty
{
	GENERATED_BODY()

public:
	UDoubleProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UDoubleProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UDoubleProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UNumericProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};

UCLASS(Config = Engine)
class UBoolProperty : public UProperty
{
	GENERATED_BODY()

	// Variables.
public:

	/** Size of the bitfield/bool property. Equal to ElementSize but used to check if the property has been properly initialized (0-8, where 0 means uninitialized). */
	uint8 FieldSize;
	/** Offset from the memeber variable to the byte of the property (0-7). */
	uint8 ByteOffset;
	/** Mask of the byte with the property value. */
	uint8 ByteMask;
	/** Mask of the field with the property value. Either equal to ByteMask or 255 in case of 'bool' type. */
	uint8 FieldMask;

	COREUOBJECT_API UBoolProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	/**
	 * Constructor.
	 *
	 * @param ECppProperty Unused.
	 * @param InOffset Offset of the property.
	 * @param InCategory Category of the property.
	 * @param InFlags Property flags.
	 * @param InBitMask Bitmask of the bitfield this property represents.
	 * @param InElementSize Sizeof of the boolean type this property represents.
	 * @param bIsNativeBool true if this property represents C++ bool type.
	 */
	COREUOBJECT_API UBoolProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, uint32 InBitMask, uint32 InElementSize, bool bIsNativeBool);

	/**
	 * Constructor.
	 *
	 * @param ObjectInitializer Properties.
	 * @param ECppProperty Unused.
	 * @param InOffset Offset of the property.
	 * @param InCategory Category of the property.
	 * @param InFlags Property flags.
	 * @param InBitMask Bitmask of the bitfield this property represents.
	 * @param InElementSize Sizeof of the boolean type this property represents.
	 * @param bIsNativeBool true if this property represents C++ bool type.
	 */
	COREUOBJECT_API UBoolProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, uint32 InBitMask, uint32 InElementSize, bool bIsNativeBool );

	// UObject interface.
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface

	/** 
	 * Sets the bitfield/bool type and size. 
	 * This function must be called before UBoolProperty can be used.
	 *
	 * @param InSize size of the bitfield/bool type.
	 * @param bIsNativeBool true if this property represents C++ bool type.
	 */
	COREUOBJECT_API void SetBoolSize( const uint32 InSize, const bool bIsNativeBool = false, const uint32 InBitMask = 0 );

	/**
	 * If the return value is true this UBoolProperty represents C++ bool type.
	 */
	UE_FORCEINLINE_HINT bool IsNativeBool() const
	{
		return FieldMask == 0xff;
	}
};

UCLASS(Abstract, Config = Engine)
class UObjectPropertyBase : public UProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<class UClass> PropertyClass;

	UObjectPropertyBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	UObjectPropertyBase(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass = NULL)
		: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
		, PropertyClass(InClass)
	{}

	UObjectPropertyBase( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass=NULL )
	:	UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	,	PropertyClass( InClass )
	{}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface

#if USE_UPROPERTY_LOAD_DEFERRING
	COREUOBJECT_API void SetPropertyClass(UClass* NewPropertyClass);
#else
	UE_FORCEINLINE_HINT void SetPropertyClass(UClass* NewPropertyClass) { PropertyClass = NewPropertyClass; }
#endif // USE_UPROPERTY_LOAD_DEFERRING
};

UCLASS(Config = Engine)
class UObjectProperty : public UObjectPropertyBase
{
	GENERATED_BODY()

public:
	UObjectProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	UObjectProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass)
		: UObjectPropertyBase(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InClass)
	{
	}

	UObjectProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass )
		: UObjectPropertyBase(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InClass)
	{
	}
};

UCLASS(Config = Engine)
class UWeakObjectProperty : public UObjectPropertyBase
{
	GENERATED_BODY()

public:
	UWeakObjectProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UWeakObjectProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass)
	: UObjectPropertyBase(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InClass)
	{
	}

	UWeakObjectProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass )
	: UObjectPropertyBase( ObjectInitializer, EC_CppProperty, InOffset, InFlags, InClass )
	{
	}
};

UCLASS(Config = Engine)
class ULazyObjectProperty : public UObjectPropertyBase
{
	GENERATED_BODY()

public:
	ULazyObjectProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	ULazyObjectProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass)
	: UObjectPropertyBase(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InClass)
	{
	}

	ULazyObjectProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass )
	: UObjectPropertyBase(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InClass)
	{
	}
};

UCLASS(Config = Engine)
class USoftObjectProperty : public UObjectPropertyBase
{
	GENERATED_BODY()

public:
	USoftObjectProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	USoftObjectProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass)
	: UObjectPropertyBase(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InClass)
	{}

	USoftObjectProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InClass )
	: UObjectPropertyBase(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InClass)
	{}
};

UCLASS(Config = Engine)
class UClassProperty : public UObjectProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<class UClass> MetaClass;

	UClassProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	UClassProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InMetaClass, UClass* InClassType)
		: UObjectProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InClassType ? InClassType : UClass::StaticClass())
		, MetaClass(InMetaClass)
	{
	}

	UClassProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InMetaClass, UClass* InClassType)
		: UObjectProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InClassType ? InClassType : UClass::StaticClass())
	,	MetaClass( InMetaClass )
	{
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface

	/**
	 * Setter function for this property's MetaClass member. Favor this function 
	 * whilst loading (since, to handle circular dependencies, we defer some 
	 * class loads and use a placeholder class instead). It properly handles 
	 * deferred loading placeholder classes (so they can properly be replaced 
	 * later).
	 * 
	 * @param  NewMetaClass    The MetaClass you want this property set with.
	 */
#if USE_UPROPERTY_LOAD_DEFERRING
	COREUOBJECT_API void SetMetaClass(UClass* NewMetaClass);
#else
	UE_FORCEINLINE_HINT void SetMetaClass(UClass* NewMetaClass) { MetaClass = NewMetaClass; }
#endif // USE_UPROPERTY_LOAD_DEFERRING
};

UCLASS(Config = Engine)
class USoftClassProperty : public USoftObjectProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<class UClass> MetaClass;

	USoftClassProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	USoftClassProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InMetaClass)
		: Super(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, UClass::StaticClass())
		, MetaClass(InMetaClass)
	{}

	USoftClassProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InMetaClass )
		:	Super(ObjectInitializer, EC_CppProperty, InOffset, InFlags, UClass::StaticClass() )
		,	MetaClass( InMetaClass )
	{}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface

	/**
	 * Setter function for this property's MetaClass member. Favor this function 
	 * whilst loading (since, to handle circular dependencies, we defer some 
	 * class loads and use a placeholder class instead). It properly handles 
	 * deferred loading placeholder classes (so they can properly be replaced 
	 * later).
	 * 
	 * @param  NewMetaClass    The MetaClass you want this property set with.
	 */
#if USE_UPROPERTY_LOAD_DEFERRING
	COREUOBJECT_API void SetMetaClass(UClass* NewMetaClass);
#else
	UE_FORCEINLINE_HINT void SetMetaClass(UClass* NewMetaClass) { MetaClass = NewMetaClass; }
#endif // USE_UPROPERTY_LOAD_DEFERRING
};

UCLASS(Config = Engine)
class UInterfaceProperty : public UProperty
{
	GENERATED_BODY()

public:

	/** The native interface class that this interface property refers to */
	UPROPERTY(SkipSerialization)
	TObjectPtr<class	UClass>		InterfaceClass;

	UInterfaceProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UInterfaceProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InInterfaceClass)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, (InFlags & ~CPF_InterfaceClearMask))
	, InterfaceClass(InInterfaceClass)
	{
	}

	UInterfaceProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UClass* InInterfaceClass )
	: UProperty( ObjectInitializer, EC_CppProperty, InOffset, (InFlags & ~CPF_InterfaceClearMask) )
	, InterfaceClass( InInterfaceClass )
	{
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface

	/**
	 * Setter function for this property's InterfaceClass member. Favor this 
	 * function whilst loading (since, to handle circular dependencies, we defer 
	 * some class loads and use a placeholder class instead). It properly 
	 * handles deferred loading placeholder classes (so they can properly be 
	 * replaced later).
	 *  
	 * @param  NewInterfaceClass    The InterfaceClass you want this property set with.
	 */
#if USE_UPROPERTY_LOAD_DEFERRING
	COREUOBJECT_API void SetInterfaceClass(UClass* NewInterfaceClass);
#else
	UE_FORCEINLINE_HINT void SetInterfaceClass(UClass* NewInterfaceClass) { InterfaceClass = NewInterfaceClass; }
#endif // USE_UPROPERTY_LOAD_DEFERRING
};

UCLASS(Config = Engine)
class UNameProperty : public UProperty
{
	GENERATED_BODY()
public:

	UNameProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UNameProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UNameProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags )
	{
	}
};

UCLASS(Config = Engine)
class UStrProperty : public UProperty
{
	GENERATED_BODY()
public:

	UStrProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UStrProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UStrProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};

UCLASS(Config = Engine)
class UArrayProperty : public UProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<UProperty> Inner;

	UArrayProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UArrayProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UArrayProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
	: UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class UMapProperty : public UProperty
{
	GENERATED_BODY()

public:

	// Properties representing the key type and value type of the contained pairs
	UPROPERTY(SkipSerialization)
	TObjectPtr<UProperty>       KeyProp;
	UPROPERTY(SkipSerialization)
	TObjectPtr<UProperty>       ValueProp;
	FScriptMapLayout MapLayout;

	UMapProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API UMapProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags);

	// UObject interface
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class USetProperty : public UProperty
{
	GENERATED_BODY()

public:

	// Properties representing the key type and value type of the contained pairs
	UPROPERTY(SkipSerialization)
	TObjectPtr<UProperty>       ElementProp;
	FScriptSetLayout SetLayout;

	USetProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API USetProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags);

	// UObject interface
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class UStructProperty : public UProperty
{
	GENERATED_BODY()

public:

	// Variables.
	UPROPERTY(SkipSerialization)
	TObjectPtr<class UScriptStruct> Struct;

	UStructProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API UStructProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UScriptStruct* InStruct);
	COREUOBJECT_API UStructProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UScriptStruct* InStruct );

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class UDelegateProperty : public UProperty
{
	GENERATED_BODY()

public:

	/** Points to the source delegate function (the function declared with the delegate keyword) used in the declaration of this delegate property. */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UFunction> SignatureFunction;

	UDelegateProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UDelegateProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = NULL)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	, SignatureFunction(InSignatureFunction)
	{
	}

	UDelegateProperty( const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = NULL )
	: UProperty( ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	, SignatureFunction(InSignatureFunction)
	{
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface
};

UCLASS(Abstract, Config = Engine)
class UMulticastDelegateProperty : public UProperty
{
	GENERATED_BODY()

public:

	/** Points to the source delegate function (the function declared with the delegate keyword) used in the declaration of this delegate property. */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UFunction> SignatureFunction;

	UMulticastDelegateProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	UMulticastDelegateProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = nullptr)
		: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags)
		, SignatureFunction(InSignatureFunction)
	{
	}

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface
};

UCLASS(Config = Engine)
class UMulticastInlineDelegateProperty : public UMulticastDelegateProperty
{
	GENERATED_BODY()

public:
	UMulticastInlineDelegateProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UMulticastInlineDelegateProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = nullptr)
	: UMulticastDelegateProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InSignatureFunction)
	{
	}

	UMulticastInlineDelegateProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = nullptr)
	: UMulticastDelegateProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InSignatureFunction)
	{
	}
};

UCLASS(Config = Engine)
class UMulticastSparseDelegateProperty : public UMulticastDelegateProperty
{
	GENERATED_BODY()

public:

	UMulticastSparseDelegateProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: Super(ObjectInitializer)
	{
	}

	UMulticastSparseDelegateProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = nullptr)
	: UMulticastDelegateProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags, InSignatureFunction)
	{
	}

	UMulticastSparseDelegateProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UFunction* InSignatureFunction = nullptr)
	: UMulticastDelegateProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags, InSignatureFunction)
	{
	}
};


UCLASS(Config = Engine)
class UEnumProperty : public UProperty
{
	GENERATED_BODY()

public:
	UEnumProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get()) 
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API UEnumProperty(const FObjectInitializer& ObjectInitializer, UEnum* InEnum);
	COREUOBJECT_API UEnumProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UEnum* InEnum);

	// UObject interface
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	UPROPERTY(SkipSerialization)
	TObjectPtr<UNumericProperty> UnderlyingProp; // The property which represents the underlying type of the enum
	UPROPERTY(SkipSerialization)
	TObjectPtr<UEnum> Enum; // The enum represented by this property
};

UCLASS(Config = Engine)
class UTextProperty : public UProperty
{
	GENERATED_BODY()

public:

	UTextProperty(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
	: UProperty(ObjectInitializer)
	{
	}

	UTextProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags)
	{
	}

	UTextProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags)
	{
	}
};
