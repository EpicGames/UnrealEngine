// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WeakFieldPtr.h: Weak pointer to FField
=============================================================================*/

#pragma once

#include "Templates/Requires.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/FieldPath.h"

#include <type_traits>

template<class T>
struct TWeakFieldPtr;

template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
>
bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs);

template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr == (const RhsType*)nullptr))
>
bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs);

template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((const LhsType*)nullptr == (RhsType*)nullptr))
>
bool operator==(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs);

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr != (RhsType*)nullptr))
>
bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs);

template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((LhsType*)nullptr != (const RhsType*)nullptr))
>
bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs);

template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(UE_REQUIRES_EXPR((const LhsType*)nullptr != (RhsType*)nullptr))
>
bool operator!=(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs);
#endif

template<class T>
struct TWeakFieldPtr
{
	template <typename>
	friend struct TWeakFieldPtr;

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
	>
	friend bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs);

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr == (const RhsType*)nullptr))
	>
	friend bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs);

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((const LhsType*)nullptr == (RhsType*)nullptr))
	>
	friend bool operator==(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs);

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr != (RhsType*)nullptr))
	>
	friend bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs);

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((LhsType*)nullptr != (const RhsType*)nullptr))
	>
	friend bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs);

	template <
		typename LhsType,
		typename RhsType
		UE_REQUIRES_FRIEND(UE_REQUIRES_EXPR((const LhsType*)nullptr != (RhsType*)nullptr))
	>
	friend bool operator!=(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs);
#endif

private:

	// These exists only to disambiguate the two constructors below
	enum EDummy1 { Dummy1 };

	TWeakObjectPtr<UObject> Owner;
	mutable TFieldPath<T> Field;

public:
	using ElementType = T;
	
	TWeakFieldPtr() = default;
	TWeakFieldPtr(const TWeakFieldPtr&) = default;
	TWeakFieldPtr& operator=(const TWeakFieldPtr&) = default;
	~TWeakFieldPtr() = default;

	/**
	* Construct from a null pointer
	**/
	inline TWeakFieldPtr(TYPE_OF_NULLPTR)
		: Owner((UObject*)nullptr)
		, Field()
	{
	}

	/**
	* Construct from an object pointer
	* @param Object object to create a weak pointer to
	**/
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TWeakFieldPtr(U* InField, EDummy1 = Dummy1)
		: Owner(InField ? InField->GetOwnerUObject() : (UObject*)nullptr)
		, Field(InField)
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TWeakFieldPtr<UUndefinedClass>.
		static_assert(std::is_convertible_v<T*, const volatile FField*>, "TWeakFieldPtr can only be constructed with FField types");
	}

	/**
	* Construct from another weak pointer of another type, intended for derived-to-base conversions
	* @param Other weak pointer to copy from
	**/
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	inline TWeakFieldPtr(const TWeakFieldPtr<OtherT>& Other)
		: Owner(Other.Owner)
		, Field(Other.Field)
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TWeakFieldPtr<UUndefinedClass>.
		static_assert(std::is_convertible_v<T*, const volatile FField*>, "TWeakFieldPtr can only be constructed with FField types");
	}

	/**
	* Reset the weak pointer back to the NULL state
	*/
	inline void Reset()
	{
		Owner.Reset();
		Field.Reset();
	}

	/**
	* Copy from an object pointer
	* @param Object object to create a weak pointer to
	**/
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline void operator=(U* InField)
	{
		Owner = InField ? InField->GetOwnerUObject() : (UObject*)nullptr;
		Field = (U*)InField;
	}

	/**
	* Assign from another weak pointer, intended for derived-to-base conversions
	* @param Other weak pointer to copy from
	**/
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	inline void operator=(const TWeakFieldPtr<OtherT>& Other)
	{
		Owner = Other.Owner;
		Field = Other.Field;
	}

	/**
	* Dereference the weak pointer
	* @param bEvenIfPendingKill, if this is true, pendingkill objects are considered valid
	* @return NULL if this object is gone or the weak pointer was NULL, otherwise a valid uobject pointer
	**/
	inline T* Get(bool bEvenIfPendingKill) const
	{
		if (Owner.Get(bEvenIfPendingKill))
		{
			return Field.Get();
		}
		else
		{
			// Clear potentially stale pointer to the actual field
			Field.ClearCachedField();
		}
		return nullptr;
	}

	/**
	* Dereference the weak pointer. This is an optimized version implying bEvenIfPendingKill=false.
	*/
	inline T* Get(/*bool bEvenIfPendingKill = false*/) const
	{
		if (Owner.Get())
		{
			return Field.Get();
		}
		else
		{
			// Clear potentially stale pointer to the actual field
			Field.ClearCachedField();
		}
		return nullptr;
	}

	/** Deferences the weak pointer even if its marked RF_Unreachable. This is needed to resolve weak pointers during GC (such as ::AddReferenceObjects) */
	inline T* GetEvenIfUnreachable() const
	{
		if (Owner.GetEvenIfUnreachable())
		{
			return Field.Get();
		}
		else
		{
			// Clear potentially stale pointer to the actual field
			Field.ClearCachedField();
		}
		return nullptr;
	}

	/**
	* Dereference the weak pointer
	**/
	UE_FORCEINLINE_HINT T & operator*() const
	{
		return *Get();
	}

	/**
	* Dereference the weak pointer
	**/
	UE_FORCEINLINE_HINT T * operator->() const
	{
		return Get();
	}

	/**
	* Test if this points to a live FField
	* @param bEvenIfPendingKill, if this is true, pendingkill objects are considered valid
	* @param bThreadsafeTest, if true then function will just give you information whether referenced
	*							FField is gone forever (@return false) or if it is still there (@return true, no object flags checked).
	* @return true if Get() would return a valid non-null pointer
	**/
	UE_FORCEINLINE_HINT bool IsValid(bool bEvenIfPendingKill, bool bThreadsafeTest = false) const
	{
		return Owner.IsValid(bEvenIfPendingKill, bThreadsafeTest) && Field.Get();
	}

	/**
	* Test if this points to a live FField. This is an optimized version implying bEvenIfPendingKill=false, bThreadsafeTest=false.
	* @return true if Get() would return a valid non-null pointer
	*/
	UE_FORCEINLINE_HINT bool IsValid(/*bool bEvenIfPendingKill = false, bool bThreadsafeTest = false*/) const
	{
		return Owner.IsValid() && Field.Get();
	}

	/**
	* Slightly different than !IsValid(), returns true if this used to point to a FField, but doesn't any more and has not been assigned or reset in the mean time.
	* @param bIncludingIfPendingKill, if this is true, pendingkill objects are considered stale
	* @param bThreadsafeTest, set it to true when testing outside of Game Thread. Results in false if WeakObjPtr point to an existing object (no flags checked)
	* @return true if this used to point at a real object but no longer does.
	**/
	UE_FORCEINLINE_HINT bool IsStale(bool bIncludingIfPendingKill = false, bool bThreadsafeTest = false) const
	{
		return Owner.IsStale(bIncludingIfPendingKill, bThreadsafeTest);
	}

	UE_FORCEINLINE_HINT bool HasSameIndexAndSerialNumber(const TWeakFieldPtr& Other) const
	{
		return Owner.HasSameIndexAndSerialNumber(Other.Owner);
	}

	/** Hash function. */
	[[nodiscard]] UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const TWeakFieldPtr& WeakObjectPtr)
	{
		return GetTypeHash(WeakObjectPtr.Field);
	}

	inline void Serialize(FArchive& Ar)
	{
		Ar << Owner;
		Ar << Field;
	}
};

/**
* Compare weak pointers for equality
* @param Lhs weak pointer to compare
* @param Rhs weak pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr == (RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs)
{
	return Lhs.Field == Rhs.Field;
}

/**
* Compare weak pointers for equality
* @param Lhs weak pointer to compare
* @param Rhs pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr == (const RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator==(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs)
{
	return Lhs.Field == Rhs;
}

/**
* Compare weak pointers for equality
* @param Lhs pointer to compare
* @param Rhs weak pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((const LhsType*)nullptr == (RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator==(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs)
{
	return Lhs == Rhs.Field;
}

/**
* Test weak pointer for null
* @param Lhs weak pointer to test
**/
template <typename LhsType>
UE_FORCEINLINE_HINT bool operator==(const TWeakFieldPtr<LhsType>& Lhs, TYPE_OF_NULLPTR)
{
	return !Lhs.Get();
}

/**
* Test weak pointer for null
* @param Rhs weak pointer to test
**/
template <typename RhsType>
UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR, const TWeakFieldPtr<RhsType>& Rhs)
{
	return !Rhs.Get();
}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
/**
* Compare weak pointers for inequality
* @param Lhs weak pointer to compare
* @param Rhs weak pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr != (RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const TWeakFieldPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}

/**
* Compare weak pointers for inequality
* @param Lhs weak pointer to compare
* @param Rhs pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((LhsType*)nullptr != (const RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, const RhsType* Rhs)
{
	return !(Lhs == Rhs);
}

/**
* Compare weak pointers for inequality
* @param Lhs pointer to compare
* @param Rhs weak pointer to compare
**/
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES_DEFINITION(UE_REQUIRES_EXPR((const LhsType*)nullptr != (RhsType*)nullptr))
>
UE_FORCEINLINE_HINT bool operator!=(const LhsType* Lhs, const TWeakFieldPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}

/**
* Test weak pointer for non-null
* @param Lhs weak pointer to test
**/
template <typename LhsType>
UE_FORCEINLINE_HINT bool operator!=(const TWeakFieldPtr<LhsType>& Lhs, TYPE_OF_NULLPTR)
{
	return !(Lhs == nullptr);
}

/**
* Test weak pointer for non-null
* @param Rhs weak pointer to test
**/
template <typename RhsType>
UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR, const TWeakFieldPtr<RhsType>& Rhs)
{
	return !(nullptr == Rhs);
}
#endif

// Helper function which deduces the type of the initializer
template <typename T>
UE_FORCEINLINE_HINT TWeakFieldPtr<T> MakeWeakFieldPtr(T* Ptr)
{
	return TWeakFieldPtr<T>(Ptr);
}

template<class T> struct TIsPODType<TWeakFieldPtr<T> > { enum { Value = true }; };
template<class T> struct TIsZeroConstructType<TWeakFieldPtr<T> > { enum { Value = true }; };
template<class T> struct TIsWeakPointerType<TWeakFieldPtr<T> > { enum { Value = true }; };


/**
* MapKeyFuncs for TWeakFieldPtrs which allow the key to become stale without invalidating the map.
*/
template <typename KeyType, typename ValueType, bool bInAllowDuplicateKeys = false>
struct TWeakFieldPtrMapKeyFuncs : public TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>
{
	typedef typename TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>::KeyInitType KeyInitType;

	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

template<class T>
FArchive& operator<<(FArchive& Ar, TWeakFieldPtr<T>& WeakFieldPtr)
{
	WeakFieldPtr.Serialize(Ar);
	return Ar;
}
