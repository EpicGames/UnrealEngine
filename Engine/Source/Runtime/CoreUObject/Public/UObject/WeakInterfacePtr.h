// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"
#include "ScriptInterface.h"

#include <type_traits>

/**
 * An alternative to TWeakObjectPtr that makes it easier to work through an interface.
 */
template<class T>
struct TWeakInterfacePtr
{
	using ElementType = T;
	using UObjectType = TCopyQualifiersFromTo_T<T, UObject>;

	[[nodiscard]] UE_FORCEINLINE_HINT TWeakInterfacePtr() = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TWeakInterfacePtr(const TWeakInterfacePtr& Other) = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TWeakInterfacePtr(TWeakInterfacePtr&& Other) = default;
	UE_FORCEINLINE_HINT ~TWeakInterfacePtr() = default;
	UE_FORCEINLINE_HINT TWeakInterfacePtr& operator=(const TWeakInterfacePtr& Other) = default;
	UE_FORCEINLINE_HINT TWeakInterfacePtr& operator=(TWeakInterfacePtr&& Other) = default;

	/**
	* Construct from a null pointer
	*/
	[[nodiscard]] UE_FORCEINLINE_HINT TWeakInterfacePtr(TYPE_OF_NULLPTR) :
		ObjectInstance(nullptr)
	{
	}

	/**
	 * Construct from an object pointer
	 * @param Object The object to create a weak pointer to. This object must implement interface T.
	 */
	template<
		typename U
		UE_REQUIRES(std::is_convertible_v<U, TCopyQualifiersFromTo_T<U, UObject>*>)
	>
	[[nodiscard]] TWeakInterfacePtr(U&& Object)
	{
		InterfaceInstance = Cast<T>(ImplicitConv<TCopyQualifiersFromTo_T<U, UObject>*>(Object));
		if (InterfaceInstance != nullptr)
		{
			ObjectInstance = Object;
		}
	}

	/**
	 * Construct from an interface pointer
	 * @param Interface The interface pointer to create a weak pointer to. There must be a UObject behind the interface.
	 */
	[[nodiscard]] TWeakInterfacePtr(T* Interface)
	{
		ObjectInstance = Cast<UObject>(Interface);
		if (ObjectInstance != nullptr)
		{
			InterfaceInstance = Interface;
		}
	}

	/**
	 * Construct from a TScriptInterface of the same interface type
	 * @param ScriptInterface 	The TScriptInterface to copy from.
	 * 							No validation is done here; passing an invalid TScriptInterface in will result in an invalid TWeakInterfacePtr.
	 */
	[[nodiscard]] TWeakInterfacePtr(const TScriptInterface<T>& ScriptInterface)
	{
		ObjectInstance = ScriptInterface.GetObject();
		InterfaceInstance = ScriptInterface.GetInterface();
	}

	/**
	 * Reset the weak pointer back to the null state.
	 */
	inline void Reset()
	{
		InterfaceInstance = nullptr;
		ObjectInstance.Reset();
	}

	/**
	 * Test if this points to a live object. Parameters are passed to the underlying TWeakObjectPtr.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid(bool bEvenIfPendingKill, bool bThreadsafeTest = false) const
	{
		return InterfaceInstance != nullptr && ObjectInstance.IsValid(bEvenIfPendingKill, bThreadsafeTest);
	}

	/**
	 * Test if this points to a live object. Calls the underlying TWeakObjectPtr's parameterless IsValid method.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		return InterfaceInstance != nullptr && ObjectInstance.IsValid();
	}

	/**
	 * Test if this pointer is stale. Parameters are passed to the underlying TWeakObjectPtr.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsStale(bool bEvenIfPendingKill = false, bool bThreadsafeTest = false) const
	{
		return InterfaceInstance != nullptr && ObjectInstance.IsStale(bEvenIfPendingKill, bThreadsafeTest);
	}

	/**
	 * Dereference the weak pointer into an interface pointer.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT T* Get() const
	{
		return IsValid() ? InterfaceInstance : nullptr;
	}

	/**
	 * Dereference the weak pointer into a UObject pointer.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT UObjectType* GetObject() const
	{
		return ObjectInstance.Get();
	}

	/**
	 * Dereference the weak pointer.
	 */
	[[nodiscard]] inline T& operator*() const
	{
		check(IsValid());
		return *InterfaceInstance;
	}

	/**
	 * Dereference the weak pointer.
	 */
	[[nodiscard]] inline T* operator->() const
	{
		check(IsValid());
		return InterfaceInstance;
	}

	/**
	 * Assign from an interface pointer.
	 */
	inline TWeakInterfacePtr<T>& operator=(T* Other)
	{
		*this = TWeakInterfacePtr<T>(Other);
		return *this;
	}

	/**
	 * Assign from a script interface.
	 */
	inline TWeakInterfacePtr<T>& operator=(const TScriptInterface<T>& Other)
	{
		ObjectInstance = Other.GetObject();
		InterfaceInstance = (T*)Other.GetInterface();
		return *this;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TWeakInterfacePtr<T>& Other) const
	{
		return InterfaceInstance == Other.InterfaceInstance;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TWeakInterfacePtr<T>& Other) const
	{
		return InterfaceInstance != Other.InterfaceInstance;
	}

	[[nodiscard]] FORCENOINLINE bool operator==(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) const
	{
		return !(*this == nullptr);
	}

	[[nodiscard]] inline TScriptInterface<T> ToScriptInterface() const
	{
		UObjectType* Object = ObjectInstance.Get();
		if (Object)
		{
			return TScriptInterface<T>(Object);
		}

		return TScriptInterface<T>();
	}

	[[nodiscard]] UE_FORCEINLINE_HINT TWeakObjectPtr<UObjectType> GetWeakObjectPtr() const
	{
		return ObjectInstance;
	}

private:
	TWeakObjectPtr<UObjectType> ObjectInstance;
	T* InterfaceInstance = nullptr;
};
