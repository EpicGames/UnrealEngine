// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WeakObjectPtr.h: Weak pointer to UObject
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ScriptDelegates.h"
#include "UObject/UObjectArray.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "UObject/ObjectHandleDefines.h"
#include "UObject/RemoteObjectTypes.h"
#include <type_traits>

class FArchive;
class UObject;
template <class, class>
struct TWeakObjectPtr;

/** Invalid FWeakObjectPtr ObjectIndex values must be 0 to support zeroed initialization (this used to be INDEX_NONE, leading to subtle bugs). */
#ifndef UE_WEAKOBJECTPTR_ZEROINIT_FIX
	#define UE_WEAKOBJECTPTR_ZEROINIT_FIX 1
#endif

namespace UE::Core::Private
{
	/** Specifies the ObjectIndex used for invalid object pointers. */
#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
	inline constexpr int32 InvalidWeakObjectIndex = 0;
#else
	inline constexpr int32 InvalidWeakObjectIndex = INDEX_NONE;
#endif
}

/**
 * FWeakObjectPtr is a weak pointer to a UObject. 
 * It can return nullptr later if the object is garbage collected.
 * It has no impact on if the object is garbage collected or not.
 * It can't be directly used across a network.
 *
 * Most often it is used when you explicitly do NOT want to prevent something from being garbage collected.
 */
struct FWeakObjectPtr
{
public:

	template <class, class>
	friend struct TWeakObjectPtr;

#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
	FWeakObjectPtr() = default;

	UE_FORCEINLINE_HINT FWeakObjectPtr(TYPE_OF_NULLPTR)
		: FWeakObjectPtr()
	{
	}
#else
	/** Null constructor **/
	UE_FORCEINLINE_HINT FWeakObjectPtr()
	{
		Reset();
	}

	/**
	 * Construct from nullptr or something that can be implicitly converted to nullptr (eg: NULL)
	 * @param Object object to create a weak pointer to
	 */
	UE_FORCEINLINE_HINT FWeakObjectPtr(TYPE_OF_NULLPTR)
	{
		(*this) = nullptr;
	}
#endif

	/**  
	 * Construct from an object pointer or something that can be implicitly converted to an object pointer
	 * @param Object object to create a weak pointer to
	 */
	UE_FORCEINLINE_HINT FWeakObjectPtr(FObjectPtr Object)
	{
		*this = Object;
	}
	UE_FORCEINLINE_HINT FWeakObjectPtr(const UObject* Object)
		: FWeakObjectPtr(FObjectPtr(const_cast<UObject*>(Object)))
	{
	}
	template <typename T>
	UE_FORCEINLINE_HINT FWeakObjectPtr(TObjectPtr<T> Object)
		: FWeakObjectPtr(FObjectPtr(const_cast<UObject*>(ImplicitConv<const UObject*>(Object.Get()))))
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in situations that TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions') doesn't, meaning it won't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion
	}

	/**  
	 * Construct from another weak pointer
	 * @param Other weak pointer to copy from
	 */
	FWeakObjectPtr(const FWeakObjectPtr& Other) = default;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	explicit FWeakObjectPtr(FRemoteObjectId RemoteId)
	{
		// When using remote object handles both ObjectIndex and SerialNumber are not used when comparing weak object pointers
		// ObjectIndex is used for performance only and will be updated when this weak pointer is accessed
		// SerialNumber will also get updated when this weak pointer is resolved
		ObjectIndex = 0;		
		ObjectSerialNumber = 0;
		ObjectRemoteId = RemoteId;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	/**
	 * Reset the weak pointer back to the null state
	 */
	inline void Reset()
	{
		using namespace UE::Core::Private;

		ObjectIndex = InvalidWeakObjectIndex;
		ObjectSerialNumber = 0;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		ObjectRemoteId = FRemoteObjectId();
#endif
	}

	/**  
	 * Copy from an object pointer
	 * @param Object object to create a weak pointer to
	 */
	COREUOBJECT_API void operator=(FObjectPtr Object);
	UE_FORCEINLINE_HINT void operator=(const UObject* Object)
	{
		*this = FObjectPtr(const_cast<UObject*>(Object));
	}
	template <typename T>
	UE_FORCEINLINE_HINT void operator=(TObjectPtr<T> Object)
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in situations that TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions') doesn't, meaning it won't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion

		*this = FObjectPtr(Object);
	}

	/**  
	 * Construct from another weak pointer
	 * @param Other weak pointer to copy from
	 */
	FWeakObjectPtr& operator=(const FWeakObjectPtr& Other) = default;

	/**  
	 * Compare weak pointers for equality.
	 * If both pointers would return nullptr from Get() they count as equal even if they were not initialized to the same object.
	 * @param Other weak pointer to compare to
	 */
	inline bool operator==(const FWeakObjectPtr& Other) const
	{
		return 
#if UE_WITH_REMOTE_OBJECT_HANDLE
			ObjectRemoteId == Other.ObjectRemoteId ||
#else
			(ObjectIndex == Other.ObjectIndex && ObjectSerialNumber == Other.ObjectSerialNumber) ||
#endif
			(!IsValid() && !Other.IsValid());
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	/**  
	 * Compare weak pointers for inequality
	 * @param Other weak pointer to compare to
	 */
	inline bool operator!=(const FWeakObjectPtr& Other) const
	{
		return 
#if UE_WITH_REMOTE_OBJECT_HANDLE
			ObjectRemoteId != Other.ObjectRemoteId &&
#else
			(ObjectIndex != Other.ObjectIndex || ObjectSerialNumber != Other.ObjectSerialNumber) &&
#endif
			(IsValid() || Other.IsValid());
	}
#endif

	/**
	 * Returns true if two weak pointers were originally set to the same object, even if they are now stale
	 * @param Other weak pointer to compare to
	 */
	inline bool HasSameIndexAndSerialNumber(const FWeakObjectPtr& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return ObjectRemoteId == Other.ObjectRemoteId;
#else
		return ObjectIndex == Other.ObjectIndex && ObjectSerialNumber == Other.ObjectSerialNumber;
#endif
	}

	/**  
	 * Dereference the weak pointer.
	 * @param bEvenIfGarbage if this is true, Garbage objects are considered valid
	 * @return nullptr if this object is gone or the weak pointer is explicitly null, otherwise a valid uobject pointer
	 */
	COREUOBJECT_API class UObject* Get(bool bEvenIfGarbage) const;

	/**  
	 * Dereference the weak pointer. This is an optimized version implying bEvenIfGarbage=false.
	 * @return nullptr if this object is gone or the weak pointer is explicitly null, otherwise a valid uobject pointer
	 */
	COREUOBJECT_API class UObject* Get(/*bool bEvenIfGarbage = false*/) const;

	/** Operators for direct access to Get() */
	UE_REWRITE UObject& operator*() const
	{
		return *Get();
	}
	UE_REWRITE UObject* operator->() const
	{
		return Get();
	}

	/** Dereference the weak pointer even if it is marked as Garbage or Unreachable */
	COREUOBJECT_API class UObject* GetEvenIfUnreachable() const;

	/**
	 * Get a strong object ptr to the weak pointer.
	 * @param bEvenIfGarbage if this is true, Garbage objects are considered valid
	 * @return TStrongObjectPtr will be invalid if this object is gone or the weak pointer is explicitly null, otherwise a valid TStrongObjectPtr
	 */
	COREUOBJECT_API class TStrongObjectPtr<UObject> Pin(bool bEvenIfGarbage) const;

	/**
	 * Get a strong object ptr to the weak pointer. This is an optimized version implying bEvenIfGarbage=false.
	 * @return TStrongObjectPtr will be invalid if this object is gone or the weak pointer is explicitly null, otherwise a valid TStrongObjectPtr
	 */
	COREUOBJECT_API class TStrongObjectPtr<UObject> Pin(/*bool bEvenIfGarbage = false*/) const;

	/*
	 * Get a strong object ptr even if it is marked as Garbage or Unreachable 
	 * @return TStrongObjectPtr will be invalid if this object is gone or the weak pointer is explicitly null, otherwise a valid TStrongObjectPtr
	 */
	COREUOBJECT_API class TStrongObjectPtr<UObject> PinEvenIfUnreachable() const;


	/**
	 * Attempt to get a strong object ptr to the weak pointer, but only if garbage collection is not in progress.
	 * @param bEvenIfGarbage if this is true, Garbage objects are considered valid
	 * @param bOutPinValid will be true if the pin attempt was successful, and false if it was blocked by garbage collection
	 * @return TStrongObjectPtr will be invalid if this object is gone or the weak pointer is explicitly null, otherwise a valid TStrongObjectPtr
	 */
	COREUOBJECT_API class TStrongObjectPtr<UObject> TryPin(bool& bOutPinValid, bool bEvenIfGarbage) const;

	/**
	 * Attempt to get a strong object ptr to the weak pointer, but only if garbage collection is not in progress. This is an optimized version implying bEvenIfGarbage=false.
	 * @param bOutPinValid will be true if the pin attempt was successful, and false if it was blocked by garbage collection
	 * @return TStrongObjectPtr will be invalid if this object is gone or the weak pointer is explicitly null, otherwise a valid TStrongObjectPtr
	 */
	COREUOBJECT_API class TStrongObjectPtr<UObject> TryPin(bool& bOutPinValid /*, bool bEvenIfGarbage = false*/) const;

	/*
	 * Attempt to get a strong object ptr even if it is marked as Garbage or Unreachable, but only if garbage collection is not in progress
	 * @param OutResult will be invalid if this object is gone, or the weak pointer is explicitly null, or garbage collection is in progress. Otherwise a valid TStrongObjectPtr
	 * @return  true if garbage collection was not in progress, and OutResult was successfully captured, false if garbage collection was in progress and OutResult was not captured 
	 */
	COREUOBJECT_API bool TryPinEvenIfUnreachable(class TStrongObjectPtr<UObject>& OutResult) const;

	// This is explicitly not added to avoid resolving weak pointers too often - use Get() once in a function.
	explicit operator bool() const = delete;

	/**  
	 * Test if this points to a live UObject
	 * This should be done only when needed as excess resolution of the underlying pointer can cause performance issues.
	 *
	 * @param bEvenIfGarbage if this is true, Garbage objects are considered valid. When false, an object is considered invalid as soon as it is
	 *                       marked for destruction.
	 * @param bThreadsafeTest if true then function will just give you information whether referenced
	 *							UObject is gone forever (return false) or if it is still there (return true, no object flags checked).
	 *							This is required as without it IsValid can return false during the mark phase of the GC
	 *							due to the presence of the Unreachable flag.
	 * @return true if Get() would return a valid non-null pointer
	 */
	COREUOBJECT_API bool IsValid(bool bEvenIfGarbage, bool bThreadsafeTest = false) const;

	/**
	 * Test if this points to a live UObject. This is an optimized version implying bEvenIfGarbage=false, bThreadsafeTest=false.
	 * This should be done only when needed as excess resolution of the underlying pointer can cause performance issues.
	 * Note that IsValid can not be used on another thread as it will incorrectly return false during the mark phase of the GC
	 * due to the Unreachable flag being set. (see bThreadsafeTest above)

	 * @return true if Get() would return a valid non-null pointer.
	 */
	COREUOBJECT_API bool IsValid(/*bool bEvenIfGarbage = false, bool bThreadsafeTest = false*/) const;

	/**  
	 * Slightly different than !IsValid(), returns true if this used to point to a UObject, but doesn't any more and has not been assigned or reset in the mean time.
	 * @param bIncludingGarbage if this is false, Garbage objects are NOT considered stale
	 * @param bThreadsafeTest set it to true when testing outside of Game Thread. Results in false if WeakObjPtr point to an existing object (no flags checked)
	 * @return true if this used to point at a real object but no longer does.
	 */
	COREUOBJECT_API bool IsStale(bool bIncludingGarbage = true, bool bThreadsafeTest = false) const;

	/**
	 * Returns true if this pointer was explicitly assigned to null, was reset, or was never initialized.
	 * If this returns true, IsValid() and IsStale() will both return false.
	 */
	inline bool IsExplicitlyNull() const
	{
		using namespace UE::Core::Private;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return !ObjectRemoteId.IsValid();
#else
#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
		return ObjectIndex == InvalidWeakObjectIndex && ObjectSerialNumber == 0;
#else
		return ObjectIndex == InvalidWeakObjectIndex;
#endif
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/**
	* Returns remote Id of the object this pointer is referencing
	*/
	UE_FORCEINLINE_HINT FRemoteObjectId GetRemoteId() const
	{
		return ObjectRemoteId;
	}

	/**
	* Returns true if this weak pointer resolves to the specified object
	*/
	COREUOBJECT_API bool HasSameObject(const UObject* Other) const;

	/**
	* Returns true if the object this weak pointer represents is owned by another server
	*/
	UE_FORCEINLINE_HINT EResidence GetResidence() const
	{
		return UE::RemoteObject::Handle::GetResidence(ObjectRemoteId);
	}

	/**
	* Returns true if the object this weak pointer represents is owned by another server
	*/
	UE_DEPRECATED(5.8, "Use GetResidence() instead.")
	UE_FORCEINLINE_HINT bool IsRemote() const
	{
		return GetResidence() == EResidence::Remote;
	}
#else
	UE_FORCEINLINE_HINT EResidence GetResidence() const
	{
		return EResidence::Local;
	}

	UE_DEPRECATED(5.8, "Use GetResidence() instead.")
	UE_FORCEINLINE_HINT bool IsRemote() const
	{
		return false;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	/** Hash function. */
	inline uint32 GetTypeHash() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return ::GetTypeHash(ObjectRemoteId);
#else
		return uint32(ObjectIndex ^ ObjectSerialNumber);
#endif
	}

	/**
	 * Weak object pointer serialization.  Weak object pointers only have weak references to objects and
	 * won't serialize the object when gathering references for garbage collection.  So in many cases, you
	 * don't need to bother serializing weak object pointers.  However, serialization is required if you
	 * want to load and save your object.
	 */
	COREUOBJECT_API void Serialize(FArchive& Ar);

private:
	inline int32 GetObjectIndex_Private() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		using namespace UE::Core::Private;
		if ((ObjectIndex == InvalidWeakObjectIndex) && ObjectRemoteId.IsValid())
		{
			TryResolveRemoteObject();
		}
#endif
		return ObjectIndex;
	}

private:
	friend struct FObjectKey;
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	COREUOBJECT_API FUObjectItem* TryResolveRemoteObject() const;
	COREUOBJECT_API bool CanBeResolved(UE::RemoteObject::Handle::ECanResolveObjectBehavior CanResolveObjectBehavior) const;
#endif

	/**  
	 * internal function to test for serial number matches
	 * @return true if the serial number in this matches the central table
	 */
	inline bool SerialNumbersMatch() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		checkSlow(ObjectIndex >= 0); // otherwise this is a corrupted weak pointer
		FRemoteObjectId ActualId = GUObjectArray.GetRemoteId(ObjectIndex);
		return ActualId == ObjectRemoteId;
#else
		checkSlow(ObjectSerialNumber > FUObjectArray::START_SERIAL_NUMBER && ObjectIndex >= 0); // otherwise this is a corrupted weak pointer
		int32 ActualSerialNumber = GUObjectArray.GetSerialNumber(ObjectIndex);
		checkSlow(!ActualSerialNumber || ActualSerialNumber >= ObjectSerialNumber); // serial numbers should never shrink
		return ActualSerialNumber == ObjectSerialNumber;
#endif
	}

	inline bool SerialNumbersMatch(FUObjectItem* ObjectItem) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		checkSlow(ObjectIndex >= 0); // otherwise this is a corrupted weak pointer
		FRemoteObjectId ActualId = ObjectItem->GetRemoteId();
		return ActualId == ObjectRemoteId;
#else
		checkSlow(ObjectSerialNumber > FUObjectArray::START_SERIAL_NUMBER && ObjectIndex >= 0); // otherwise this is a corrupted weak pointer
		const int32 ActualSerialNumber = ObjectItem->GetSerialNumber();
		checkSlow(!ActualSerialNumber || ActualSerialNumber >= ObjectSerialNumber); // serial numbers should never shrink
		return ActualSerialNumber == ObjectSerialNumber;
#endif
	}

	inline FUObjectItem* Internal_GetObjectItem() const
	{
		using namespace UE::Core::Private;

#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (IsExplicitlyNull())
		{
			return nullptr;
		}
		if (ObjectIndex < 0)
		{
			return nullptr;
		}

		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
		bool bResolved = false;
		// if an existing object is marked as Remote it needs to be resolved before its ObjectItem is returned
		if (!ObjectItem || ObjectItem->HasAnyFlags(EInternalObjectFlags::Remote))
		{
			ObjectItem = TryResolveRemoteObject();
			if (!ObjectItem)
			{
				return nullptr;
			}
			bResolved = true;
		}
		// TryResolveRemoteObject already validated the serial number
		if (!bResolved && !SerialNumbersMatch(ObjectItem))
		{
			ObjectItem = TryResolveRemoteObject();
			if (!ObjectItem)
			{
				return nullptr;
			}
		}
		return ObjectItem;
#else
		if (ObjectSerialNumber == 0)
		{
#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
			checkSlow(ObjectIndex == InvalidWeakObjectIndex); // otherwise this is a corrupted weak pointer
#else
			checkSlow(ObjectIndex == 0 || ObjectIndex == -1); // otherwise this is a corrupted weak pointer
#endif

			return nullptr;
		}

		if (ObjectIndex < 0)
		{
			return nullptr;
		}
		FUObjectItem* const ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
		if (!ObjectItem)
		{
			return nullptr;
		}
		if (!SerialNumbersMatch(ObjectItem))
		{
			return nullptr;
		}
		return ObjectItem;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** 
	* Private (inlined) version for internal use only. Remote object version of IsValid does not try to resolve remote objects. 
	* WeakObjectPtr is considered valid if it points to an existing local object or to a remote object.
	*/
	inline bool Internal_IsValid(bool bEvenIfGarbage, bool bThreadsafeTest) const
	{
		using namespace UE::Core::Private;

		if (IsExplicitlyNull())
		{
			return false;
		}
		if (ObjectIndex < 0)
		{
			return false;
		}

		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
		if (!ObjectItem || !SerialNumbersMatch(ObjectItem))
		{
			if (bEvenIfGarbage)
			{
				return CanBeResolved(UE::RemoteObject::Handle::ECanResolveObjectBehavior::Default);
			}
			else
			{
				return CanBeResolved(UE::RemoteObject::Handle::ECanResolveObjectBehavior::CannotResolveGarbageObjects);
			}
		}
		else if (bThreadsafeTest)
		{
			return true;
		}
		return GUObjectArray.IsValid(ObjectItem, bEvenIfGarbage);
	}
#else
	/** Private (inlined) version for internal use only. */
	inline bool Internal_IsValid(bool bEvenIfGarbage, bool bThreadsafeTest) const
	{
		FUObjectItem* const ObjectItem = Internal_GetObjectItem();
		if (bThreadsafeTest)
		{
			return (ObjectItem != nullptr);
		}
		else
		{
			return (ObjectItem != nullptr) && GUObjectArray.IsValid(ObjectItem, bEvenIfGarbage);
		}
	}
#endif

	/** Private (inlined) version for internal use only. */
	inline UObject* Internal_Get(bool bEvenIfGarbage) const
	{
		FUObjectItem* const ObjectItem = Internal_GetObjectItem();
		return ((ObjectItem != nullptr) && GUObjectArray.IsValid(ObjectItem, bEvenIfGarbage)) ? (UObject*)ObjectItem->GetObject() : nullptr;
	}

	COREUOBJECT_API TStrongObjectPtr<UObject> Internal_Pin(bool bEvenIfGarbage) const;
	COREUOBJECT_API TStrongObjectPtr<UObject> Internal_TryPin(bool& bOutPinValid, bool bEvenIfGarbage) const;
	
#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
	int32		ObjectIndex = UE::Core::Private::InvalidWeakObjectIndex;
	int32		ObjectSerialNumber = 0;
#else
	int32		ObjectIndex;
	int32		ObjectSerialNumber;
#endif // UE_WEAKOBJECTPTR_ZEROINIT_FIX
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FRemoteObjectId ObjectRemoteId;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

};

/** Hash function. */
UE_FORCEINLINE_HINT uint32 GetTypeHash(const FWeakObjectPtr& WeakObjectPtr)
{
	return WeakObjectPtr.GetTypeHash();
}
