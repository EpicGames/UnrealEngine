// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WeakObjectPtrFwd.h: Weak object pointer forward declarations
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "UObject/WeakObjectPtrTemplatesFwd.h"

struct FWeakObjectPtr;

template <typename T> struct TIsPODType;
template <typename T> struct TIsWeakPointerType;
template <typename T> struct TIsZeroConstructType;

template<> struct TIsPODType<FWeakObjectPtr> { enum { Value = true }; };
template<> struct TIsZeroConstructType<FWeakObjectPtr> { enum { Value = true }; };
template<> struct TIsWeakPointerType<FWeakObjectPtr> { enum { Value = true }; };

// IWYU pragma: end_exports
