// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassDefaultObjectTest.h"

#if WITH_TESTS

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClassDefaultObjectTest)

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "Async/ManualResetEvent.h"
#include "Tasks/Task.h"
#include "UObject/Package.h"

TEST_CASE("UE::CoreUObject::GetDefaultObjectRace", "[CoreUObject][GetDefaultObjectRace]")
{
	for (int32 Index = 0; Index < 1000; ++Index)
	{
		// Move old object out of the way and force recreation
		if (UClassDefaultObjectTest* Obj = (UClassDefaultObjectTest*)UClassDefaultObjectTest::StaticClass()->GetDefaultObject(false))
		{
			Obj->Rename(nullptr, GetTransientPackage());
		}
		UClassDefaultObjectTest::StaticClass()->SetDefaultObject(nullptr);

		std::atomic<bool> bContinue = true;
		UE::FManualResetEvent StartedEvent;
		int32 NumPostInitProperties = 0;

		// One worker thread tries to query the default object if its ready.
		UE::Tasks::FTask Task =
			UE::Tasks::Launch(
				TEXT("Task"),
				[&]()
				{
					StartedEvent.Notify();
					while (bContinue.load(std::memory_order_relaxed))
					{
						UClassDefaultObjectTest* Obj = (UClassDefaultObjectTest*)UClassDefaultObjectTest::StaticClass()->GetDefaultObject(false);
						if (Obj)
						{
							// The flag is only safe to read if we have a barrier between the time where the default object pointer is 
							// published, and the time we read the flag. It needs to be paired with a release barrier when the default
							// object is set.
							std::atomic_thread_fence(std::memory_order_acquire);
							TSAN_AFTER(Obj); // TSAN doesn't understand fence, help it understand what's going on
							FPlatformProcess::YieldCycles(FMath::RandRange(0, 4000));
							if (!Obj->HasAnyFlags(RF_NeedInitialization))
							{
								FPlatformProcess::YieldCycles(FMath::RandRange(0, 4000));
								// This value is only safe to read if RF_NeedInitialization has been removed
								// and that we applied the required barrier.
								std::atomic_thread_fence(std::memory_order_acquire);
								TSAN_AFTER(Obj); // TSAN doesn't understand fence, help it understand what's going on
								NumPostInitProperties = Obj->NumPostInitProperties;
								return;
							}
						}
					}
				}
			);

		// Make sure the async task has started
		CHECK(StartedEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1)));

		CHECK(UClassDefaultObjectTest::StaticClass()->GetDefaultObject(false) == nullptr);

		// Main thread creates the default object.
		UClassDefaultObjectTest::StaticClass()->GetDefaultObject(true);

		CHECK(Task.Wait(FTimespan::FromSeconds(1.0f)));

		// Store the loop to kill the task if it failed to return on time
		bContinue = false;
		CHECK(Task.Wait(FTimespan::FromSeconds(1.0f)));
		CHECK(NumPostInitProperties != 0);
	}

	// Move old object out of the way and force recreation
	if (UClassDefaultObjectTest* Obj = (UClassDefaultObjectTest*)UClassDefaultObjectTest::StaticClass()->GetDefaultObject(false))
	{
		Obj->Rename(nullptr, GetTransientPackage());
	}
	UClassDefaultObjectTest::StaticClass()->SetDefaultObject(nullptr);

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
}

#endif // WITH_LOW_LEVEL_TESTS
#endif // WITH_TESTS