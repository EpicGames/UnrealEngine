// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TESTS

#include "UObject/Object.h"

#include "ClassDefaultObjectTest.generated.h"

UCLASS()
class UClassDefaultObjectTest : public UObject
{
	GENERATED_BODY()

public:
	int32 NumPostInitProperties = 0;

	// PostInitProperties is called before RF_NeedInitialization is removed
	// so its a good place to modify a non-atomic value to see if there is 
	// any race during default object construction.
	virtual void PostInitProperties()
	{
		Super::PostInitProperties();

		NumPostInitProperties++;
	}
};

#endif