// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "ObjectPtrTestClass.h"
#include "UObject/ObjectHandle.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/ObjectResource.h"
#include "UObject/MetaData.h"
#include "HAL/PlatformProperties.h"
#include "ObjectRefTrackingTestBase.h"
#include "IO/IoDispatcher.h"
#include "TestHarness.h"
#include "UObject/ObjectRef.h"
#include "UObject/ObjectPathId.h"
#include "UObject/PropertyBagRepository.h"

static_assert(sizeof(FObjectHandle) == sizeof(void*), "FObjectHandle type must always compile to something equivalent to a pointer size.");

class FObjectHandleTestBase : public FObjectRefTrackingTestBase
{
public:
	
protected:
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	void TestResolveFailure(UE::CoreUObject::Private::FPackedObjectRef PackedRef)
	{
		FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
		FObjectHandle TargetHandle = FObjectHandle{ PackedRef.EncodedRef };
		UObject* ResolvedObject = FObjectPtr(TargetHandle).Get();
		ObjectRefMetrics.TestNumResolves(TEXT("NumResolves should be incremented by one after a resolve attempt"), 1);
		ObjectRefMetrics.TestNumReads(TEXT("NumReads should be incremented by one after a resolve attempt"), 1);

		CHECK(ResolvedObject == nullptr);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("NumFailedResolves should be incremented by one after a failed resolve attempt"), 1);
	}
#endif

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
	void TestResolvableNonNull(const ANSICHAR* PackageName, const ANSICHAR* ObjectName, bool bExpectSubRefReads)
	{

		FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
		FObjectRef TargetRef(FName(PackageName), NAME_None, NAME_None, UE::CoreUObject::Private::FObjectPathId(ObjectName));
		UObject* ResolvedObject = TargetRef.Resolve();
		FObjectPtr Ptr(ResolvedObject);
		(void)Ptr.Get();
		TEST_TRUE(TEXT("expected not null"), ResolvedObject != nullptr);
		ObjectRefMetrics.TestNumResolves(TEXT("NumResolves should be incremented by one after a resolve attempt"), 1);
		ObjectRefMetrics.TestNumReads(TEXT("NumReads should be incremented by one after a resolve attempt"), 1, bExpectSubRefReads /*bAllowAdditionalReads*/);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("NumFailedResolves should not change after a successful resolve attempt"), 0);
	}

	void TestResolveFailure(const ANSICHAR* PackageName, const ANSICHAR* ObjectName)
	{
		FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
		FObjectRef TargetRef(FName(PackageName), NAME_None, NAME_None, UE::CoreUObject::Private::FObjectPathId(ObjectName));
		const UObject* ResolvedObject = TargetRef.Resolve();
		ObjectRefMetrics.TestNumResolves(TEXT("NumResolves should be incremented by one after a resolve attempt"), 1);
		ObjectRefMetrics.TestNumReads(TEXT("NumReads should be incremented by one after a resolve attempt"), 1);
		CHECK(ResolvedObject == nullptr);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("NumFailedResolves should be incremented by one after a failed resolve attempt"), 1);
	}
#endif
};

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Null Behavior", "[CoreUObject][ObjectHandle]")
{
	FObjectHandle TargetHandle = UE::CoreUObject::Private::MakeObjectHandle(nullptr);

	TEST_TRUE(TEXT("Handle to target is null"), IsObjectHandleNull(TargetHandle));
	TEST_TRUE(TEXT("Handle to target is resolved"), IsObjectHandleResolved(TargetHandle));

	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
	UObject* ResolvedObject = UE::CoreUObject::Private::ResolveObjectHandle(TargetHandle);

	TEST_EQUAL(TEXT("Resolved object is equal to original object"), (UObject*)nullptr, ResolvedObject);

	ObjectRefMetrics.TestNumFailedResolves(TEXT("NumFailedResolves should not change after a resolve attempt on a null handle"), 0);
	ObjectRefMetrics.TestNumResolves(TEXT("NumResolves should not change after a resolve attempt on a null handle"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should be incremented by one after a resolve attempt on a null handle"), 1);
}

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Pointer Behavior", "[CoreUObject][ObjectHandle]")
{
	FObjectHandle TargetHandle = UE::CoreUObject::Private::MakeObjectHandle((UObject*)0x0042);

	TEST_FALSE(TEXT("Handle to target is null"), IsObjectHandleNull(TargetHandle));
	TEST_TRUE(TEXT("Handle to target is resolved"), IsObjectHandleResolved(TargetHandle));

	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
	UObject* ResolvedObject = UE::CoreUObject::Private::ResolveObjectHandle(TargetHandle);

	TEST_EQUAL(TEXT("Resolved object is equal to original object"), (UObject*)0x0042, ResolvedObject);

	ObjectRefMetrics.TestNumResolves(TEXT("NumResolves should not change after a resolve attempt on a pointer handle"), 0);
	ObjectRefMetrics.TestNumFailedResolves(TEXT("NumFailedResolves should not change after a resolve attempt on a pointer handle"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should be incremented by one after a resolve attempt on a pointer handle"),1);
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Resolve Engine Content Target", "[CoreUObject][ObjectHandle]")
{
	const FName TestPackageName("/Engine/Test/ObjectPtrDefaultSerialize/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* TestSoftObject = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));
	UObject* TestSubObject = NewObject<UObjectPtrTestClass>(TestSoftObject, TEXT("SubObject"));
	ON_SCOPE_EXIT{
		TestPackage->RemoveFromRoot();
	};

	TestResolvableNonNull("/Engine/Test/ObjectPtrDefaultSerialize/Transient", "DefaultSerializeObject.SubObject", true);
	TestResolvableNonNull("/Engine/Test/ObjectPtrDefaultSerialize/Transient", "DefaultSerializeObject", false);
}


// TODO: Disabled until warnings and errors related to loading a non-existent package have been fixed.
DISABLED_TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Resolve Non Existent Target", "[CoreUObject][ObjectHandle]")
{
	// Confirm we don't successfully resolve an incorrect reference to engine content
	TestResolveFailure("/Engine/EngineResources/NonExistentPackageName_0", "DefaultTexture");

	const FName TestPackageName("/Engine/Test/ObjectPtrDefaultSerialize/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* TestSoftObject = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));
	ON_SCOPE_EXIT{
		TestPackage->RemoveFromRoot();
	};

	TestResolveFailure("/Engine/Test/ObjectPtrDefaultSerialize/Transient", "DefaultSerializeObject_DoesNotExist");
}

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Resolve Script Target", "[CoreUObject][ObjectHandle]")
{
	// Confirm we successfully resolve a correct reference to engine content
	TestResolvableNonNull("/Script/CoreUObject", "MetaData", true);
}

#endif

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::TObjectPtr::HandleNullGetClass", "[CoreUObject][ObjectHandle]")
{
	TObjectPtr<UObject> Ptr = nullptr;
	TEST_TRUE(TEXT("TObjectPtr.GetClass should return null on a null object"), Ptr.GetClass() == nullptr);
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE("CoreUObject::FObjectHandle::Names")
{
	const FName TestPackageName("/Engine/Test/PackageResolve/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));
	ON_SCOPE_EXIT{
		TestPackage->RemoveFromRoot();
	};

	FObjectPtr Test;
	FObjectPtr PackagePtr(MakeUnresolvedHandle(TestPackage));
	FObjectPtr Obj1Ptr(MakeUnresolvedHandle(Obj1));

	CHECK(!PackagePtr.IsResolved());
	CHECK(TestPackage->GetPathName() == PackagePtr.GetPathName());
	CHECK(TestPackage->GetFName() == PackagePtr.GetFName());
	CHECK(TestPackage->GetName() == PackagePtr.GetName());
	CHECK(TestPackage->GetFullName() == PackagePtr.GetFullName());
	CHECK(!PackagePtr.IsResolved());

	CHECK(!Obj1Ptr.IsResolved());
	CHECK(Obj1->GetPathName() == Obj1Ptr.GetPathName());
	CHECK(Obj1->GetFName() == Obj1Ptr.GetFName());
	CHECK(Obj1->GetName() == Obj1Ptr.GetName());
	CHECK(Obj1->GetFullName() == Obj1Ptr.GetFullName());
	CHECK(!Obj1Ptr.IsResolved());
}
#endif

#if UE_WITH_OBJECT_HANDLE_TRACKING || UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE("CoreUObject::ObjectRef")
{
	const FName TestPackageName("/Engine/Test/ObjectRef/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner"));
	ON_SCOPE_EXIT{
		TestPackage->RemoveFromRoot();
	};

	{
		FObjectImport ObjectImport(Obj1);
		FObjectRef ObjectRef(Obj1);

		CHECK(ObjectImport.ClassPackage == ObjectRef.ClassPackageName);
		CHECK(ObjectImport.ClassName == ObjectRef.ClassName);
		CHECK(TestPackage->GetFName() == ObjectRef.PackageName);
	}

	{
		FObjectImport ObjectImport(Inner1);
		FObjectRef ObjectRef(Inner1);

		CHECK(ObjectImport.ClassPackage == ObjectRef.ClassPackageName);
		CHECK(ObjectImport.ClassName == ObjectRef.ClassName);
		CHECK(TestPackage->GetFName() == ObjectRef.PackageName);
	}
}

#endif

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::TObjectPtr::Null Behavior", "[CoreUObject][ObjectHandle]")
{
	TObjectPtr<UObject> Ptr = nullptr;
	UObjectPtrTestClass* TestObject = nullptr;

	uint32 ResolveCount = 0;
	auto ResolveDelegate = [&ResolveCount](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++ResolveCount;
		};
	auto Handle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback(ResolveDelegate);
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(Handle);
	};
	//compare against all flavours of nullptr, should not try and resolve this pointer
	CHECK(Ptr == nullptr); CHECK(ResolveCount == 0u);
	CHECK(nullptr == Ptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(Ptr != nullptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(nullptr != Ptr); CHECK(ResolveCount == 0u);
	CHECK(!Ptr); CHECK(ResolveCount == 0u);

	//using an if otherwise the macros try to convert to a pointer and not use the bool operator
	if (Ptr)
	{
		CHECK(false);
	}
	else
	{
		CHECK(true);
	}
	CHECK(ResolveCount == 0u);

	CHECK(Ptr == TestObject); CHECK(ResolveCount == 0u);
	CHECK(TestObject == Ptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(Ptr != TestObject); CHECK(ResolveCount == 0u);
	CHECK_FALSE(TestObject != Ptr); CHECK(ResolveCount == 0u);

	FObjectRef TargetRef(FName("SomePackage"), FName("ClassPackageName"), FName("ClassName"), UE::CoreUObject::Private::FObjectPathId("ObjectName"));
	UE::CoreUObject::Private::FPackedObjectRef PackedObjectRef = UE::CoreUObject::Private::MakePackedObjectRef(TargetRef);
	FObjectPtr ObjectPtr(FObjectHandle{ PackedObjectRef.EncodedRef });
	REQUIRE(!ObjectPtr.IsResolved()); //make sure not resolved

	//an unresolved pointers compared against nullptr should still not resolve
	Ptr = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjectPtr);
	CHECK_FALSE(Ptr == nullptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(nullptr == Ptr); CHECK(ResolveCount == 0u);
	CHECK(Ptr != nullptr); CHECK(ResolveCount == 0u);
	CHECK(nullptr != Ptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(!Ptr); CHECK(ResolveCount == 0u);

	//using an if otherwise the macros try to convert to a pointer and not use the bool operator
	if (Ptr)
	{
		CHECK(true);
	}
	else
	{
		CHECK(false);
	}
	CHECK(ResolveCount == 0u);

	//test an unresolve pointer against a null raw pointer
	CHECK_FALSE(Ptr == TestObject); CHECK(ResolveCount == 0u);
	CHECK_FALSE(TestObject == Ptr);	CHECK(ResolveCount == 0u);
	CHECK(Ptr != TestObject); CHECK(ResolveCount == 0u);
	CHECK(TestObject != Ptr); CHECK(ResolveCount == 0u);

	//creating a real object for something that can resolve
	const FName TestPackageName("/Engine/Test/ObjectPtrDefaultSerialize/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();

	const FName TestObjectName("MyObject");
	TestObject = NewObject<UObjectPtrTestClass>(TestPackage, TestObjectName, RF_Transient);
	TObjectPtr<UObject> TestNotLazyObject = NewObject<UObjectPtrNotLazyTestClass>(TestPackage, TEXT("NotLazy"), RF_Transient);

	//compare resolved ptr against nullptr
	TObjectPtr<UObject> ResolvedPtr = TestObject;
	CHECK(ResolvedPtr.IsResolved());
	CHECK(Ptr != ResolvedPtr);  CHECK(ResolveCount == 0u);
	CHECK(ResolvedPtr != Ptr);  CHECK(ResolveCount == 0u);
	CHECK_FALSE(Ptr == ResolvedPtr);  CHECK(ResolveCount == 0u);
	CHECK_FALSE(ResolvedPtr == Ptr);  CHECK(ResolveCount == 0u);

	//compare unresolved against nullptr
	FObjectPtr FPtr(MakeUnresolvedHandle(TestObject));
	TObjectPtr<UObject> UnResolvedPtr = *reinterpret_cast<TObjectPtr<UObject>*>(&FPtr);
	CHECK(!UnResolvedPtr.IsResolved());
	CHECK_FALSE(Ptr == UnResolvedPtr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(UnResolvedPtr == Ptr); CHECK(ResolveCount == 0u);
	CHECK(Ptr != UnResolvedPtr); CHECK(ResolveCount == 0u);
	CHECK(UnResolvedPtr != Ptr); CHECK(ResolveCount == 0u);

	//compare unresolved against resolved not equal
	CHECK_FALSE(TestNotLazyObject == UnResolvedPtr); CHECK(ResolveCount == 0u); 
	CHECK_FALSE(UnResolvedPtr == TestNotLazyObject); CHECK(ResolveCount == 0u);
	CHECK(TestNotLazyObject != UnResolvedPtr); CHECK(ResolveCount == 0u);
	CHECK(UnResolvedPtr != TestNotLazyObject); CHECK(ResolveCount == 0u);

	//compare resolved against naked pointer
	Ptr = TestObject;
	REQUIRE(Ptr.IsResolved());
	CHECK(Ptr == TestObject); CHECK(ResolveCount == 0u);
	CHECK(TestObject == Ptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(Ptr != TestObject); CHECK(ResolveCount == 0u);
	CHECK_FALSE(TestObject != Ptr); CHECK(ResolveCount == 0u);

	//compare resolved pointer and unresolved of the same object
	CHECK(Ptr == UnResolvedPtr); CHECK(ResolveCount == 0u);
	CHECK(UnResolvedPtr == Ptr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(Ptr != UnResolvedPtr); CHECK(ResolveCount == 0u);
	CHECK_FALSE(UnResolvedPtr != Ptr); CHECK(ResolveCount == 0u);

	TestObject = nullptr;
	CHECK_FALSE(Ptr == TestObject); CHECK(ResolveCount == 0u);
	CHECK_FALSE(TestObject == Ptr); CHECK(ResolveCount == 0u);
	CHECK(Ptr != TestObject); CHECK(ResolveCount == 0u);
	CHECK(TestObject != Ptr); CHECK(ResolveCount == 0u);

	TestObject = static_cast<UObjectPtrTestClass*>(Ptr.Get());
	Ptr = nullptr;
	CHECK_FALSE(Ptr == TestObject); CHECK(ResolveCount == 0u);
	CHECK_FALSE(TestObject == Ptr); CHECK(ResolveCount == 0u);
	CHECK(Ptr != TestObject); CHECK(ResolveCount == 0u);
	CHECK(TestObject != Ptr); CHECK(ResolveCount == 0u);

}

#endif

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Resolve Malformed Handle", "[CoreUObject][ObjectHandle]")
{
	// make one packed ref guarantee something is in the object handle index
	FObjectRef TargetRef(FName("/Test/DummyPackage"), FName("ClassPackageName"), FName("ClassName"), UE::CoreUObject::Private::FObjectPathId("DummyObjectName"));
	UE::CoreUObject::Private::MakePackedObjectRef(TargetRef);

	uint32 ObjectId = ~0u;
	UPTRINT PackedId = ObjectId << 1 | 1;
	UE::CoreUObject::Private::FPackedObjectRef PackedObjectRef{ PackedId };
	TestResolveFailure(PackedObjectRef); // packed ref has a valid package id but invalid object id

	TestResolveFailure(UE::CoreUObject::Private::FPackedObjectRef { 0xFFFF'FFFF'FFFF'FFFFull });
	TestResolveFailure(UE::CoreUObject::Private::FPackedObjectRef { 0xEFEF'EFEF'EFEF'EFEFull });
}
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Hash Object Without Index", "[CoreUObject][ObjectHandle]")
{
	UObject DummyObjectWithInvalidIndex(EC_StaticConstructor, RF_NoFlags);
	CHECK(DummyObjectWithInvalidIndex.GetUniqueID() == -1);

	FObjectHandle DummyObjectHandle = UE::CoreUObject::Private::MakeObjectHandle(&DummyObjectWithInvalidIndex);
	CHECK(GetTypeHash(DummyObjectHandle) == GetTypeHash(&DummyObjectWithInvalidIndex));
}

#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
TEST_CASE_METHOD(FObjectHandleTestBase, "CoreUObject::FObjectHandle::Type Safety", "[CoreUObject][ObjectHandle]")
{
	const FName TestPackageName("/Engine/Test/ObjectHandle/TypeSafety/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};

	// construct an unsafe class type
	UClass* TestUnsafeClass = UE::FPropertyBagRepository::CreatePropertyBagPlaceholderClass(TestPackage, UClass::StaticClass(), TEXT("TestUnsafeClass"));

	// construct objects for testing
	UObjectPtrTestClass* TestSafeObject = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("TestSafeObject"), RF_Transient);
	UObject* TestUnsafeObject = NewObject<UObject>(TestPackage, TestUnsafeClass, TEXT("TestUnsafeObject"), RF_Transient);

	// invalid address value for testing
	UObject* TestInvalidAddress = (UObject*)0xFFFF'FFFF'FFFF'FFFCull;

	// construct object handles for testing
	FObjectHandle NullObjectHandle = UE::CoreUObject::Private::MakeObjectHandle(nullptr);
	FObjectHandle TestSafeObjectHandle = UE::CoreUObject::Private::MakeObjectHandle(TestSafeObject);
	FObjectHandle TestSafeInvalidAddressHandle = UE::CoreUObject::Private::MakeObjectHandle(TestInvalidAddress);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectHandle TestLateResolveSafeObjectHandle{ UE::CoreUObject::Private::MakePackedObjectRef(TestSafeObject).EncodedRef };
	FObjectHandle TestLateResolveUnsafeObjectHandle{ UE::CoreUObject::Private::MakePackedObjectRef(TestUnsafeObject).EncodedRef };
	FObjectHandle TestLateResolveSafeInvalidAddressHandle{ (UE::CoreUObject::Private::FPackedObjectRef { reinterpret_cast<UPTRINT>(TestInvalidAddress) | 1 }).EncodedRef };
	FObjectHandle TestLateResolveUnsafeInvalidAddressHandle{ (UE::CoreUObject::Private::FPackedObjectRef { reinterpret_cast<UPTRINT>(TestInvalidAddress) | (1 << UE::CoreUObject::Private::TypeIdShift) | 1 }).EncodedRef };
#endif

	// NULL/type-safe object handles should report as being safe
	CHECK(IsObjectHandleTypeSafe(NullObjectHandle));
	CHECK(IsObjectHandleTypeSafe(TestSafeObjectHandle));
	CHECK(IsObjectHandleTypeSafe(TestSafeInvalidAddressHandle));
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(IsObjectHandleTypeSafe(TestLateResolveSafeObjectHandle));
	CHECK(IsObjectHandleTypeSafe(TestLateResolveSafeInvalidAddressHandle));
#endif

	// unsafe type object handles should report as being unsafe
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK_FALSE(IsObjectHandleTypeSafe(TestLateResolveUnsafeObjectHandle));
	CHECK_FALSE(IsObjectHandleTypeSafe(TestLateResolveUnsafeInvalidAddressHandle));
#endif

	// unsafe type object handles should resolve the class to the unsafe type
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(UE::CoreUObject::Private::ResolveObjectHandleClass(TestLateResolveUnsafeObjectHandle) == TestUnsafeClass);
#endif

	// an unsafe type object handle should not equate to other unsafe type object handles (including NULL), except for itself
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(NullObjectHandle != TestLateResolveUnsafeObjectHandle);
	CHECK(TestLateResolveUnsafeObjectHandle != NullObjectHandle);
	CHECK(TestSafeObjectHandle != TestLateResolveUnsafeObjectHandle);
	CHECK(TestLateResolveUnsafeObjectHandle != TestSafeObjectHandle);
	CHECK(TestLateResolveSafeObjectHandle != TestLateResolveUnsafeObjectHandle);
	CHECK(TestLateResolveUnsafeObjectHandle != TestLateResolveSafeObjectHandle);
	CHECK(TestLateResolveUnsafeObjectHandle == TestLateResolveUnsafeObjectHandle);
//	CHECK(TestSafeInvalidAddressHandle != TestLateResolveUnsafeInvalidAddressHandle);			// note: commented out for now; FObjectHandle::operator==() will call FindExistingPackedObjectRef() when comparing
//	CHECK(TestLateResolveUnsafeInvalidAddressHandle != TestSafeInvalidAddressHandle);			// resolved to unresolved values, which will then attempt to dereference the resolved address and crash (known issue)
	CHECK(TestLateResolveSafeInvalidAddressHandle != TestLateResolveUnsafeInvalidAddressHandle);
	CHECK(TestLateResolveUnsafeInvalidAddressHandle != TestLateResolveSafeInvalidAddressHandle);
	CHECK(TestLateResolveUnsafeInvalidAddressHandle == TestLateResolveUnsafeInvalidAddressHandle);
#endif

	// the type safety and class queries above should not have resolved an object handle that's using late resolve
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveSafeObjectHandle));
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveUnsafeObjectHandle));
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveSafeInvalidAddressHandle));
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveUnsafeInvalidAddressHandle));
#endif

	// unsafe type object handles should resolve/evaluate to the original object/address, or NULL for an invalid object w/ late resolve
	CHECK(UE::CoreUObject::Private::ResolveObjectHandle(TestSafeInvalidAddressHandle) == TestInvalidAddress);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(UE::CoreUObject::Private::ResolveObjectHandle(TestLateResolveSafeObjectHandle) == TestSafeObject);
	CHECK(UE::CoreUObject::Private::ResolveObjectHandle(TestLateResolveUnsafeObjectHandle) == TestUnsafeObject);
	CHECK(UE::CoreUObject::Private::ResolveObjectHandle(TestLateResolveSafeInvalidAddressHandle) == nullptr);
	CHECK(UE::CoreUObject::Private::ResolveObjectHandle(TestLateResolveUnsafeInvalidAddressHandle) == nullptr);
#endif

	// unsafe type object handles should report as NOT being resolved (in order to preserve the bit flag on the underlying packed reference)
	CHECK(IsObjectHandleResolved(TestSafeInvalidAddressHandle));
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(IsObjectHandleResolved(TestLateResolveSafeObjectHandle));
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveUnsafeObjectHandle));
	CHECK(IsObjectHandleResolved(TestLateResolveSafeInvalidAddressHandle));
	CHECK_FALSE(IsObjectHandleResolved(TestLateResolveUnsafeInvalidAddressHandle));
#endif

	// construct object pointers for testing intentionally different behaviors of UObject-type vs. non-UObject-type bindings
	TObjectPtr<UObject> NullObjectPtr(nullptr);
	TObjectPtr<UObject> TestSafeObjectPtr(TestUnsafeObject);								// type safe pointer to placeholder (bound to UObject type)
	TObjectPtr<const UObject> TestSafeConstObjectPtr(TestUnsafeObject);						// type safe const pointer to placeholder (bound to UObject type)
	FObjectPtr TestSafeInvalidObjectPtr_Untyped(TestSafeInvalidAddressHandle);
	TObjectPtr<UObject> TestSafeInvalidObjectPtr(TestSafeInvalidObjectPtr_Untyped);
	TObjectPtr<const UObject> TestSafeInvalidConstObjectPtr(TestSafeInvalidObjectPtr_Untyped);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	// note: "safe" in this context means the pointer should be type safe because it's bound to the UObject base type, but both reference the same "unsafe" object and as such do not update the handle on resolve
	FObjectPtr TestLateResolveUnsafeObjectPtr_Untyped(TestLateResolveUnsafeObjectHandle);
	TObjectPtr<UObject> TestLateResolveSafeObjectPtr(TestLateResolveUnsafeObjectPtr_Untyped);
	TObjectPtr<const UObject> TestLateResolveSafeConstObjectPtr(TestLateResolveUnsafeObjectPtr_Untyped);
	TObjectPtr<UObjectPtrTestClass> TestLateResolveUnsafeObjectPtr(TestLateResolveUnsafeObjectPtr_Untyped);
	TObjectPtr<const UObjectPtrTestClass> TestLateResolveUnsafeConstObjectPtr(TestLateResolveUnsafeObjectPtr_Untyped);
	FObjectPtr TestLateResolveUnsafeInvalidObjectPtr_Untyped(TestLateResolveUnsafeInvalidAddressHandle);
	TObjectPtr<UObject> TestLateResolveSafeInvalidObjectPtr(TestLateResolveUnsafeInvalidObjectPtr_Untyped);
	TObjectPtr<const UObject> TestLateResolveSafeInvalidConstObjectPtr(TestLateResolveUnsafeInvalidObjectPtr_Untyped);
	TObjectPtr<UObjectPtrTestClass> TestLateResolveUnsafeInvalidObjectPtr(TestLateResolveUnsafeInvalidObjectPtr_Untyped);
	TObjectPtr<const UObjectPtrTestClass> TestLateResolveUnsafeInvalidConstObjectPtr(TestLateResolveUnsafeInvalidObjectPtr_Untyped);
	TArray<TObjectPtr<UObject>> TestLateResolveSafeObjectPtrArray = { TestLateResolveSafeObjectPtr, TestLateResolveSafeInvalidObjectPtr };
	TArray<TObjectPtr<UObjectPtrTestClass>> TestLateResolveUnsafeObjectPtrArray = { TestLateResolveUnsafeObjectPtr, TestLateResolveUnsafeInvalidObjectPtr };
	TArray<TObjectPtr<const UObject>> TestLateResolveSafeConstObjectPtrArray = { TestLateResolveSafeConstObjectPtr, TestLateResolveSafeInvalidConstObjectPtr };
	TArray<TObjectPtr<const UObjectPtrTestClass>> TestLateResolveUnsafeConstObjectPtrArray = { TestLateResolveUnsafeConstObjectPtr, TestLateResolveUnsafeInvalidConstObjectPtr };
#endif

	// type safe object pointers should evaluate to true/non-NULL (including invalid pointers)
	CHECK(TestSafeObjectPtr);
	CHECK(!!TestSafeObjectPtr);
	CHECK(NULL != TestSafeObjectPtr);
	CHECK(TestSafeObjectPtr != NULL);
	CHECK(nullptr != TestSafeObjectPtr);
	CHECK(TestSafeObjectPtr != nullptr);
	CHECK(TestSafeConstObjectPtr);
	CHECK(!!TestSafeConstObjectPtr);
	CHECK(NULL != TestSafeConstObjectPtr);
	CHECK(TestSafeConstObjectPtr != NULL);
	CHECK(nullptr != TestSafeConstObjectPtr);
	CHECK(TestSafeConstObjectPtr != nullptr);
	CHECK(TestSafeInvalidObjectPtr);
	CHECK(!!TestSafeInvalidObjectPtr);
	CHECK(NULL != TestSafeInvalidObjectPtr);
	CHECK(TestSafeInvalidObjectPtr != NULL);
	CHECK(nullptr != TestSafeInvalidObjectPtr);
	CHECK(TestSafeInvalidObjectPtr != nullptr);
	CHECK(TestSafeInvalidConstObjectPtr);
	CHECK(!!TestSafeInvalidConstObjectPtr);
	CHECK(NULL != TestSafeInvalidConstObjectPtr);
	CHECK(TestSafeInvalidConstObjectPtr != NULL);
	CHECK(nullptr != TestSafeInvalidConstObjectPtr);
	CHECK(TestSafeInvalidConstObjectPtr != nullptr);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(TestLateResolveSafeObjectPtr);
	CHECK(!!TestLateResolveSafeObjectPtr);
	CHECK(NULL != TestLateResolveSafeObjectPtr);
	CHECK(TestLateResolveSafeObjectPtr != NULL);
	CHECK(nullptr != TestLateResolveSafeObjectPtr);
	CHECK(TestLateResolveSafeObjectPtr != nullptr);
	CHECK(TestLateResolveSafeConstObjectPtr);
	CHECK(!!TestLateResolveSafeConstObjectPtr);
	CHECK(NULL != TestLateResolveSafeConstObjectPtr);
	CHECK(TestLateResolveSafeConstObjectPtr != NULL);
	CHECK(nullptr != TestLateResolveSafeConstObjectPtr);
	CHECK(TestLateResolveSafeConstObjectPtr != nullptr);
	CHECK(TestLateResolveSafeInvalidObjectPtr);
	CHECK(!!TestLateResolveSafeInvalidObjectPtr);
	CHECK(NULL != TestLateResolveSafeInvalidObjectPtr);
	CHECK(TestLateResolveSafeInvalidObjectPtr != NULL);
	CHECK(nullptr != TestLateResolveSafeInvalidObjectPtr);
	CHECK(TestLateResolveSafeInvalidObjectPtr != nullptr);
	CHECK(TestLateResolveSafeInvalidConstObjectPtr);
	CHECK(!!TestLateResolveSafeInvalidConstObjectPtr);
	CHECK(NULL != TestLateResolveSafeInvalidConstObjectPtr);
	CHECK(TestLateResolveSafeInvalidConstObjectPtr != NULL);
	CHECK(nullptr != TestLateResolveSafeInvalidConstObjectPtr);
	CHECK(TestLateResolveSafeInvalidConstObjectPtr != nullptr);
#endif

	// unsafe type object pointers should evaluate to NULL/false (for type safety)
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK_FALSE(TestLateResolveUnsafeObjectPtr);
	CHECK_FALSE(!!TestLateResolveUnsafeObjectPtr);
	CHECK(NULL == TestLateResolveUnsafeObjectPtr);
	CHECK(TestLateResolveUnsafeObjectPtr == NULL);
	CHECK(nullptr == TestLateResolveUnsafeObjectPtr);
	CHECK(TestLateResolveUnsafeObjectPtr == nullptr);
	CHECK_FALSE(TestLateResolveUnsafeConstObjectPtr);
	CHECK_FALSE(!!TestLateResolveUnsafeConstObjectPtr);
	CHECK(NULL == TestLateResolveUnsafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr == NULL);
	CHECK(nullptr == TestLateResolveUnsafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr == nullptr);
	CHECK_FALSE(TestLateResolveUnsafeInvalidObjectPtr);
	CHECK_FALSE(!!TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(NULL == TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr == NULL);
	CHECK(nullptr == TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr == nullptr);
	CHECK_FALSE(TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK_FALSE(!!TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(NULL == TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr == NULL);
	CHECK(nullptr == TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr == nullptr);
#endif

	// an unsafe type object pointer should not equate to other unsafe type object pointers, excluding NULL and itself
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(NullObjectPtr == TestLateResolveUnsafeObjectPtr);		// note: this intentionally differs from object *handles* (see above)
	CHECK(TestLateResolveUnsafeObjectPtr == NullObjectPtr);		// see note directly above
	CHECK(TestSafeObjectPtr != TestLateResolveUnsafeObjectPtr);
	CHECK(TestLateResolveUnsafeObjectPtr != TestSafeObjectPtr);
	CHECK(NullObjectPtr != TestLateResolveSafeConstObjectPtr);
	CHECK(TestLateResolveSafeConstObjectPtr != NullObjectPtr);
	CHECK(NullObjectPtr == TestLateResolveUnsafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr == NullObjectPtr);
	CHECK(TestSafeConstObjectPtr != TestLateResolveUnsafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr != TestSafeConstObjectPtr);
	CHECK(TestLateResolveSafeObjectPtr != TestLateResolveUnsafeObjectPtr);
	CHECK(TestLateResolveUnsafeObjectPtr != TestLateResolveSafeObjectPtr);
	CHECK(TestLateResolveSafeConstObjectPtr != TestLateResolveUnsafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr != TestLateResolveSafeConstObjectPtr);
	CHECK(TestLateResolveUnsafeObjectPtr == TestLateResolveUnsafeObjectPtr);
	CHECK(TestLateResolveUnsafeConstObjectPtr == TestLateResolveUnsafeConstObjectPtr);
	CHECK(NullObjectPtr == TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr == NullObjectPtr);
	CHECK(TestSafeInvalidObjectPtr != TestLateResolveUnsafeInvalidObjectPtr);						// note: these do not result in a handle-to-handle test because the underlying
	CHECK(TestLateResolveUnsafeInvalidObjectPtr != TestSafeInvalidObjectPtr);						// handle will resolve to NULL for the late resolve case with an invalid address
	CHECK(TestLateResolveSafeInvalidObjectPtr != TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr != TestLateResolveSafeInvalidObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr == TestLateResolveUnsafeInvalidObjectPtr);
	CHECK(NullObjectPtr == TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr == NullObjectPtr);
	CHECK(TestSafeInvalidConstObjectPtr != TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr != TestSafeInvalidConstObjectPtr);
	CHECK(TestLateResolveSafeInvalidConstObjectPtr != TestLateResolveUnsafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr != TestLateResolveSafeInvalidConstObjectPtr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr == TestLateResolveUnsafeInvalidConstObjectPtr);
#endif

	// an unsafe type object should evaluate the object's attributes correctly (applicable only to valid object pointers)
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(TestLateResolveUnsafeObjectPtr.GetName() == TestUnsafeObject->GetName());
	CHECK(TestLateResolveUnsafeObjectPtr.GetFName() == TestUnsafeObject->GetFName());
	CHECK(TestLateResolveUnsafeObjectPtr.GetPathName() == TestUnsafeObject->GetPathName());
	CHECK(TestLateResolveUnsafeObjectPtr.GetFullName() == TestUnsafeObject->GetFullName());
	CHECK(TestLateResolveUnsafeObjectPtr.GetOuter() == TestUnsafeObject->GetOuter());
	CHECK(TestLateResolveUnsafeObjectPtr.GetClass() == TestUnsafeObject->GetClass());
	CHECK(TestLateResolveUnsafeObjectPtr.GetPackage() == TestUnsafeObject->GetPackage());
#endif

	// the type safety and queries above should not have resolved an object pointer that's using late resolve
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK_FALSE(TestLateResolveSafeObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveSafeConstObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeConstObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveSafeInvalidObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeInvalidObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveSafeInvalidConstObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeInvalidConstObjectPtr.IsResolved());
#endif

	// a type safe object pointer should resolve to a non-NULL value when dereferenced, or NULL for an invalid object w/ late resolve
	CHECK(TestSafeObjectPtr.Get() == TestUnsafeObject);
	CHECK(TestSafeConstObjectPtr.Get() == TestUnsafeObject);
	CHECK(TestSafeInvalidObjectPtr.Get() == TestInvalidAddress);
	CHECK(TestSafeInvalidConstObjectPtr.Get() == TestInvalidAddress);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(TestLateResolveSafeObjectPtr.Get() == TestUnsafeObject);
	CHECK(TestLateResolveSafeConstObjectPtr.Get() == TestUnsafeObject);
	CHECK(TestLateResolveSafeInvalidObjectPtr.Get() == nullptr);
	CHECK(TestLateResolveSafeInvalidConstObjectPtr.Get() == nullptr);
#endif

	// an unsafe type object pointer should resolve to NULL when dereferenced (for type safety)
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(TestLateResolveUnsafeObjectPtr.Get() == nullptr);
	CHECK(TestLateResolveUnsafeConstObjectPtr.Get() == nullptr);
	CHECK(TestLateResolveUnsafeInvalidObjectPtr.Get() == nullptr);
	CHECK(TestLateResolveUnsafeInvalidConstObjectPtr.Get() == nullptr);
#endif

	// unsafe type object pointers should report as NOT being resolved; all others as resolved
	CHECK(TestSafeObjectPtr.IsResolved());
	CHECK(TestSafeConstObjectPtr.IsResolved());
	CHECK(TestSafeInvalidObjectPtr.IsResolved());
	CHECK(TestSafeInvalidConstObjectPtr.IsResolved());
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK_FALSE(TestLateResolveSafeObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveSafeConstObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveSafeInvalidObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeConstObjectPtr.IsResolved());
	CHECK_FALSE(TestLateResolveUnsafeInvalidObjectPtr.IsResolved());
#endif

	// unsafe type object pointers should not convert to raw pointer / array
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(ToRawPtr(TestLateResolveSafeObjectPtr) == TestUnsafeObject);
	CHECK(ToRawPtr(TestLateResolveSafeConstObjectPtr) == TestUnsafeObject);
	CHECK(ToRawPtr(TestLateResolveUnsafeObjectPtr) == nullptr);
	CHECK(ToRawPtr(TestLateResolveUnsafeConstObjectPtr) == nullptr);
	CHECK(ToRawPtr(TestLateResolveSafeInvalidObjectPtr) == nullptr);
	CHECK(ToRawPtr(TestLateResolveUnsafeInvalidObjectPtr) == nullptr);

	SKIP(); // comment this out to run the tests below (they should assert as they will not be resolved)
	CHECK(true); // Catch2 bug, will mark code past SKIP as unreachable at compile time. This small hack silences the error.

	(void)ToRawPtrTArrayUnsafe(TestLateResolveSafeObjectPtrArray);
	(void)ToRawPtrTArrayUnsafe(TestLateResolveSafeConstObjectPtrArray);
	(void)ToRawPtrTArrayUnsafe(TestLateResolveUnsafeObjectPtrArray);
	(void)ToRawPtrTArrayUnsafe(TestLateResolveUnsafeConstObjectPtrArray);
#endif
}
#endif

#endif
