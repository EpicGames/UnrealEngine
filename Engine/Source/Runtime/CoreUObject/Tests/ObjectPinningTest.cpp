// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "Tests/Benchmark.h"
#include "Tasks/Task.h"
#include "UObject/GCObject.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::CoreObject::Private::Tests
{

void ObjectPinningStressTest(int32 Count, int32 NumTasks)
{
	TArray<UE::Tasks::FTask> Tasks;
	Tasks.Reserve(NumTasks);

	std::atomic<int64> Success{ 0 };
	std::atomic<int64> Failure{ 0 };

	for (int32 Index = 0; Index < Count; ++Index)
	{
		UObject* Obj = NewObject<UClass>(GetTransientPackage());

		UE::Tasks::FTaskEvent Trigger(TEXT("Trigger"));
		FWeakObjectPtr WeakPtr = Obj;
		auto Lambda =
			[WeakPtr, &Success, &Failure]()
			{
				FPlatformProcess::YieldCycles(FMath::RandRange(0, 10000));

				if (TStrongObjectPtr<UObject> StrongPtr = WeakPtr.Pin())
				{
					Success++;
					for (int i = 0; i < 1000; ++i)
					{
						check(WeakPtr.IsValid());
					}
				}
				else
				{
					Failure++;
				}
			};

		Tasks.Reset();

		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			Tasks.Emplace(UE::Tasks::Launch(TEXT("Pin"), [Lambda]() { Lambda(); }, Trigger));
		}

		Trigger.Trigger();

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		UE::Tasks::Wait(Tasks);
		if (WeakPtr.IsValid())
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			check(!WeakPtr.IsValid());
		}
	}
}

TEST_CASE("CoreUObject::ObjectPinning::Stress Tests", "[CoreUObject][ObjectPinning]")
{
	// We need to be sure we've done the static GC initialization before we start doing a garbage collection.
	FGCObject::StaticInit();

	// Ignore oversubscription limit
	LowLevelTasks::FScheduler::Get().GetOversubscriptionLimitReachedEvent().Clear();

	UE_BENCHMARK(5, [] { ObjectPinningStressTest(1000, 1); } );
	UE_BENCHMARK(5, [] { ObjectPinningStressTest(1000, 2); });
	UE_BENCHMARK(5, [] { ObjectPinningStressTest(1000, 4); });
}

} // namespace UE::CoreObject::Private::Tests

#endif // WITH_LOW_LEVEL_TESTS
