// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "ObjectPtrTestClass.h"
#include "UObject/Package.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/ScopeExit.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/ObjectHandleTracking.h"
#include "UObject/PropertyHelper.h"
#include "Tests/WarnFilterScope.h"
#include "Misc/AssetRegistryInterface.h"
#include "AssetRegistry/AssetData.h"
#include "ObjectRefTrackingTestBase.h"

TEST_CASE("UE::CoreUObject::FObjectProperty::CheckValidAddress")
{
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto CallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([](const FObjectRef&, UPackage*, UObject*)
		{
			FAIL("Unexpected resolve during CheckValidObject");
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackHandle);
	};
#endif
	UClass* Class = UObjectPtrTestClassWithRef::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(Property != nullptr);

	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};
	UObjectPtrTestClassWithRef* Obj = NewObject<UObjectPtrTestClassWithRef>(TestPackage, TEXT("Object"));
	UObjectPtrTestClass* Other = NewObject<UObjectPtrTestClass>(Obj, TEXT("Other"));

#if UE_WITH_OBJECT_HANDLE_TRACKING && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectHandle Handle = MakeUnresolvedHandle(Other);
	TObjectPtr<UObjectPtrTestClass> ObjectPtr = *reinterpret_cast<TObjectPtr<UObjectPtrTestClass>*>(&Handle);
#else
	TObjectPtr<UObjectPtrTestClass> ObjectPtr = Other;
#endif

	//verify nothing happens by default
	CHECK(!Obj->ObjectPtr);
	Property->CheckValidObject(&Obj->ObjectPtr, nullptr);
	CHECK(!Obj->ObjectPtr);

	//valid assignment
	Obj->ObjectPtr = ObjectPtr;
	Property->CheckValidObject(&Obj->ObjectPtr, nullptr);
	CHECK(Obj->ObjectPtr == ObjectPtr);

	//assign a bad value to the pointer
	Obj->ObjectPtr = reinterpret_cast<UObjectPtrTestClass*>(Obj);
	CHECK(Obj->ObjectPtr != nullptr);

	FWarnFilterScope _([](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)
		{
			if (Category == TEXT("LogProperty") && FCString::Strstr(Message, TEXT("Reference will be nulled")) && Verbosity == ELogVerbosity::Type::Warning)
			{
				return true;
			}
			return false;
		});
	Property->CheckValidObject(&Obj->ObjectPtr, ObjectPtr);
	CHECK(!Obj->ObjectPtr); //value should be nulled since the type was not compatible	
}

TEST_CASE("UE::CoreUObject::FObjectProperty::CheckValidAddressNonNullable")
{
	bool bAllowRead = false;
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto CallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([&bAllowRead](const FObjectRef & SourceRef, UPackage * ClassPackage, UObject* Object)
		{
			if (!bAllowRead)
			{
				FAIL("Unexpected resolve during CheckValidObject");
			}
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackHandle);
	};
#endif
	UClass* Class = UObjectPtrTestClassWithRef::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtrNonNullable")));
	FObjectProperty* AbstractProperty = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtrAbstractNonNullable")));
	// Force non-nullable flag on because UHT does not yet support TNonNullPtr<TObjectPtr<UObject>>
	Property->SetPropertyFlags(CPF_NonNullable);
	AbstractProperty->SetPropertyFlags(CPF_NonNullable);
	REQUIRE(Property != nullptr);
	REQUIRE(AbstractProperty != nullptr);

	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/TestPackageName"), RF_Transient);
	UPackage* OtherTestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/CheckValidAddressNonNullableOther"), RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};
	UObjectPtrTestClassWithRef* Obj = NewObject<UObjectPtrTestClassWithRef>(TestPackage, TEXT("Object"));
	UObjectPtrTestClass* Other = NewObject<UObjectPtrTestClass>(Obj, TEXT("Other"));
	UObjectPtrAbstractDerivedTestClass* AbstractDerivedOther = NewObject<UObjectPtrAbstractDerivedTestClass>(Obj, TEXT("AbstractDerivedOther"));

#if UE_WITH_OBJECT_HANDLE_TRACKING && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectHandle Handle = MakeUnresolvedHandle(Other);
	FObjectHandle AbstractDerivedHandle = MakeUnresolvedHandle(AbstractDerivedOther);
	TObjectPtr<UObjectPtrTestClass> ObjectPtr = *reinterpret_cast<TObjectPtr<UObjectPtrTestClass>*>(&Handle);
	TObjectPtr<UObjectPtrAbstractDerivedTestClass> AbstractDerivedObjectPtr = *reinterpret_cast<TObjectPtr<UObjectPtrAbstractDerivedTestClass>*>(&AbstractDerivedHandle);
#else
	TObjectPtr<UObjectPtrTestClass> ObjectPtr = Other;
	TObjectPtr<UObjectPtrAbstractDerivedTestClass> AbstractDerivedObjectPtr = AbstractDerivedOther;
#endif

	//property is already null should stay null
	CHECK(!Obj->ObjectPtrNonNullable);
	CHECK(!Obj->ObjectPtrAbstractNonNullable);
	Property->CheckValidObject(&Obj->ObjectPtrNonNullable, ObjectPtr);
	AbstractProperty->CheckValidObject(&Obj->ObjectPtrAbstractNonNullable, AbstractDerivedObjectPtr);
	CHECK(!Obj->ObjectPtrNonNullable);
	CHECK(!Obj->ObjectPtrAbstractNonNullable);

	//valid assignment
	Obj->ObjectPtrNonNullable = ObjectPtr;
	Obj->ObjectPtrAbstractNonNullable = AbstractDerivedObjectPtr;
	Property->CheckValidObject(&Obj->ObjectPtrNonNullable, nullptr);
	AbstractProperty->CheckValidObject(&Obj->ObjectPtrAbstractNonNullable, nullptr);
	CHECK(Obj->ObjectPtrNonNullable == ObjectPtr);
	CHECK(Obj->ObjectPtrAbstractNonNullable == AbstractDerivedObjectPtr);

	// Disable property warnings that will fire because we're deliberately setting non-nullable properties to null
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogProperty, ELogVerbosity::NoLogging);

	using UE::CoreUObject::Private::ENonNullableBehavior;
	using UE::CoreUObject::Private::GetNonNullableBehavior;
	ENonNullableBehavior NonNullableBehavior = GetNonNullableBehavior();

	bAllowRead = true; //has resolve the old value to construct a new default value for the property

	//assign a bad value to the pointer
	Obj->ObjectPtrNonNullable = reinterpret_cast<UObjectPtrTestClass*>(OtherTestPackage);
	Obj->ObjectPtrAbstractNonNullable = reinterpret_cast<UObjectPtrAbstractTestClass*>(OtherTestPackage);
	CHECK(Obj->ObjectPtrNonNullable != nullptr);
	CHECK(Obj->ObjectPtrAbstractNonNullable != nullptr);

	FWarnFilterScope _([](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)
		{
			if (Category == TEXT("LogProperty") && FCString::Strstr(Message, TEXT("Reference will be defaulted to")) && Verbosity == ELogVerbosity::Type::Warning)
			{
				return true;
			}
			return false;
		});
	Property->CheckValidObject(&Obj->ObjectPtrNonNullable, ObjectPtr);
	AbstractProperty->CheckValidObject(&Obj->ObjectPtrAbstractNonNullable, AbstractDerivedObjectPtr);
	if (NonNullableBehavior == ENonNullableBehavior::CreateDefaultObjectIfPossible)
	{
		CHECK(Obj->ObjectPtrNonNullable == ObjectPtr); //non nullable properties should be assigned the old value
		CHECK(Obj->ObjectPtrAbstractNonNullable == AbstractDerivedObjectPtr); //abstract non nullable properties should be assigned the old value
	}
	else
	{
		CHECK(Obj->ObjectPtrNonNullable == nullptr); //non nullable properties should be nulled if invalid
		CHECK(Obj->ObjectPtrAbstractNonNullable == nullptr); //abstract non nullable properties should be nulled
	}

	//assign a bad value to the pointer
	Obj->ObjectPtrNonNullable = reinterpret_cast<UObjectPtrTestClass*>(Obj);
	Obj->ObjectPtrAbstractNonNullable = reinterpret_cast<UObjectPtrAbstractTestClass*>(Obj);
	CHECK(Obj->ObjectPtrNonNullable != nullptr);
	CHECK(Obj->ObjectPtrAbstractNonNullable != nullptr);

	if (NonNullableBehavior == ENonNullableBehavior::CreateDefaultObjectIfPossible)
	{
		//new value is required for non nullable properties
		Property->CheckValidObject(&Obj->ObjectPtrNonNullable, nullptr);
		AbstractProperty->CheckValidObject(&Obj->ObjectPtrAbstractNonNullable, nullptr);
		CHECK(Obj->ObjectPtrNonNullable != nullptr);
		CHECK(Obj->ObjectPtrNonNullable->IsA(UObjectPtrTestClass::StaticClass()));
		CHECK(Obj->ObjectPtrAbstractNonNullable == nullptr);
	}
	else
	{
		//null is required for invalid non nullable properties
		Property->CheckValidObject(&Obj->ObjectPtrNonNullable, nullptr);
		AbstractProperty->CheckValidObject(&Obj->ObjectPtrAbstractNonNullable, nullptr);
		CHECK(Obj->ObjectPtrNonNullable == nullptr);
		CHECK(Obj->ObjectPtrAbstractNonNullable == nullptr);
	}
}

class FMockArchive : public FArchive
{
public:
	
	FMockArchive(UObject* Obj)
		: ArchiveValue(Obj)
	{
	}

	union Value
	{
		FObjectPtr ObjectPtrValue;
		UObject* ObjectValue;
		Value(UObject* Obj) : ObjectValue(Obj) {}
	} ArchiveValue;

	virtual FArchive& operator<<(FObjectPtr& Value) override
	{
		Value = ArchiveValue.ObjectPtrValue;
		return *this;
	}

	virtual FArchive& operator<<(UObject*& Value) override
	{
		Value = ArchiveValue.ObjectValue;
		return *this;
	}
};

template<typename T>
static void TestSerializeItem(FName ObjectName)
{
	UClass* Class = T::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(Property != nullptr);

	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};
	T* Obj = NewObject<T>(TestPackage, ObjectName);
	UObjectPtrTestClass* Other = NewObject<UObjectPtrTestClass>(Obj, TEXT("Other"));
	ULinkerPlaceholderExportObject* PlaceHolderExport = NewObject<ULinkerPlaceholderExportObject>(TestPackage, TEXT("PlaceHolderExport"));
	ULinkerPlaceholderClass* PlaceHolderClass = NewObject<ULinkerPlaceholderClass>(TestPackage, TEXT("PlaceHolderClass"));
	PlaceHolderClass->Bind(); //must call bind or crashes on shutdown

	int ResolveCount = 0;
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto CallbackId = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([&ResolveCount](const FObjectRef&, UPackage*, UObject*)
		{
			++ResolveCount;
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackId);
	};
#endif
	{
		//verify that if the property is null no reads are triggered
		FMockArchive MockArchive(PlaceHolderExport);
		FBinaryArchiveFormatter Formatter(MockArchive);
		FStructuredArchive Ar(Formatter);
		FStructuredArchiveSlot Slot = Ar.Open();
		Property->SerializeItem(Slot, &Obj->ObjectPtr, nullptr);
		CHECK(ResolveCount == 0);
		CHECK(*reinterpret_cast<ULinkerPlaceholderExportObject**>(&Obj->ObjectPtr) == PlaceHolderExport);

	}
	{
		//verify that if the property is not null no reads are triggered
		FMockArchive MockArchive(PlaceHolderClass);
		FBinaryArchiveFormatter Formatter(MockArchive);
		FStructuredArchive Ar(Formatter);
		FStructuredArchiveSlot Slot = Ar.Open();
		Obj->ObjectPtr = Other;
		Property->SerializeItem(Slot, &Obj->ObjectPtr, nullptr);
		CHECK(ResolveCount == 0);
		CHECK(*reinterpret_cast<ULinkerPlaceholderClass**>(&Obj->ObjectPtr) == PlaceHolderClass);
	}
}

TEST_CASE("UE::CoreUObject::FObjectProperty::StaticSerializeItem")
{
	TestSerializeItem<UObjectPtrTestClassWithRef>(TEXT("Object1"));
	TestSerializeItem<UObjectWithRawProperty>(TEXT("Object2"));
}

class MockAssetRegistryInterface : public IAssetRegistryInterface
{
public:
	MockAssetRegistryInterface()
		: Old(IAssetRegistryInterface::Default)
	{
		IAssetRegistryInterface::Default = this;
	}

	virtual ~MockAssetRegistryInterface()
	{
		IAssetRegistryInterface::Default = Old;
	}

	virtual void GetDependencies(FName InPackageName, TArray<FName>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) override
	{

	}

	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, FAssetData& OutAssetData) const override
	{

		const FAssetData* Found = AssetData.Find(ObjectPath);
		if (Found)
		{
			OutAssetData = *Found;
			return UE::AssetRegistry::EExists::Exists;
		}
		return UE::AssetRegistry::EExists::DoesNotExist;
	}

	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, FAssetPackageData& OutPackageData, bool bFailIfLockHeld) const override
	{
		return UE::AssetRegistry::EExists::Exists;
	}

	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, FAssetPackageData& OutPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld) const override
	{
		return UE::AssetRegistry::EExists::Exists;
	}

	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const override
	{
		return true;
	}

	virtual bool RegisterOnAssetsAddedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		return false;
	};

	virtual bool UnregisterOnAssetsAddedDelegate(const FDelegateHandle& Handle) override
	{
		return false;
	};

	virtual bool RegisterOnAssetsRemovedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		return false;
	};

	virtual bool UnregisterOnAssetsRemovedDelegate(const FDelegateHandle& Handle) override
	{
		return false;
	};

	IAssetRegistryInterface* Old;
	TMap<FSoftObjectPath, FAssetData> AssetData;
};

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE("UE::CoreUObject::FObjectProperty::ParseObjectPropertyValue")
{
	UClass* Class = UObjectPtrTestClassWithRef::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtr")));

	//make a fake entry for the asset registry
	//this allows lazy load to return an unresolved handle
	MockAssetRegistryInterface MockAssetRegistry;
	const TCHAR* Text = TEXT("/TestPackageName.Other");
	FSoftObjectPath Path(Text);
	FAssetData AssetData;
	AssetData.PackageName = "/TestPackageName";
	AssetData.PackagePath = "/";
	AssetData.AssetName = "Other";
	AssetData.AssetClassPath.TrySetPath(UObjectPtrTestClass::StaticClass());

	MockAssetRegistry.AssetData.Add(Path, AssetData);
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};
	UObjectPtrTestClassWithRef* Obj = NewObject<UObjectPtrTestClassWithRef>(TestPackage, "ObjectName");
	TObjectPtr<UObject> Result;
	FObjectPropertyBase::ParseObjectPropertyValue(Property, Obj, UObjectPtrTestClass::StaticClass(), 0, Text, Result);
	CHECK(Result != nullptr);
	CHECK(!Result.IsResolved());
	{
		const TCHAR * Buffer = TEXT("ObjectPtr=/TestPackageName.Other");
		TArray<FDefinedProperty> DefinedProperties;
		Buffer = FProperty::ImportSingleProperty(Buffer, Obj, Class, Obj, PPF_Delimited, nullptr, DefinedProperties);

		CHECK(Obj->ObjectPtr != nullptr);
		CHECK(!Obj->ObjectPtr.IsResolved());
	}

	{
		const TCHAR* Buffer = TEXT("ObjectPtr=/TestPackageName.Other");

		FProperty* ObjProperty = Class->FindPropertyByName(TEXT("ObjectPtr"));
		ObjProperty->ImportText_InContainer(Buffer, Obj, Obj, 0);
		CHECK(Obj->ObjectPtr == nullptr); //TODO this should not resolve and not be null
		CHECK(Obj->ObjectPtr.IsResolved());

	}

	{
		const TCHAR * Buffer = TEXT("ArrayObjPtr=(/TestPackageName.Other)");
		TArray<FDefinedProperty> DefinedProperties;
		Buffer = FProperty::ImportSingleProperty(Buffer, Obj, Class, Obj, PPF_Delimited, nullptr, DefinedProperties);

		CHECK(Obj->ArrayObjPtr.Num() == 1);
		CHECK(!Obj->ArrayObjPtr[0].IsResolved());
	}

	{
		const TCHAR* Buffer = TEXT("ArrayObjPtr=(/TestPackageName.Other)");
		
		FProperty* ArrayProperty = Class->FindPropertyByName(TEXT("ArrayObjPtr"));

		ArrayProperty->ImportText_InContainer(Buffer, Obj, Obj, 0);

		CHECK(Obj->ArrayObjPtr.Num() == 1);
		CHECK(!Obj->ArrayObjPtr[0].IsResolved());
	}

	{
		const TCHAR * Buffer = TEXT("ObjectPtr=/TestPackageName.Other");
		TArray<FDefinedProperty> DefinedProperties;
		Buffer = FProperty::ImportSingleProperty(Buffer, Obj, Class, Obj, PPF_Delimited, nullptr, DefinedProperties);

		CHECK(Obj->ObjectPtr != nullptr);
		CHECK(!Obj->ObjectPtr.IsResolved());
	}

	{
		const TCHAR* Buffer = TEXT("ObjectPtr=/TestPackageName.Other");

		FProperty* ObjProperty = Class->FindPropertyByName(TEXT("ObjectPtr"));
		ObjProperty->ImportText_InContainer(Buffer, Obj, Obj, 0);
		CHECK(Obj->ObjectPtr == nullptr); //TODO this should not resolve and not be null
		CHECK(Obj->ObjectPtr.IsResolved());

	}

	{
		const TCHAR * Buffer = TEXT("ArrayObjPtr=(/TestPackageName.Other)");
		TArray<FDefinedProperty> DefinedProperties;
		Buffer = FProperty::ImportSingleProperty(Buffer, Obj, Class, Obj, PPF_Delimited, nullptr, DefinedProperties);

		CHECK(Obj->ArrayObjPtr.Num() == 1);
		CHECK(!Obj->ArrayObjPtr[0].IsResolved());
	}

	{
		const TCHAR* Buffer = TEXT("ArrayObjPtr=(/TestPackageName.Other)");
		
		FProperty* ArrayProperty = Class->FindPropertyByName(TEXT("ArrayObjPtr"));

		ArrayProperty->ImportText_InContainer(Buffer, Obj, Obj, 0);

		CHECK(Obj->ArrayObjPtr.Num() == 1);
		CHECK(!Obj->ArrayObjPtr[0].IsResolved());
	}
}
#endif


TEST_CASE("UE::FObjectProperty::Identical::ObjectPtr")
{
	int ResolveCount = 0;
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto CallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([&ResolveCount](const FObjectRef&, UPackage*, UObject*)
		{
			++ResolveCount;
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackHandle);
	};
#endif
	UClass* Class = UObjectPtrTestClassWithRef::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(Property != nullptr);


	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("Test/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	UPackage* TestPackage2 = NewObject<UPackage>(nullptr, TEXT("Test/TestPackageName2"), RF_Transient);
	TestPackage2->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
		TestPackage2->RemoveFromRoot();
	};
	TObjectPtr<UObject> ObjWithRef = NewObject<UObjectPtrTestClassWithRef>(TestPackage, TEXT("UObjectWithClassProperty"));
	TObjectPtr<UObject> Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("UObjectPtrTestClass"));
	TObjectPtr<UObject> Obj2 = NewObject<UObjectPtrTestClass>(TestPackage2, TEXT("UObjectPtrTestClass"));

#if UE_WITH_OBJECT_HANDLE_TRACKING && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

	FObjectHandle Handle1 = MakeUnresolvedHandle(ObjWithRef.Get());
	FObjectHandle Handle2 = MakeUnresolvedHandle(Obj1.Get());
	FObjectHandle Handle3 = MakeUnresolvedHandle(Obj2.Get());
	TObjectPtr<UObjectPtrTestClass> ObjectPtr = *reinterpret_cast<TObjectPtr<UObjectPtrTestClass>*>(&Handle1);

	ObjWithRef = *reinterpret_cast<TObjectPtr<UObject>*>(&Handle1);
	Obj1 = *reinterpret_cast<TObjectPtr<UObject>*>(&Handle2);
	Obj2 = *reinterpret_cast<TObjectPtr<UObject>*>(&Handle3);
#endif

	CHECK(Property->Identical(&Obj1, &Obj1, 0u));
	CHECK(!Property->Identical(&Obj1, nullptr, 0u));
	CHECK(!Property->Identical(&ObjWithRef, &Obj2, 0u));
	CHECK(!Property->Identical(&ObjWithRef, &Obj2, 0u));
	CHECK(!Property->Identical(&Obj1, &ObjWithRef, PPF_DeepComparison));

	CHECK(ResolveCount == 0);
	CHECK(Property->Identical(&Obj1, &Obj2, PPF_DeepComparison));
#if UE_WITH_OBJECT_HANDLE_TRACKING && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(ResolveCount == 2);
#endif
	
}

TEST_CASE("UE::FObjectProperty::Identical::Object")
{
	UClass* Class = UObjectWithRawProperty::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(Property != nullptr);

	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("Test/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	UPackage* TestPackage2 = NewObject<UPackage>(nullptr, TEXT("Test/TestPackageName2"), RF_Transient);
	TestPackage2->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
		TestPackage2->RemoveFromRoot();
	};
	TObjectPtr<UObject> ObjWithRef = NewObject<UObjectWithRawProperty>(TestPackage, TEXT("UObjectWithRawProperty"));
	TObjectPtr<UObject> Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("UObjectPtrTestClass"));
	TObjectPtr<UObject> Obj2 = NewObject<UObjectPtrTestClass>(TestPackage2, TEXT("UObjectPtrTestClass"));

	CHECK(Property->Identical(&Obj1, &Obj1, 0u));
	CHECK(!Property->Identical(&Obj1, nullptr, 0u));
	CHECK(!Property->Identical(&ObjWithRef, &Obj2, 0u));
	CHECK(!Property->Identical(&ObjWithRef, &Obj2, 0u));
	CHECK(!Property->Identical(&Obj1, &ObjWithRef, PPF_DeepComparison));

	CHECK(Property->Identical(&Obj1, &Obj2, PPF_DeepComparison));
}

TEST_CASE("UE::FObjectProperty::CopySingleValue")
{
	FObjectProperty* RawProperty = CastField<FObjectProperty>(UObjectWithRawProperty::StaticClass()->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(RawProperty != nullptr);

	FObjectProperty* PtrProperty = CastField<FObjectProperty>(UObjectPtrTestClassWithRef::StaticClass()->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(PtrProperty != nullptr);

	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("Test/CopySingleValue"), RF_Transient);
	TestPackage->AddToRoot();
	UPackage* TestPackage2 = NewObject<UPackage>(nullptr, TEXT("Test/CopySingleValue2"), RF_Transient);
	TestPackage2->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
		TestPackage2->RemoveFromRoot();
	};
	TObjectPtr<UObject> ObjWithRawRef = NewObject<UObjectWithRawProperty>(TestPackage, TEXT("UObjectWithRawProperty"));
	TObjectPtr<UObject> ObjWithPtrRef = NewObject<UObjectPtrTestClassWithRef>(TestPackage, TEXT("UObjectWithPtrProperty"));
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("UObjectPtrTestClass"));
	UObject* Obj2 = NewObject<UObjectPtrTestClass>(TestPackage2, TEXT("UObjectPtrTestClass2"));
	UObject* RawPtr = Obj1;
	TObjectPtr<UObject> PtrObj2 = Obj2;

#if UE_WITH_OBJECT_HANDLE_TRACKING && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectHandle Handle = MakeUnresolvedHandle(Obj2);
	PtrObj2 = TObjectPtr<UObject>(FObjectPtr(Handle) );
#endif
	//copy an unresolved TObjectPtr to an UObject* pointer. this should resolve the pointer
	RawProperty->CopySingleValue(&RawPtr, &PtrObj2);
	CHECK(RawPtr == Obj2);

	PtrObj2 = nullptr;
	//copy an UObject* to a TObjectPtr
	PtrProperty->CopySingleValue(&PtrObj2, &RawPtr);
	CHECK(PtrObj2 == RawPtr);

	CHECK(RawProperty->GetClass() == PtrProperty->GetClass());
}


TEST_CASE("UE::FObjectProperty::GetCPPType")
{
	FObjectProperty* RawProperty = CastField<FObjectProperty>(UObjectWithRawProperty::StaticClass()->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(RawProperty != nullptr);

	FObjectProperty* PtrProperty = CastField<FObjectProperty>(UObjectPtrTestClassWithRef::StaticClass()->FindPropertyByName(TEXT("ObjectPtr")));
	REQUIRE(PtrProperty != nullptr);

	FString RawType = RawProperty->GetCPPType(nullptr, 0u);
	CHECK(RawType == TEXT("UObjectPtrTestClass*"));

	RawType = RawProperty->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr);
	CHECK(RawType == TEXT("UObjectPtrTestClass*"));

	FString PtrType = PtrProperty->GetCPPType(nullptr, 0u);
	CHECK(PtrType == TEXT("TObjectPtr<UObjectPtrTestClass>"));

	PtrType = PtrProperty->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr);
	CHECK(PtrType == TEXT("UObjectPtrTestClass*")); 
}

TEST_CASE("UE::FObjectProperty::ArrayProperty")
{

	FArrayProperty* PtrProperty = CastField<FArrayProperty>(UObjectPtrTestClassWithRef::StaticClass()->FindPropertyByName(TEXT("ArrayObjPtr")));
	REQUIRE(PtrProperty != nullptr);
}
#endif
