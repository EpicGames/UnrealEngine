// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectPtr.h"

#include "ObjectRefTrackingTestBase.h"
#include "ObjectPtrTestClass.h"
#include "Concepts/EqualityComparable.h"
#include "Serialization/ArchiveCountMem.h"
#include "Templates/Models.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/Interface.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectPathId.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "Misc/ScopeExit.h"
#include "Tests/Benchmark.h"
#include <type_traits>

namespace UE::CoreObject::Private::Tests
{
using FMutableObjectPtr = TObjectPtr<UObject>;
using FMutableInterfacePtr = TObjectPtr<UInterface>;
using FMutablePackagePtr = TObjectPtr<UPackage>;
using FConstObjectPtr = TObjectPtr<const UObject>;
using FConstInterfacePtr = TObjectPtr<const UInterface>;
using FConstPackagePtr = TObjectPtr<const UPackage>;

class UForwardDeclaredObjDerived;
class FForwardDeclaredNotObjDerived;

static_assert(sizeof(FObjectPtr) == sizeof(FObjectHandle), "FObjectPtr type must always compile to something equivalent to an FObjectHandle size.");
static_assert(sizeof(FObjectPtr) == sizeof(void*), "FObjectPtr type must always compile to something equivalent to a pointer size.");
static_assert(sizeof(TObjectPtr<UObject>) == sizeof(void*), "TObjectPtr<UObject> type must always compile to something equivalent to a pointer size.");

// Ensure that a TObjectPtr is trivially copyable, (copy/move) constructible, (copy/move) assignable, and destructible
#if !UE_OBJECT_PTR_GC_BARRIER
static_assert(std::is_trivially_copyable<FMutableObjectPtr>::value, "TObjectPtr must be trivially copyable");
static_assert(std::is_trivially_copy_constructible<FMutableObjectPtr>::value, "TObjectPtr must be trivially copy constructible");
static_assert(std::is_trivially_move_constructible<FMutableObjectPtr>::value, "TObjectPtr must be trivially move constructible");
static_assert(std::is_trivially_copy_assignable<FMutableObjectPtr>::value, "TObjectPtr must be trivially copy assignable");
static_assert(std::is_trivially_move_assignable<FMutableObjectPtr>::value, "TObjectPtr must be trivially move assignable");
#endif // !UE_OBJECT_PTR_GC_BARRIER
static_assert(std::is_trivially_destructible<FMutableObjectPtr>::value, "TObjectPtr must be trivially destructible");

// Ensure that raw pointers can be used to construct wrapped object pointers and that const-ness isn't stripped when constructing or converting with raw pointers
static_assert(std::is_constructible<FMutableObjectPtr, UObject*>::value, "TObjectPtr<UObject> must be constructible from a raw UObject*");
static_assert(!std::is_constructible<FMutableObjectPtr, const UObject*>::value, "TObjectPtr<UObject> must not be constructible from a const raw UObject*");
static_assert(std::is_convertible<FMutableObjectPtr, UObject*>::value, "TObjectPtr<UObject> must be convertible to a raw UObject*");
static_assert(std::is_convertible<FMutableObjectPtr, const UObject*>::value, "TObjectPtr<UObject> must be convertible to a const raw UObject*");

static_assert(std::is_constructible<FConstObjectPtr, UObject*>::value, "TObjectPtr<const UObject> must be constructible from a raw UObject*");
static_assert(std::is_constructible<FConstObjectPtr, const UObject*>::value, "TObjectPtr<const UObject> must be constructible from a const raw UObject*");
static_assert(!std::is_convertible<FConstObjectPtr, UObject*>::value, "TObjectPtr<const UObject> must not be convertible to a raw UObject*");
static_assert(std::is_convertible<FConstObjectPtr, const UObject*>::value, "TObjectPtr<const UObject> must be convertible to a const raw UObject*");

// Ensure that a TObjectPtr<const UObject> is constructible and assignable from a TObjectPtr<UObject> but not vice versa
static_assert(std::is_constructible<FConstObjectPtr, const FMutableObjectPtr&>::value, "Missing constructor (TObjectPtr<const UObject> from TObjectPtr<UObject>)");
static_assert(!std::is_constructible<FMutableObjectPtr, const FConstObjectPtr&>::value, "Invalid constructor (TObjectPtr<UObject> from TObjectPtr<const UObject>)");
static_assert(std::is_assignable<FConstObjectPtr, const FMutableObjectPtr&>::value, "Missing assignment (TObjectPtr<const UObject> from TObjectPtr<UObject>)");
static_assert(!std::is_assignable<FMutableObjectPtr, const FConstObjectPtr&>::value, "Invalid assignment (TObjectPtr<UObject> from TObjectPtr<const UObject>)");

static_assert(std::is_constructible<FConstObjectPtr, const FConstObjectPtr&>::value, "Missing constructor (TObjectPtr<const UObject> from TObjectPtr<const UObject>)");
static_assert(std::is_assignable<FConstObjectPtr, const FConstObjectPtr&>::value, "Missing assignment (TObjectPtr<const UObject> from TObjectPtr<const UObject>)");

// Ensure that a TObjectPtr<UObject> is constructible and assignable from a TObjectPtr<UInterface> but not vice versa
static_assert(std::is_constructible<FMutableObjectPtr, const FMutableInterfacePtr&>::value, "Missing constructor (TObjectPtr<UObject> from TObjectPtr<UInterface>)");
static_assert(!std::is_constructible<FMutableInterfacePtr, const FMutableObjectPtr&>::value, "Invalid constructor (TObjectPtr<UInterface> from TObjectPtr<UObject>)");
static_assert(std::is_constructible<FConstObjectPtr, const FConstInterfacePtr&>::value, "Missing constructor (TObjectPtr<const UObject> from TObjectPtr<const UInterface>)");
static_assert(std::is_constructible<FConstObjectPtr, const FMutableInterfacePtr&>::value, "Missing constructor (TObjectPtr<const UObject> from TObjectPtr<UInterface>)");
static_assert(!std::is_constructible<FConstInterfacePtr, const FConstObjectPtr&>::value, "Invalid constructor (TObjectPtr<const UInterface> from TObjectPtr<const UObject>)");
static_assert(!std::is_constructible<FConstInterfacePtr, const FMutableObjectPtr&>::value, "Invalid constructor (TObjectPtr<const UInterface> from TObjectPtr<UObject>)");

static_assert(std::is_assignable<FMutableObjectPtr, const FMutableInterfacePtr&>::value, "Missing assignment (TObjectPtr<UObject> from TObjectPtr<UInterface>)");
static_assert(std::is_assignable<FConstObjectPtr, const FMutableInterfacePtr&>::value, "Missing assignment (TObjectPtr<const UObject> from TObjectPtr<UInterface>)");
static_assert(std::is_assignable<FConstObjectPtr, const FConstInterfacePtr&>::value, "Missing assignment (TObjectPtr<const UObject> from TObjectPtr<const UInterface>)");
static_assert(!std::is_assignable<FMutableInterfacePtr, const FMutableObjectPtr&>::value, "Invalid assignment (TObjectPtr<UInterface> from TObjectPtr<UObject>)");
static_assert(!std::is_assignable<FConstInterfacePtr, const FMutableObjectPtr&>::value, "Invalid assignment (TObjectPtr<const UInterface> from TObjectPtr<UObject>)");
static_assert(!std::is_assignable<FConstInterfacePtr, const FConstObjectPtr&>::value, "Invalid assignment (TObjectPtr<const UInterface> from TObjectPtr<const UObject>)");

// Ensure that TObjectPtr<[const] UObject> is comparable with another TObjectPtr<[const] UObject> regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, FConstObjectPtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and TObjectPtr<const UObject>");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, FConstObjectPtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and TObjectPtr<const UObject>");

// Ensure that TObjectPtr<[const] UObject> is comparable with another TObjectPtr<[const] UInterface> regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, FConstInterfacePtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and TObjectPtr<const UInterface>");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, FConstInterfacePtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and TObjectPtr<const UInterface>");
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, FMutableInterfacePtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and TObjectPtr<UInterface>");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, FMutableInterfacePtr>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and TObjectPtr<UInterface>");

// Ensure that TObjectPtr<[const] UPackage> is not comparable with a TObjectPtr<[const] UInterface> regardless of constness
// TODO: This only ensures that at least one of the A==B,B==A,A!=B,B!=A operations fail, not that they all fail.
#if !(PLATFORM_MICROSOFT) || !defined(_MSC_EXTENSIONS) // MSVC static analyzer is run in non-conformance mode, and that causes these checks to fail.
static_assert(!TModels_V<CEqualityComparableWith, FConstPackagePtr, FConstInterfacePtr>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UPackage> and TObjectPtr<const UInterface>");
static_assert(!TModels_V<CEqualityComparableWith, FMutablePackagePtr, FConstInterfacePtr>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UPackage> and TObjectPtr<const UInterface>");
static_assert(!TModels_V<CEqualityComparableWith, FConstPackagePtr, FMutableInterfacePtr>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UPackage> and TObjectPtr<UInterface>");
static_assert(!TModels_V<CEqualityComparableWith, FMutablePackagePtr, FMutableInterfacePtr>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UPackage> and TObjectPtr<UInterface>");
#endif // #if !(PLATFORM_MICROSOFT) || !defined(_MSC_EXTENSIONS)

// Ensure that TObjectPtr<[const] UObject> is comparable with a raw pointer of the same referenced type regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, const UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and const UObject*");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, const UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and const UObject*");
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and UObject*");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and UObject*");

// Ensure that TObjectPtr<[const] UObject> is comparable with a UInterface raw pointer regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, const UInterface*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and const UInterface*");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, const UInterface*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and const UInterface*");
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, UInterface*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and UInterface*");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, UInterface*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and UInterface*");

// Ensure that TObjectPtr<[const] UInterface> is comparable with a UObject raw pointer regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstInterfacePtr, const UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UInterface> and const UObject*");
static_assert(TModels_V<CEqualityComparableWith, FMutableInterfacePtr, const UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UInterface> and const UObject*");
static_assert(TModels_V<CEqualityComparableWith, FConstInterfacePtr, UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UInterface> and UObject*");
static_assert(TModels_V<CEqualityComparableWith, FMutableInterfacePtr, UObject*>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UInterface> and UObject*");

// Ensure that TObjectPtr<[const] UInterface> is not comparable with a UPackage raw pointer regardless of constness
// TODO: This only ensures that at least one of the A==B,B==A,A!=B,B!=A operations fail, not that they all fail.
static_assert(!TModels_V<CEqualityComparableWith, FConstInterfacePtr, const UPackage*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UInterface> and const UPackage*");
static_assert(!TModels_V<CEqualityComparableWith, FMutableInterfacePtr, const UPackage*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UInterface> and const UPackage*");
static_assert(!TModels_V<CEqualityComparableWith, FConstInterfacePtr, UPackage*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UInterface> and UPackage*");
static_assert(!TModels_V<CEqualityComparableWith, FMutableInterfacePtr, UPackage*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UInterface> and UPackage*");

// Ensure that TObjectPtr<[const] UInterface> is not comparable with a char raw pointer regardless of constness
// TODO: This only ensures that at least one of the A==B,B==A,A!=B,B!=A operations fail, not that they all fail.
static_assert(!TModels_V<CEqualityComparableWith, FConstObjectPtr, const char*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and const UObject*");
static_assert(!TModels_V<CEqualityComparableWith, FMutableObjectPtr, const char*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and const UObject*");
static_assert(!TModels_V<CEqualityComparableWith, FConstObjectPtr, char*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and UObject*");
static_assert(!TModels_V<CEqualityComparableWith, FMutableObjectPtr, char*>, "Must not be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and UObject*");

// Ensure that TObjectPtr<[const] UObject> is comparable with nullptr regardless of constness
static_assert(TModels_V<CEqualityComparableWith, FConstObjectPtr, TYPE_OF_NULLPTR>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and nullptr");
static_assert(TModels_V<CEqualityComparableWith, FMutableObjectPtr, TYPE_OF_NULLPTR>, "Must be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and nullptr");

// Ensure allowing comparison against NULL doesn't mean allowing comparison against int
static_assert(!TModels_V<CEqualityComparableWith, FConstObjectPtr, int>, "Should not be able to compare equality and inequality bidirectionally between TObjectPtr<const UObject> and int");
static_assert(!TModels_V<CEqualityComparableWith, FMutableObjectPtr, int>, "Should not be able to compare equality and inequality bidirectionally between TObjectPtr<UObject> and int");


#if WITH_LOW_LEVEL_TESTS

// Ensure that the use of incomplete types doesn't provide a means to bypass type safety on TObjectPtr
// NOTE: This is disabled because we're permitting this operation with a deprecation warning.
//static_assert(!std::is_assignable<TObjectPtr<UForwardDeclaredObjDerived>, UForwardDeclaredObjDerived*>::value, "Should not be able to assign raw pointer of incomplete type that descends from UObject to exactly this type of TObjectPtr");
//static_assert(!std::is_assignable<TObjectPtr<FForwardDeclaredNotObjDerived>, FForwardDeclaredNotObjDerived*>::value, "Should not be able to assign raw pointer of incomplete type that does not descend from UObject to exactly this type of TObjectPtr");

class FObjectPtrTestBase : public FObjectRefTrackingTestBase
{
public:

protected:
};

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE("CoreUObject::TObjectPtr::FindLoadBehavior")
{
	CHECK(UE::LinkerLoad::FindLoadBehavior(*UObjectPtrTestClass::StaticClass()) == UE::LinkerLoad::EImportBehavior::LazyOnDemand);
	CHECK(::UE::LinkerLoad::FindLoadBehavior(*UObjectPtrDerrivedTestClass::StaticClass()) == ::UE::LinkerLoad::EImportBehavior::LazyOnDemand);
	CHECK(::UE::LinkerLoad::FindLoadBehavior(*UObjectPtrNotLazyTestClass::StaticClass()) == ::UE::LinkerLoad::EImportBehavior::Eager);
}
#endif

TEST_CASE("CoreUObject::TObjectPtr::Null", "[CoreUObject][ObjectPtr]")
{
	TObjectPtr<UObject> NullObjectPtr(nullptr);
	TEST_TRUE(TEXT("NULL should equal a null object pointer"), NULL == NullObjectPtr);
	TEST_TRUE(TEXT("A null object pointer should equal NULL"), NullObjectPtr == NULL);
	TEST_TRUE(TEXT("Nullptr should equal a null object pointer"), nullptr == NullObjectPtr);
	TEST_TRUE(TEXT("A null object pointer should equal nullptr"), NullObjectPtr == nullptr);
	TEST_FALSE(TEXT("A null object pointer should evaluate to false"), !!NullObjectPtr);
	TEST_TRUE(TEXT("Negation of a null object pointer should evaluate to true"), !NullObjectPtr);
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Default Serialize", "[CoreUObject][ObjectPtr]")
{
	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);

	const FName TestPackageName("/Engine/Test/ObjectPtrDefaultSerialize/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* TestSoftObject = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectPtr DefaultSerializeObjectPtr(MakeUnresolvedHandle(TestSoftObject));
#else
	FObjectPtr DefaultSerializeObjectPtr(TestSoftObject);
#endif

	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after initializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after initializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should not change when initializing an FObjectPtr"),
																TestPackage,
																0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should not change when initializing an FObjectPtr"),
																TestSoftObject,
																0);

	FArchiveUObject Writer;
	Writer << DefaultSerializeObjectPtr;

	ObjectRefMetrics.TestNumResolves(TEXT("Serializing an FObjectPtr should force it to resolve"), UE_WITH_OBJECT_HANDLE_LATE_RESOLVE ? 1 : 0);
	ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after serializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should increase after serializing an FObjectPtr"),
																TestSoftObject,
																1);

	Writer << DefaultSerializeObjectPtr;

	ObjectRefMetrics.TestNumResolves(TEXT("Serializing an FObjectPtr twice should only require it to resolve once"), UE_WITH_OBJECT_HANDLE_LATE_RESOLVE ? 1 : 0);
	ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after serializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should increase after serializing an FObjectPtr"),
																TestSoftObject,
																2);

	TestPackage->RemoveFromRoot();
}
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING

TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Soft Object Path", "[CoreUObject][ObjectPtr]")
{
	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);

	const FName TestPackageName("/Engine/Test/ObjectPtrSoftObjectPath/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* TestSoftObject = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("TestSoftObject"));

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectPtr DefaultSoftObjPtr(MakeUnresolvedHandle(TestSoftObject));
#else
	FObjectPtr DefaultSoftObjPtr(TestSoftObject);
#endif
	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after initializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after initializing an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should not change when initializing an FObjectPtr"),
																TestSoftObject,
																0);

	// Initializing a soft object path from a TObjectPtr that's unresolved should stay unresolved.
	FSoftObjectPath DefaultSoftObjPath(DefaultSoftObjPtr);
	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after initializing an FSoftObjectPath from an FObjectPtr"), 0);
	ObjectRefMetrics.TestNumReads(TEXT("NumReads should have changed when initializing a FSoftObjectPath"),
																TestSoftObject,
																UE_WITH_OBJECT_HANDLE_LATE_RESOLVE ? 0 : 1);

	TEST_EQUAL_STR(TEXT("Soft object path constructed from an FObjectPtr does not have the expected path value"), TEXT("/Engine/Test/ObjectPtrSoftObjectPath/Transient.TestSoftObject"), *DefaultSoftObjPath.ToString());

	TestPackage->RemoveFromRoot();
}
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING

TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Forward Declared", "[CoreUObject][ObjectPtr]")
{
	UForwardDeclaredObjDerived* PtrFwd = nullptr;
	TObjectPtr<UForwardDeclaredObjDerived> ObjPtrFwd(MakeObjectPtrUnsafe<UForwardDeclaredObjDerived>(reinterpret_cast<UObject*>(PtrFwd)));
	TEST_TRUE(TEXT("Null forward declared pointer used to construct a TObjectPtr should result in a null TObjectPtr"), !ObjPtrFwd);
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Hash Consistency", "[CoreUObject][ObjectPtr]")
{
	const FName TestPackage1Name("/Engine/Test/ObjectPtrHashConsistency1/Transient");
	UPackage* TestPackage1 = NewObject<UPackage>(nullptr, TestPackage1Name, RF_Transient);
	TestPackage1->AddToRoot();
	UObject* TestOuter1 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter1"));
	UObject* TestOuter2 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter2"));
	UObject* TestOuter3 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter3"));
	UObject* TestOuter4 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter4"));

	UObject* TestPublicObject = NewObject<UObjectPtrTestClass>(TestOuter1, TEXT("TestPublicObject"), RF_Public);

	UE::CoreUObject::Private::MakePackedObjectRef(TestPublicObject); //construct a packed object ref the exported object. replicates linker load
	// Perform hash consistency checks on public object reference
	{
		// Check that unresolved/resolved pointers produce the same hash
		FSnapshotObjectRefMetrics ObjectRefMetrics(*this);

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		FObjectPtr TestPublicWrappedObjectPtr(MakeUnresolvedHandle(TestPublicObject));
#else
		FObjectPtr TestPublicWrappedObjectPtr(TestPublicObject);
#endif
		ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after initializing an FObjectPtr"), 0);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after initializing an FObjectPtr"), 0);

		uint32 HashWrapped = GetTypeHash(TestPublicWrappedObjectPtr);
		ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after hashing an FObjectPtr"), 0);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after hashing an FObjectPtr"), 0);

		(void)TestPublicWrappedObjectPtr.Get();

		ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after resolving an FObjectPtr"), 1);
		ObjectRefMetrics.TestNumFailedResolves(TEXT("Unexpected resolve failure after resolving an FObjectPtr"), 0);

		TObjectPtr<UObject> ObjectPtr = TestPublicObject;
		uint32 ObjectPtrHash = GetTypeHash(ObjectPtr);
		TEST_EQUAL(TEXT("Hash of FObjectPtr should equal hash of TObjectPtr"), ObjectPtrHash, HashWrapped);

		//Check that renaming an object doesn't change its hash
		TestPublicObject->Rename(TEXT("TestPublicObjectRenamed"));
		uint32 HashWrappedAfterRename = GetTypeHash(TestPublicWrappedObjectPtr);
		TEST_EQUAL(TEXT("Hash of resolved public FObjectPtr before rename should equal hash of resolved public FObjectPtr after rename"), HashWrappedAfterRename, HashWrapped);

		//Check that reparenting an object doesn't change its hash
		TestPublicObject->Rename(nullptr, TestOuter2);
		uint32 HashWrappedAfterReparent = GetTypeHash(TestPublicWrappedObjectPtr);
		TEST_EQUAL(TEXT("Hash of resolved public FObjectPtr before reparenting should equal hash of resolved public FObjectPtr after reparenting"), HashWrappedAfterReparent, HashWrapped);
	}

	TestPackage1->RemoveFromRoot();
}
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Long Path", "[CoreUObject][ObjectPtr]")
{
	const FName TestPackage1Name("/Engine/Test/FObjectPtrTestLongPath/Transient");
	UPackage* TestPackage1 = NewObject<UPackage>(nullptr, TestPackage1Name, RF_Transient);
	TestPackage1->AddToRoot();

	ON_SCOPE_EXIT{
		TestPackage1->RemoveFromRoot();
	};

	UObject* TestObject1 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestObject1"));
	UObject* TestObject2 = NewObject<UObjectPtrTestClass>(TestObject1, TEXT("TestObject2"));
	UObject* TestObject3 = NewObject<UObjectPtrTestClass>(TestObject2, TEXT("TestObject3"));
	UObject* TestObject4 = NewObject<UObjectPtrTestClass>(TestObject3, TEXT("TestObject4"));

	UE::CoreUObject::Private::FObjectPathId LongPath(TestObject4);
	UE::CoreUObject::Private::FObjectPathId::ResolvedNameContainerType ResolvedNames;
	LongPath.Resolve(ResolvedNames);

	TEST_EQUAL(TEXT("Resolved path from FObjectPathId should have 4 elements"), ResolvedNames.Num(), 4);
	TEST_EQUAL(TEXT("Resolved path from FObjectPathId should have TestObject1 at element 0"), ResolvedNames[0], TestObject1->GetFName());
	TEST_EQUAL(TEXT("Resolved path from FObjectPathId should have TestObject2 at element 1"), ResolvedNames[1], TestObject2->GetFName());
	TEST_EQUAL(TEXT("Resolved path from FObjectPathId should have TestObject3 at element 2"), ResolvedNames[2], TestObject3->GetFName());
	TEST_EQUAL(TEXT("Resolved path from FObjectPathId should have TestObject4 at element 3"), ResolvedNames[3], TestObject4->GetFName());
}
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Type Specialization", "[CoreUObject][ObjectPtr]")
{
	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);

	const FName TestPackageName("/Engine/Test/ObjectPtrTypeSafety/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};

	UObject* TestAsBaseUObject = NewObject<UObjectPtrTestClass>(TestPackage);
	UObjectPtrTestClass* TestAsDerivedUObject = NewObject<UObjectPtrTestClass>(TestPackage);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectPtr UntypedBaseUObjectPtr(MakeUnresolvedHandle(TestAsBaseUObject));
	FObjectPtr UntypedDerivedUObjectPtr(MakeUnresolvedHandle(TestAsDerivedUObject));
#else
	FObjectPtr UntypedBaseUObjectPtr(TestAsBaseUObject);
	FObjectPtr UntypedDerivedUObjectPtr(TestAsDerivedUObject);
#endif

	TObjectPtr<UObject> TestUObjectTypePtr(UntypedBaseUObjectPtr);
	TObjectPtr<UObjectPtrTestClass> TestUObjectSubTypePtr(UntypedDerivedUObjectPtr);

	// Evaluating the UObject base pointer type multiple times should not resolve the underlying handle.
	TEST_TRUE(TEXT("UObject base pointer types should evaluate to true"), !!TestUObjectTypePtr);
	TEST_TRUE(TEXT("UObject base pointer types should not equate to NULL"), TestUObjectTypePtr != nullptr);

	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after UObject base pointer evaluation tests"), 0);

	// Evaluating the UObject subtype pointer multiple times should also not resolve the underlying handle.
	TEST_TRUE(TEXT("UObject subtype pointers should evaluate to true"), !!TestUObjectSubTypePtr);
	TEST_TRUE(TEXT("UObject subtype pointers should not equate to NULL"), TestUObjectSubTypePtr != nullptr);

	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after UObject subtype pointer evaluation tests"), 0);

	// Dereferencing the UObject base pointer type multiple times should only increment the test's overall resolve count one time (if late resolve is enabled).
	const UObject* ResolvedUObject = TestUObjectTypePtr;
	TEST_TRUE(TEXT("UObject base pointer types should resolve to the original object"), ResolvedUObject == TestAsBaseUObject);
	TEST_TRUE(TEXT("UObject base pointer types should resolve to a non-NULL reference"), Cast<UObject>(TestUObjectTypePtr) != nullptr);

	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after dereferencing a UObject base pointer multiple times"), UE_WITH_OBJECT_HANDLE_LATE_RESOLVE ? 1 : 0);

	// Dereferencing the UObject subtype pointer multiple times should only increment the test's overall resolve count one additional time (if late resolve is enabled).
	const UObjectPtrTestClass* ResolvedUObjectSubType = TestUObjectSubTypePtr;
	TEST_TRUE(TEXT("UObject subtype pointers should resolve to the original object"), ResolvedUObjectSubType == TestAsDerivedUObject);
	TEST_TRUE(TEXT("UObject subtype pointers should resolve to a non-NULL reference"), Cast<UObjectPtrTestClass>(TestUObjectSubTypePtr) != nullptr);

	ObjectRefMetrics.TestNumResolves(TEXT("Unexpected resolve count after dereferencing a UObject subtype pointer multiple times"), UE_WITH_OBJECT_HANDLE_LATE_RESOLVE ? 2 : 0);
}

void ObjectPtrStressTest(int32 NumTestNodes, bool bIsContiguousTest, bool bIsEvalOnlyTest)
{
	const FName TestPackageName("/Engine/Test/ObjectPtr/EvalStressTest/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	};

	int32 NumObjectsAllocated = 0;
	const int32 MaxAllowedAllocations = GUObjectArray.GetObjectArrayEstimatedAvailable();
	TArray<TObjectPtr<UObjectPtrStressTestClass>*> TestNodePtrs;
	TArray<TObjectPtr<UObjectPtrStressTestClass>> TestNodeStore;

	if (bIsContiguousTest)
	{
		TestNodeStore.Reserve(NumTestNodes);
		for (int32 NodeIdx = 0; NodeIdx < NumTestNodes; ++NodeIdx)
		{
			TestNodePtrs.Add(&TestNodeStore.AddDefaulted_GetRef());
		}
	}
	else
	{
		for (int32 NodeIdx = 0; NodeIdx < NumTestNodes; ++NodeIdx)
		{
			TestNodePtrs.Add(new TObjectPtr<UObjectPtrStressTestClass>());
		}
	}

	for (TObjectPtr<UObjectPtrStressTestClass>* TestNodePtr : TestNodePtrs)
	{
		// randomize the allocation of a test object
		UObjectPtrStressTestClass* TestObject = nullptr;
		if (NumObjectsAllocated < MaxAllowedAllocations && (FMath::Rand() % 2))
		{
			TestObject = NewObject<UObjectPtrStressTestClass>(TestPackage, NAME_None, RF_Transient);
			++NumObjectsAllocated;
		}

		*TestNodePtr = TestObject;
	}

	for (TObjectPtr<UObjectPtrStressTestClass>* TestNodePtr : TestNodePtrs)
	{
		// fill out the remaining unallocated slots if we can
		TObjectPtr<UObjectPtrStressTestClass>& TestNode = *TestNodePtr;
		if (!TestNode && NumObjectsAllocated < MaxAllowedAllocations)
		{
			TestNode = NewObject<UObjectPtrStressTestClass>(TestPackage, NAME_None, RF_Transient);
			++NumObjectsAllocated;
		}

		if (!bIsEvalOnlyTest)
		{
			// access the pointer (i.e. resolve to a raw UObject ptr) and do something with it
			UObjectPtrStressTestClass* ResolvedObject = TestNode.Get();
			if (ResolvedObject)
			{
				FMemory::Memzero(ResolvedObject->Data, PLATFORM_CACHE_LINE_SIZE);
			}
		}

		if (!bIsContiguousTest)
		{
			if (TestNode)
			{
				TestNode->MarkAsGarbage();
			}

			delete TestNodePtr;
		}
	}
}

// Enable this method to add stress test benchmarks or to do A/B comparisons with features turned on/off.
DISABLED_TEST_CASE_METHOD(FObjectPtrTestBase, "CoreUObject::TObjectPtr::Stress Tests", "[CoreUObject][ObjectPtr]")
{
	constexpr bool CONTIGUOUS = true;
	constexpr bool NON_CONTIGUOUS = false;
	constexpr bool EVAL_ONLY = true;
	constexpr bool EVAL_AND_RESOLVE = false;

	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);

	// contiguous object pointer blocks, eval-only
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(1000, CONTIGUOUS, EVAL_ONLY); } );
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(10000, CONTIGUOUS, EVAL_ONLY); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(100000, CONTIGUOUS, EVAL_ONLY); });

	// non-contiguous list of object pointers, eval-only
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(1000, NON_CONTIGUOUS, EVAL_ONLY); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(10000, NON_CONTIGUOUS, EVAL_ONLY); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(100000, NON_CONTIGUOUS, EVAL_ONLY); });

	// make sure that nothing was resolved above
	ObjectRefMetrics.TestNumResolves(TEXT("Eval-only stress tests should not have triggered a resolve attempt"), 0);

	// contiguous object pointer blocks, with resolve attempts
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(1000, CONTIGUOUS, EVAL_AND_RESOLVE); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(10000, CONTIGUOUS, EVAL_AND_RESOLVE); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(100000, CONTIGUOUS, EVAL_AND_RESOLVE); });

	// non-contiguous list of object pointers, with resolve attempts
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(1000, NON_CONTIGUOUS, EVAL_AND_RESOLVE); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(10000, NON_CONTIGUOUS, EVAL_AND_RESOLVE); });
	UE_BENCHMARK(5, [] { ObjectPtrStressTest(100000, NON_CONTIGUOUS, EVAL_AND_RESOLVE); });
}

TEST_CASE("CoreUObject::TObjectPtr::GetPathName", "[CoreUObject][ObjectPtr]")
{
	const FName TestPackage1Name("/Engine/Test/FObjectPtrTestLongPath/Transient");
	UPackage* TestPackage1 = NewObject<UPackage>(nullptr, TestPackage1Name, RF_Transient);
	TestPackage1->AddToRoot();
	UObject* TestObject1 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestObject1"));

#if UE_WITH_OBJECT_HANDLE_TRACKING
	int ResolveCount = 0;
	auto ResolveDelegate = [&ResolveCount](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++ResolveCount;
		};
	auto Handle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback(ResolveDelegate);
#endif

	ON_SCOPE_EXIT
	{
		TestPackage1->RemoveFromRoot();
#if UE_WITH_OBJECT_HANDLE_TRACKING
	UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(Handle);
#endif
	};

	TObjectPtr<UObject> Ptr = nullptr;
	CHECK(TEXT("None") == Ptr.GetPathName());

	Ptr = TestObject1;
	CHECK(TestObject1->GetPathName() == Ptr.GetPathName());

	Ptr = TestPackage1;
	CHECK(TestPackage1->GetPathName() == Ptr.GetPathName());

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	{
		FObjectPtr ObjPtr(MakeUnresolvedHandle(TestObject1));
		Ptr = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr);
		REQUIRE(!Ptr.IsResolved());
		CHECK(TestObject1->GetPathName() == Ptr.GetPathName());
		CHECK(TestObject1->GetClass() == Ptr.GetClass());
	}
	{
		FObjectPtr ObjPtr(MakeUnresolvedHandle(TestPackage1));
		Ptr = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr);
		REQUIRE(!Ptr.IsResolved());
		CHECK(TestPackage1->GetPathName() == Ptr.GetPathName());
		CHECK(TestPackage1->GetClass() == Ptr.GetClass());
	}
#endif 

#if UE_WITH_OBJECT_HANDLE_TRACKING
	REQUIRE(ResolveCount == 0);
#endif
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
TEST_CASE("CoreUObject::TObjectPtr::PackageRename")
{
	const FName TestPackageName("/Engine/Test/TestName/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();

	//register package with the object handle registry
	UE::CoreUObject::Private::MakePackedObjectRef(TestPackage);

	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));

	UObject* Class = Obj1->GetClass();

	Class->GetPackage()->GetMetaData().SetValue(Class, TEXT("LoadBehavior"), TEXT("LazyOnDemand"));

	FObjectPtr ObjPtr(MakeUnresolvedHandle(Obj1));
	FObjectPtr InnerPtr(MakeUnresolvedHandle(Inner1));

	TObjectPtr<UObject> BeforeRename = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr);
	TObjectPtr<UObject> BeforeRenameInner = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr);

	REQUIRE(!BeforeRename.IsResolved()); 
	CHECK(Obj1->GetPathName() == BeforeRename.GetPathName());
	REQUIRE(!BeforeRenameInner.IsResolved());
	CHECK(Inner1->GetPathName() == BeforeRenameInner.GetPathName());
	
	TestPackage->Rename(TEXT("/Engine/Test/TestName/NewName"));
	CHECK(Obj1->GetPathName() == BeforeRename.GetPathName());
	CHECK(Inner1->GetPathName() == BeforeRenameInner.GetPathName());

	TObjectPtr<UObject> AfterRenameResolved = Obj1;
	TObjectPtr<UObject> AfterRenameInnerResolved = Inner1;
	CHECK(BeforeRename == AfterRenameResolved);
	CHECK(BeforeRenameInner == AfterRenameInnerResolved);

	FObjectPtr ObjPtr2(MakeUnresolvedHandle(Obj1));
	FObjectPtr InnerPtr2(MakeUnresolvedHandle(Inner1));
	TObjectPtr<UObject> AfterRenameUnresolved = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr2);
	TObjectPtr<UObject> AfterRenameInnerUnresolved = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr2);

	CHECK(BeforeRename == AfterRenameUnresolved);
	CHECK(BeforeRenameInner == AfterRenameInnerUnresolved);

	Obj1->Rename(TEXT("RenamedObj"), nullptr);
	CHECK(Obj1->GetPathName() == BeforeRename.GetPathName());
	CHECK(Inner1->GetPathName() == BeforeRenameInner.GetPathName());
	CHECK(BeforeRename == AfterRenameResolved);
	CHECK(BeforeRenameInner == AfterRenameInnerResolved);
}

TEST_CASE("CoreUObject::TObjectPtr::InnerRename")
{
	const FName TestPackageName("/Engine/TestPackage");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));

	UObject* Obj2 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj2"));
	UObject* Inner2 = NewObject<UObjectPtrTestClass>(Obj2, TEXT("Inner2"));

	FObjectPtr ObjPtr1(MakeUnresolvedHandle(Obj1));
	FObjectPtr InnerPtr1(MakeUnresolvedHandle(Inner1));

	FObjectPtr ObjPtr2(MakeUnresolvedHandle(Obj2));
	FObjectPtr InnerPtr2(MakeUnresolvedHandle(Inner2));

	TObjectPtr<UObject> BeforeRename1 = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr1);
	TObjectPtr<UObject> BeforeRenameInner1 = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr1);

	TObjectPtr<UObject> BeforeRename2 = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr2);
	TObjectPtr<UObject> BeforeRenameInner2 = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr2);

	REQUIRE(!BeforeRename1.IsResolved());
	CHECK(Obj1->GetPathName() == BeforeRename1.GetPathName());
	REQUIRE(!BeforeRenameInner1.IsResolved());
	CHECK(Inner1->GetPathName() == BeforeRenameInner1.GetPathName());

	REQUIRE(!BeforeRename2.IsResolved());
	CHECK(Obj2->GetPathName() == BeforeRename2.GetPathName());
	REQUIRE(!BeforeRenameInner2.IsResolved());
	CHECK(Inner2->GetPathName() == BeforeRenameInner2.GetPathName());

	Obj1->Rename(TEXT("RenamedObj"), nullptr);

	FObjectPtr AfterRenameObjPtr1(MakeUnresolvedHandle(Obj1));
	FObjectPtr AfterRenameInnerPtr1(MakeUnresolvedHandle(Inner1));

	TObjectPtr<UObject> AfterRenameUnresolved1 = *reinterpret_cast<TObjectPtr<UObject>*>(&AfterRenameObjPtr1);
	TObjectPtr<UObject> AfterRenameInnerUnresolved1 = *reinterpret_cast<TObjectPtr<UObject>*>(&AfterRenameInnerPtr1);

	TObjectPtr<UObject> AfterRename1 = Obj1;
	TObjectPtr<UObject> AfterRenameInner1 = Inner1;

	CHECK(Obj1->GetPathName() == BeforeRename1.GetPathName());
	REQUIRE(!BeforeRenameInner1.IsResolved());
	CHECK(Inner1->GetPathName() == BeforeRenameInner1.GetPathName());

	CHECK(BeforeRename1 == AfterRename1);
	CHECK(BeforeRenameInner1 == AfterRenameInner1);

	CHECK(BeforeRename1 == AfterRenameUnresolved1);
	CHECK(BeforeRenameInner1 == AfterRenameInnerUnresolved1);

	Obj1->Rename(nullptr, Inner2);
	CHECK(Obj2->GetPathName() == BeforeRename2.GetPathName());
	REQUIRE(!BeforeRenameInner2.IsResolved());
	CHECK(Inner2->GetPathName() == BeforeRenameInner2.GetPathName());
	CHECK(BeforeRename1 == AfterRename1);
	CHECK(BeforeRename1.GetPathName() == AfterRename1.GetPathName());
	CHECK(BeforeRenameInner1 == AfterRenameInner1);

	TestPackage->RemoveFromRoot();
}

TEST_CASE("CoreUObject::TObjectPtr::Swap")
{
	const FName TestPackageName("/Engine/TestPackage");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));
	UObject* RawPtrA = Obj1;
	UObject* RawPtrB = Inner1;

	FObjectPtr ObjPtr1(MakeUnresolvedHandle(RawPtrA));
	FObjectPtr InnerPtr1(MakeUnresolvedHandle(RawPtrB));

	TObjectPtr<UObject> PtrA = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr1);
	TObjectPtr<UObject> PtrB = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr1);
	CHECK(!PtrA.IsResolved());
	CHECK(!PtrB.IsResolved());

	Swap(PtrA, PtrB);
	
	CHECK(!PtrA.IsResolved());
	CHECK(!PtrB.IsResolved());
	CHECK(PtrA != PtrB);
	CHECK(PtrA != RawPtrA);
	
	Swap(PtrA, RawPtrA);
	CHECK(PtrA.IsResolved());
	CHECK(RawPtrA == Inner1);
	CHECK(PtrA == Obj1);

	Swap(PtrB, PtrA);
	CHECK(!PtrA.IsResolved());
	CHECK(PtrB.IsResolved());

}

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
TEST_CASE("CoreUObject::TObjectPtr::SwapArray")
{
	const FName TestPackageName("/Engine/TestPackage");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));
	UObject* RawPtrA = Obj1;
	UObject* RawPtrB = Inner1;

	FObjectPtr ObjPtr1(MakeUnresolvedHandle(RawPtrA));
	FObjectPtr InnerPtr1(MakeUnresolvedHandle(RawPtrB));
	TObjectPtr<UObject> PtrA = *reinterpret_cast<TObjectPtr<UObject>*>(&ObjPtr1);
	TObjectPtr<UObject> PtrB = *reinterpret_cast<TObjectPtr<UObject>*>(&InnerPtr1);

	
	TArray<TObjectPtr<UObject>> ArrayPtr;
	auto t = ArrayPtr.begin();
	ArrayPtr.Add(PtrA);

	TArray<UObject*> ArrayRaw;
	ArrayRaw.Add(RawPtrB);

	Swap(ArrayRaw, ArrayPtr);

	CHECK(ArrayRaw[0] == RawPtrA);
	CHECK(ArrayPtr[0] == RawPtrB);
}
#endif // !UE_DEPRECATE_MUTABLE_TOBJECTPTR

TEST_CASE("CoreUObject::TObjectPtr::Move")
{
	UPackage* TestPackageA = NewObject<UPackage>(nullptr, TEXT("/Engine/PackageA"), RF_Transient);
	TestPackageA->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackageA, TEXT("Obj1"));
	UObject* Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));
	
	UPackage* TestPackageB = NewObject<UPackage>(nullptr, TEXT("/Engine/PackageB"), RF_Transient);
	TestPackageB->AddToRoot();

	FObjectPtr FObjPtr(MakeUnresolvedHandle(Obj1));
	FObjectPtr FInnerPtr(MakeUnresolvedHandle(Inner1));

	TObjectPtr<UObject> BeforeRename = *reinterpret_cast<TObjectPtr<UObject>*>(&FObjPtr);
	TObjectPtr<UObject> BeforeRenameInner = *reinterpret_cast<TObjectPtr<UObject>*>(&FInnerPtr);

	TObjectPtr<UObject> ObjPtr1 = Obj1;
	TObjectPtr<UObject> InnerPtr1 = Inner1;

	REQUIRE(!BeforeRename.IsResolved());
	CHECK(BeforeRename == ObjPtr1);
	CHECK(Obj1->GetPathName() == BeforeRename.GetPathName());
	REQUIRE(!BeforeRenameInner.IsResolved());
	CHECK(InnerPtr1 == BeforeRenameInner);
	CHECK(Inner1->GetPathName() == BeforeRenameInner.GetPathName());

	Obj1->Rename(TEXT("Obj2"), TestPackageB);
	bool e = BeforeRename == ObjPtr1;

	REQUIRE(!BeforeRename.IsResolved());
	CHECK(BeforeRename == ObjPtr1);
	CHECK(Obj1->GetPathName() == BeforeRename.GetPathName());
	CHECK(InnerPtr1 == BeforeRenameInner);
	CHECK(Inner1->GetPathName() == BeforeRenameInner.GetPathName());

	TObjectPtr<UObject> AfterRename = Obj1;
	TObjectPtr<UObject> AfterRenameInner = Inner1;
	CHECK(BeforeRename == AfterRename);
	CHECK(BeforeRenameInner == AfterRenameInner);

	TestPackageA->RemoveFromRoot();
	TestPackageB->RemoveFromRoot();
}

TEST_CASE("CoreUObject::TObjectPtr::GetTypeHash")
{
	int ResolveCount = 0;
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto ResolveDelegate = [&ResolveCount](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++ResolveCount;
		};
	auto Handle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback(ResolveDelegate);
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(Handle);
	};
#endif

	UPackage* TestPackageA = NewObject<UPackage>(nullptr, TEXT("/Engine/PackageA"), RF_Transient);
	TestPackageA->AddToRoot();
	TObjectPtr<UObject> Obj1 = NewObject<UObjectPtrTestClass>(TestPackageA, TEXT("Obj1"));
	TObjectPtr<UObject> Inner1 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Inner1"));

	UE::CoreUObject::Private::MakePackedObjectRef(TestPackageA);
	UE::CoreUObject::Private::MakePackedObjectRef(Obj1.Get());
	UE::CoreUObject::Private::MakePackedObjectRef(Inner1.Get());

	TMap<TObjectPtr<UObject>, int> TestMap;

	int32 BeforeKey = GetTypeHash(Obj1);
	int32 BeforeKeyInner = GetTypeHash(Inner1);
	int32 BeforeKeyPackage = GetTypeHash(TestPackageA);

	TestMap.Add(Obj1, 0);
	TestMap.Add(Inner1, 1);
	TestMap.Add(TestPackageA, 3);
	
	//add a bunch of objects
	for (int i = 0; i < 10; ++i)
	{
		TestMap.Add(NewObject<UObjectPtrTestClass>(TestPackageA), TestMap.Num());;
	}
	for (int i = 0; i < 10; ++i)
	{
		TestMap.Add(NewObject<UObjectPtrNotLazyTestClass>(TestPackageA), i);;
	}

	TObjectPtr<UObject> NotLazy = NewObject<UObjectPtrNotLazyTestClass>(TestPackageA);
	int NotLazyValue = TestMap.Num();
	TestMap.Add(NotLazy, NotLazyValue);
	CHECK(GetTypeHash(NotLazy) == GetTypeHash(NotLazy.Get()));
	CHECK(TestMap[NotLazy] == NotLazyValue);


	FObjectPtr FPackagePtr(MakeUnresolvedHandle(TestPackageA));
	FObjectPtr FObjPtr(MakeUnresolvedHandle(Obj1));
	FObjectPtr FInnerPtr(MakeUnresolvedHandle(Inner1));

	TObjectPtr<UObject> BeforeRename = *reinterpret_cast<TObjectPtr<UObject>*>(&FObjPtr);
	TObjectPtr<UObject> BeforeRenameInner = *reinterpret_cast<TObjectPtr<UObject>*>(&FInnerPtr);
	TObjectPtr<UObject> BeforeRenamePackage = *reinterpret_cast<TObjectPtr<UObject>*>(&FPackagePtr);

	int32 BeforeRenameTypeHash = GetTypeHash(BeforeRename);
	int32 BeforeRenameTypeHashInner = GetTypeHash(BeforeRenameInner);
	int32 BeforeRenamePackageTypeHash = GetTypeHash(BeforeRenamePackage);

	CHECK(BeforeKey == BeforeRenameTypeHash);
	CHECK(BeforeKey == BeforeRenameTypeHash);
	CHECK(BeforeKeyInner == BeforeRenameTypeHashInner);
	CHECK(ResolveCount == 0);
		
	CHECK(TestMap[Obj1] == 0);
	CHECK(TestMap[BeforeRename] == 0);

	CHECK(TestMap[Inner1] == 1);
	CHECK(TestMap[BeforeRenameInner] == 1);

	CHECK(TestMap[TestPackageA] == 3);
	CHECK(TestMap[BeforeRenamePackage] == 3);

	CHECK(!BeforeRename.IsResolved());
	CHECK(!BeforeRenameInner.IsResolved());
	CHECK(!BeforeRenamePackage.IsResolved());

	auto TestMapFunc = [&]()
	{
		TMap<TObjectPtr<UObject>, bool> Map;
		Map.Add(Obj1, false);
		Map.Add(Inner1, true);

		CHECK(ResolveCount == 0);
		bool* Found = Map.Find(BeforeRename);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(!*Found);

		Found = Map.Find(BeforeRenameInner);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(*Found);

		CHECK(ResolveCount == 0);
		Found = Map.Find(Obj1);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(!*Found);

		CHECK(ResolveCount == 0);
		Found = Map.Find(Inner1);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(*Found);
	};

	auto TestUnResolvedMapFunc = [&]()
	{
		TMap<TObjectPtr<UObject>, bool> Map;
		Map.Add(BeforeRename, false);
		Map.Add(BeforeRenameInner, true);

		CHECK(ResolveCount == 0);
		bool* Found = Map.Find(BeforeRename);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(!*Found);

		Found = Map.Find(BeforeRenameInner);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(*Found);

		CHECK(ResolveCount == 0);
		Found = Map.Find(Obj1);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(!*Found);

		CHECK(ResolveCount == 0);
		Found = Map.Find(Inner1);
		CHECK(ResolveCount == 0);
		CHECK(Found != nullptr);
		CHECK(*Found);
	};

	TestMapFunc();
	TestUnResolvedMapFunc();

	UPackage* TestPackageB = NewObject<UPackage>(nullptr, TEXT("/Engine/PackageB"), RF_Transient);
	TestPackageB->AddToRoot();

	Obj1->Rename(nullptr, TestPackageB);

	TestMapFunc();
	TestUnResolvedMapFunc();

	TestPackageA->RemoveFromRoot();
	TestPackageB->RemoveFromRoot();
}


#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

template<typename TObj>
void TestArrayConversion()
{
	const FName TestPackageName("/Engine/TestPackage");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));

#if UE_WITH_OBJECT_HANDLE_TRACKING
	int ResolveCount = 0;
	uint32 ObjCount = 0;
	auto ResolveDelegate = [&](const TArrayView<const UObject* const>& Objects)
	{
		++ResolveCount;
		ObjCount = Objects.Num();
		for(const UObject* ReadObj : Objects)
		{
			CHECK(ReadObj == Obj1);
		}
	};
	auto Handle = UE::CoreUObject::AddObjectHandleReadCallback(ResolveDelegate);
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReadCallback(Handle);
	};
#endif
	TArray<TObjectPtr<TObj>> PtrArray;
	uint32  NumObjs = 5;
	for (uint32 i = 0; i < NumObjs; ++i)
	{
		PtrArray.Add(Obj1);
	}

	{
		TArray<TObj*> RawArray;
		RawArray = PtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == NumObjs);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == NumObjs);
		for (uint32 i = 0; i < NumObjs; ++i)
		{
			CHECK(RawArray[i] == Obj1);
		}
	}
	{
		TArray<TObjectPtr<TObj>> EmptyArray;
		TArray<TObj*> RawArray;
		RawArray = EmptyArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == 0);
	}
	{
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = PtrArray;
		TArray<TObj*> RawArray;
		RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == NumObjs);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == NumObjs);
		for (uint32 i = 0; i < NumObjs; ++i)
		{
			CHECK(RawArray[i] == Obj1);
		}
	}
	{
		TArray<TObjectPtr<TObj>> EmptyArray;
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = EmptyArray;
		TArray<TObj*> RawArray;
		RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == 0);
	}
	{
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = PtrArray;
		const TArray<TObj*>& RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == NumObjs);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == NumObjs);
		for (uint32 i = 0; i < NumObjs; ++i)
		{
			CHECK(RawArray[i] == Obj1);
		}
	}
	{
		TArray<TObjectPtr<TObj>> EmptyArray;
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = EmptyArray;
		const TArray<TObj*>& RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == 0);
	}

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
	//ArrayView
	{
		TArrayView<TObj*> RawArray = PtrArray;
#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == NumObjs);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == NumObjs);
		for (uint32 i = 0; i < NumObjs; ++i)
		{
			CHECK(RawArray[i] == Obj1);
		}
	}
#endif

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
	{
		TArray<TObjectPtr<TObj>> EmptyArray;
		TArrayView<TObj*> RawArray = EmptyArray;
#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == 0);
	}
#endif
  
	{
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = PtrArray;
		const TArrayView<TObj* const> RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == NumObjs);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == NumObjs);
		for (uint32 i = 0; i < NumObjs; ++i)
		{
			CHECK(RawArray[i] == Obj1);
		}
	}
	{
		TArray<TObjectPtr<TObj>> EmptyArray;
		const TArray<TObjectPtr<TObj>>& ConstPtrArray = EmptyArray;
		const TArrayView<TObj* const> RawArray = ConstPtrArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawArray.Num() == 0);
	}
}

TEST_CASE("CoreUObject::TObjectPtr::ArrayConversion")
{
	TestArrayConversion<UObject>();
	TestArrayConversion<const UObject>();
}

TEST_CASE("CoreUObject::TObjectPtr::ArrayConversionReferenceForSet")
{
	const FName TestPackageName("/Engine/TestPackage");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	UObject* Obj2 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj2"));

#if UE_WITH_OBJECT_HANDLE_TRACKING
	int ResolveCount = 0;
	uint32 ObjCount = 0;
	auto ResolveDelegate = [&](const TArrayView<const UObject* const>& Objects)
	{
		++ResolveCount;
		ObjCount = Objects.Num();

		if (ObjCount == 2)
		{
			CHECK(Objects[0] == Obj1);
			CHECK(Objects[1] == Obj2);
		}
	};
	auto Handle = UE::CoreUObject::AddObjectHandleReadCallback(ResolveDelegate);
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReadCallback(Handle);
	};
#endif
	{
		TArray<TObjectPtr<UObject>> Array;
		Array.Add(Obj1);
		Array.Add(Obj2);

		TSet<UObject*> RawSet(Array);

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 2);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawSet.Num() == 2);
	}
	{
		TArray<TObjectPtr<UObject>> Array;
		TSet<UObject*> RawSet(Array);

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		CHECK(ObjCount == 0);
		ResolveCount = 0;
		ObjCount = 0;
#endif
		CHECK(RawSet.Num() == 0);
	}
}


TEST_CASE("CoreUObject::TObjectPtr::ConstArrayViewConversion")
{
	TObjectPtr<UObject> ObjectArray[3];
	TConstArrayView<TObjectPtr<UObject>> View(&ObjectArray[0], 3);

#if UE_WITH_OBJECT_HANDLE_TRACKING
	int ResolveCount = 0;
	auto ResolveDelegate = [&](const TArrayView<const UObject* const>& Objects)
	{
		++ResolveCount;
		CHECK(Objects.Num() == 3);
		CHECK(Objects[0] == nullptr);
		CHECK(Objects[1] == nullptr);
		CHECK(Objects[2] == nullptr);
	};
	auto Handle = UE::CoreUObject::AddObjectHandleReadCallback(ResolveDelegate);
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReadCallback(Handle);
	};
#endif

	{
		TConstArrayView<UObject*> ConvertedArray = View;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
		ResolveCount = 0;
#endif

		CHECK(ConvertedArray.Num() == 3);
		CHECK(ConvertedArray[0] == nullptr);
		CHECK(ConvertedArray[1] == nullptr);
		CHECK(ConvertedArray[2] == nullptr);
	}
	{
		TArrayView<const TObjectPtr<UObject>> ConstArray(&ObjectArray[0], 3);
		TArrayView<const UObject* const> ConvertedArray = ConstArray;

#if UE_WITH_OBJECT_HANDLE_TRACKING
		CHECK(ResolveCount == 1);
#endif

		CHECK(ConvertedArray.Num() == 3);
		CHECK(ConvertedArray[0] == nullptr);
		CHECK(ConvertedArray[1] == nullptr);
		CHECK(ConvertedArray[2] == nullptr);
	}

	
}
TEST_CASE("CoreUObject::TObjectPtr::GetOuter")
{
	UPackage* TestPackage = NewObject<UPackage>(nullptr, "/Test/MyPackage", RF_Transient);
	TestPackage->AddToRoot();
	TObjectPtr<UObject> Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("Obj1"));
	TObjectPtr<UObject> Obj2 = NewObject<UObjectPtrTestClass>(Obj1, TEXT("Obj2"));
	int ResolveCount = 0;

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	FObjectPtr Ptr1(MakeUnresolvedHandle(TestPackage));
	FObjectPtr Ptr2(MakeUnresolvedHandle(Obj1));
	FObjectPtr Ptr3(MakeUnresolvedHandle(Obj2));

	TObjectPtr<UPackage> PackagePtr = *reinterpret_cast<TObjectPtr<UPackage>*>(&Ptr1);
	TObjectPtr<UObject> Obj1Ptr = *reinterpret_cast<TObjectPtr<UObject>*>(&Ptr2);
	TObjectPtr<UObject> Obj2Ptr = *reinterpret_cast<TObjectPtr<UObject>*>(&Ptr3);


	auto CallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([&ResolveCount](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++ResolveCount;
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackHandle);
	};
#else
	TObjectPtr<UPackage> PackagePtr = TestPackage;
	TObjectPtr<UObject> Obj1Ptr = Obj1;
	TObjectPtr<UObject> Obj2Ptr = Obj2;
#endif
	TObjectPtr<UObject> Obj1RawOuter = Obj1->GetOuter();
	TObjectPtr<UObject> Obj2RawOuter = Obj2->GetOuter();

	TObjectPtr<UObject> PackageOuter = PackagePtr.GetOuter();
	TObjectPtr<UObject> Obj1Outer = Obj1Ptr.GetOuter();
	TObjectPtr<UObject> Obj2Outer = Obj2Ptr.GetOuter();

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(!Obj1Outer.IsResolved());
	CHECK(!Obj2Outer.IsResolved());

	//sanity check that the packed refs are identical
	CHECK(Obj1Outer.GetHandle().PointerOrRef == PackagePtr.GetHandle().PointerOrRef);
	CHECK(Obj2Outer.GetHandle().PointerOrRef == Obj1Ptr.GetHandle().PointerOrRef);

#endif

	CHECK(Obj1Outer.GetHandle() == PackagePtr.GetHandle());
	CHECK(Obj2Outer.GetHandle() == Obj1Ptr.GetHandle());

	CHECK(PackageOuter == TestPackage->GetOuter());
	CHECK(PackageOuter == nullptr);
	CHECK(Obj1Outer == Obj1RawOuter);
	CHECK(Obj1Outer.GetFName() == Obj1RawOuter->GetFName());
	CHECK(Obj1Outer.GetPathName() == Obj1RawOuter->GetPathName());
	CHECK(Obj1Outer.GetFullName() == Obj1RawOuter->GetFullName());
	CHECK(Obj1Outer.GetClass() == Obj1RawOuter->GetClass());

	CHECK(Obj2Outer == Obj2RawOuter);
	
	CHECK(Obj2Outer.GetFName() == Obj2RawOuter->GetFName());
	CHECK(Obj2Outer.GetPathName() == Obj2RawOuter->GetPathName());
	CHECK(Obj2Outer.GetFullName() == Obj2RawOuter->GetFullName());
	CHECK(Obj2Outer.GetClass() == Obj2RawOuter->GetClass());


	TObjectPtr<UPackage> Package = PackagePtr.GetPackage();
	TObjectPtr<UPackage> Obj1Package = Obj1Ptr.GetPackage();
	TObjectPtr<UPackage> Obj2Package = Obj2Ptr.GetPackage();

	CHECK(Package == PackagePtr);
	CHECK(Obj1Package == PackagePtr);
	CHECK(Obj2Package == PackagePtr);

	CHECK(ResolveCount == 0);
}
// @TODO: OBJPTR: We should have a test that ensures that lazy loading of an object with an external package is handled correctly.
//				  This should also include external packages in the outer chain of the target object.
// IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FObjectPtrTestExternalPackages, FObjectPtrTestBase, TEXT(TEST_NAME_ROOT ".ExternalPackages"), ObjectPtrTestFlags)
// bool FObjectPtrTestExternalPackages::RunTest(const FString& Parameters)
// {
// 	const FName TestExternalPackage1Name("/Engine/Test/ObjectPtrExternalPackages1/Transient");
// 	UPackage* TestExternalPackage1 = NewObject<UPackage>(nullptr, TestExternalPackage1Name, RF_Transient);
// 	TestExternalPackage1->SetPackageFlags(PKG_EditorOnly | PKG_ContainsMapData);
// 	TestExternalPackage1->AddToRoot();

// 	const FName TestPackage1Name("/Engine/Test/ObjectPtrExternalPackages1/Transient");
// 	UPackage* TestPackage1 = NewObject<UPackage>(nullptr, TestPackage1Name, RF_Transient);
// 	TestPackage1->AddToRoot();
// 	UObject* TestOuter1 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter1"));
// 	UObject* TestOuter2 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter2"));
// 	UObject* TestOuter3 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter3"));
// 	UObject* TestOuter4 = NewObject<UObjectPtrTestClass>(TestPackage1, TEXT("TestOuter4"));

// 	UObject* TestPublicObject = NewObject<UObjectPtrTestClass>(TestOuter4, TEXT("TestPublicObject"), RF_Public);
// 	TestPublicObject->SetExternalPackage(TestExternalPackage1);

// 	TestPackage1->RemoveFromRoot();
// 	TestExternalPackage1->RemoveFromRoot();
// 	return true;
// }

// @TODO: OBJPTR: We should have a test that ensures that we can (de)serialize an FObjectPtr to FLinkerSave/FLinkerLoad and that upon load the object
//			pointer is not resolved if we are in a configuration that supports lazy load.  This is proving difficult due to the restrictions around how
//			FLinkerSave/FLinkerLoad is used.
// IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FObjectPtrTestLinkerSerializeBehavior, FObjectPtrTestBase, TEXT(TEST_NAME_ROOT ".LinkerSerialize"), ObjectPtrTestFlags)
// bool FObjectPtrTestLinkerSerializeBehavior::RunTest(const FString& Parameters)
// {
// 	FSnapshotObjectRefMetrics ObjectRefMetrics(*this);
// 	FObjectPtr DefaultTexturePtr(FObjectRef {FName("/Engine/EngineResources/DefaultTexture"), NAME_None, NAME_None, FObjectPathId("DefaultTexture")});

// 	TUniquePtr<FLinkerSave> Linker = TUniquePtr<FLinkerSave>(new FLinkerSave(nullptr /*InOuter*/, false /*bForceByteSwapping*/, true /*bSaveUnversioned*/));
// 	return true;
// }


#endif // WITH_LOW_LEVEL_TESTS

class UForwardDeclaredObjDerived: public UObject {};
class FForwardDeclaredNotObjDerived {};

}

#if WITH_LOW_LEVEL_TESTS

TEST_CASE("CoreUObject::TObjectPtr::TestEquals")
{
	const FName TestPackageName("/Engine/Test/TestEquals/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UDerrivedClass* Obj = NewObject<UDerrivedClass>(TestPackage, TEXT("DefaultSerializeObject"));

	FTestBaseClass* BasePtr = Obj;
	TObjectPtr<UDerrivedClass> ObjPtr(Obj);

	CHECK(BasePtr == ObjPtr);
}

TEST_CASE("CoreUObject::TObjectPtr::DecayAndWrap")
{
	const FName TestPackageName("/Engine/Test/TestName/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	

	UObject* RawPtr1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("RawPtr1"));
	UObject* RawPtr2 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("RawPtr2"));	
	{
		TObjectPtr<UObject> ObjPtr{RawPtr1};
		CHECK(RawPtr1 == ObjectPtrDecay(ObjPtr));
		CHECK(ObjPtr == ObjectPtrWrap(RawPtr1));
		CHECK(ObjectPtrDecay(ObjectPtrWrap(RawPtr1)) == RawPtr1);
		CHECK(ObjectPtrWrap(ObjectPtrDecay(ObjPtr)) == ObjPtr);
	}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	{
		FObjectPtr Unresolved{MakeUnresolvedHandle(RawPtr1)};
		TObjectPtr<UObject> Ptr = reinterpret_cast<TObjectPtr<UObject>&>(Unresolved);
		REQUIRE(!Unresolved.IsResolved());
		CHECK(ObjectPtrDecay(Ptr) == RawPtr1);
		CHECK(Ptr.IsResolved());
		
		TArray<UObject*> RawArray = {RawPtr1, RawPtr2};
		TArray<FObjectPtr> UnresolvedArray = {FObjectPtr(MakeUnresolvedHandle(RawPtr1)),
																					FObjectPtr(RawPtr2)};
		REQUIRE(!UnresolvedArray[0].IsResolved());
		REQUIRE(UnresolvedArray[1].IsResolved());		 
		TArray<TObjectPtr<UObject>> ObjArray = reinterpret_cast<TArray<TObjectPtr<UObject>>&>(UnresolvedArray);
		CHECK(ObjectPtrDecay(ObjArray) == RawArray);
		CHECK(ObjArray[0].IsResolved());
		CHECK(ObjArray[1].IsResolved());		
	}
#endif
};

#endif
