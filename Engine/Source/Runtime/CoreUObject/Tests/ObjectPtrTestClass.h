// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TESTS

#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Templates/NonNullPointer.h"
#include "Templates/SubclassOf.h"

#include "ObjectPtrTestClass.generated.h"

//simple test class for testing TObjectPtr resolve behavior
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrTestClass : public UObject
{
	GENERATED_BODY()
};

//abstract test class for testing TObjectPtr resolve behavior
UCLASS(Abstract, meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrAbstractTestClass : public UObject
{
	GENERATED_BODY()
};

//derived-from-abstract test class for testing TObjectPtr resolve behavior
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrAbstractDerivedTestClass : public UObjectPtrAbstractTestClass
{
	GENERATED_BODY()
};

//test class with typed reference to another class
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrTestClassWithRef : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UObjectPtrTestClass> ObjectPtr;

	UPROPERTY()
	TObjectPtr<UObjectPtrTestClass> ObjectPtrNonNullable;

	UPROPERTY()
	TObjectPtr<UObjectPtrAbstractTestClass> ObjectPtrAbstractNonNullable;

	UPROPERTY()
	TArray<TObjectPtr<UObjectPtrTestClass>> ArrayObjPtr;
};


//test class with typed reference to another class
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectWithClassProperty : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UClass> ClassPtr;
	UPROPERTY()
	TSubclassOf<UObjectPtrTestClass> SubClass;
	UPROPERTY()
	UClass* ClassRaw;
};

//test class with raw pointer
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectWithRawProperty : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	UObjectPtrTestClass* ObjectPtr;
	UPROPERTY()
	UObjectPtrTestClass* ObjectPtrNonNullable;
};


//derived test class
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrDerrivedTestClass : public UObjectPtrTestClass
{
	GENERATED_BODY()
};


//non lazy test class
UCLASS()
class UObjectPtrNotLazyTestClass : public UObject
{
	GENERATED_BODY()
};


//stress testing class
UCLASS(meta=(LoadBehavior=LazyOnDemand))
class UObjectPtrStressTestClass : public UObject
{
	GENERATED_BODY()

public:
	uint8 Data[PLATFORM_CACHE_LINE_SIZE];
};

class FTestBaseClass
{
public:
	virtual ~FTestBaseClass() = default;
	virtual void VirtFunc() { };
};

UCLASS()
class UMiddleClass : public UObject, public FTestBaseClass
{
	GENERATED_BODY()

public:
	virtual void VirtFunc() override { };
};


class FAnotherBaseClass
{
public:
	virtual ~FAnotherBaseClass() = default;
	virtual void AnotherVirtFunc() { };
};


UCLASS()
class UDerrivedClass : public UMiddleClass, public FAnotherBaseClass
{
	GENERATED_BODY()
public:
	virtual void AnotherVirtFunc() override { };
};


#endif