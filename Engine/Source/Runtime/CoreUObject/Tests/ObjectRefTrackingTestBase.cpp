// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "ObjectRefTrackingTestBase.h"
#include "UObject/Package.h"
#include "ObjectPtrTestClass.h"
#include "Misc/ScopeExit.h"
#include "UObject/ObjectPtr.h"

thread_local uint32 FObjectRefTrackingTestBase::NumResolves = 0;
thread_local uint32 FObjectRefTrackingTestBase::NumFailedResolves = 0;
thread_local uint32 FObjectRefTrackingTestBase::NumReads = 0;
thread_local TMap<const UObject* const, uint32> FObjectRefTrackingTestBase::NumReadsPerObject{};

#if UE_WITH_OBJECT_HANDLE_TRACKING
UE::CoreUObject::FObjectHandleTrackingCallbackId FObjectRefTrackingTestBase::ResolvedCallbackHandle;
UE::CoreUObject::FObjectHandleTrackingCallbackId FObjectRefTrackingTestBase::HandleReadCallbackHandle;

TEST_CASE("UE::CoreUObject::ObjectHandleTracking::Callbacks")
{
	using namespace UE::CoreUObject;
	using namespace UE::CoreUObject::Private;

	int32 CallbackCount1 = 0;
	int32 CallbackCount2 = 0;
	int32 CallbackCount3 = 0;
	int32 CallbackCount4 = 0;
	int32 CallbackCount5 = 0;
	int32 CallbackCount6 = 0;
	int32 CallbackCount7 = 0;
	int32 CallbackCount8 = 0;

	FObjectHandleReadFunc ReadCallback1 = [&](const TArrayView<const UObject* const>& Objects)
	{
		++CallbackCount1;
	};
	FObjectHandleReadFunc ReadCallback2 = [&](const TArrayView<const UObject* const>& Objects)
	{
		++CallbackCount2;
	};

	FObjectHandleClassResolvedFunc ClassResolvedCallback1 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UClass* Class)
	{
		++CallbackCount3;
	};
	FObjectHandleClassResolvedFunc ClassResolvedCallback2 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UClass* Class)
	{
		++CallbackCount4;
	};

	FObjectHandleReferenceResolvedFunc ReferenceResolvedCallback1 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
	{
		++CallbackCount5;
	};
	FObjectHandleReferenceResolvedFunc ReferenceResolvedCallback2 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
	{
		++CallbackCount6;
	};

	FObjectHandleReferenceLoadedFunc ReferenceLoadedCallback1 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
	{
		++CallbackCount7;
	};
	FObjectHandleReferenceLoadedFunc ReferenceLoadedCallback2 = [&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
	{
		++CallbackCount8;
	};
	auto ReadCallbackHandle1 = AddObjectHandleReadCallback(ReadCallback1);
	auto ReadCallbackHandle2 = AddObjectHandleReadCallback(ReadCallback2);

	auto ClassResolvedHandle1 = AddObjectHandleClassResolvedCallback(ClassResolvedCallback1);
	auto ClassResolvedHandle2 = AddObjectHandleClassResolvedCallback(ClassResolvedCallback2);

	auto ReferenceResolvedHandle1 = AddObjectHandleReferenceResolvedCallback(ReferenceResolvedCallback1);
	auto ReferenceResolvedHandle2 = AddObjectHandleReferenceResolvedCallback(ReferenceResolvedCallback2);

	auto ReferenceLoadedHandle1 = AddObjectHandleReferenceLoadedCallback(ReferenceLoadedCallback1);
	auto ReferenceLoadedHandle2 = AddObjectHandleReferenceLoadedCallback(ReferenceLoadedCallback2);

	auto TestCounts = [&](int32 a, int32 b, int32 c, int32 d, int32 e, int32 f, int32 g, int32 h)
	{
		CHECK(CallbackCount1 == a);
		CHECK(CallbackCount2 == b);
		CHECK(CallbackCount3 == c);
		CHECK(CallbackCount4 == d);
		CHECK(CallbackCount5 == e);
		CHECK(CallbackCount6 == f);
		CHECK(CallbackCount7 == g);
		CHECK(CallbackCount8 == h);
	};

	UE::CoreUObject::Private::OnHandleRead(nullptr);
	TestCounts(1, 1, 0, 0, 0, 0, 0, 0);

	//remove the first added read callback to verify they can be remove in any order
	RemoveObjectHandleReadCallback(ReadCallbackHandle1);

	UE::CoreUObject::Private::OnHandleRead(nullptr);
	TestCounts(1, 2, 0, 0, 0, 0, 0, 0);
	
	//add it back
	ReadCallbackHandle1 = AddObjectHandleReadCallback(ReadCallback1);
	UE::CoreUObject::Private::OnHandleRead(nullptr);
	TestCounts(2, 3, 0, 0, 0, 0, 0, 0);

	//remove it again
	RemoveObjectHandleReadCallback(ReadCallbackHandle1);
	UE::CoreUObject::Private::OnHandleRead(nullptr);
	TestCounts(2, 4, 0, 0, 0, 0, 0, 0);

	//remove last callback
	RemoveObjectHandleReadCallback(ReadCallbackHandle2);
	UE::CoreUObject::Private::OnHandleRead(nullptr);
	TestCounts(2, 4, 0, 0, 0, 0, 0, 0);

	//////////////////////////////////////////////////////////////////////////////////
	UE::CoreUObject::Private::OnClassReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 1, 1, 0, 0, 0, 0);

	//remove the first added read callback to verify they can be remove in any order
	RemoveObjectHandleClassResolvedCallback(ClassResolvedHandle1);

	UE::CoreUObject::Private::OnClassReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 1, 2, 0, 0, 0, 0);

	//add it back
	ClassResolvedHandle1 = AddObjectHandleClassResolvedCallback(ClassResolvedCallback1);
	UE::CoreUObject::Private::OnClassReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 3, 0, 0, 0, 0);

	//remove it again
	RemoveObjectHandleClassResolvedCallback(ClassResolvedHandle1);
	UE::CoreUObject::Private::OnClassReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 0, 0, 0, 0);

	//remove last callback
	RemoveObjectHandleClassResolvedCallback(ClassResolvedHandle2);
	UE::CoreUObject::Private::OnClassReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 0, 0, 0, 0);


	//////////////////////////////////////////////////////////////////////////////////
	UE::CoreUObject::Private::OnReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 1, 1, 0, 0);

	//remove the first added read callback to verify they can be remove in any order
	RemoveObjectHandleReferenceResolvedCallback(ReferenceResolvedHandle1);

	UE::CoreUObject::Private::OnReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 1, 2, 0, 0);

	//add it back
	ReferenceResolvedHandle1 = AddObjectHandleReferenceResolvedCallback(ReferenceResolvedCallback1);
	UE::CoreUObject::Private::OnReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 3, 0, 0);

	//remove it again
	RemoveObjectHandleReferenceResolvedCallback(ReferenceResolvedHandle1);
	UE::CoreUObject::Private::OnReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 0, 0);

	//remove last callback
	RemoveObjectHandleReferenceResolvedCallback(ReferenceResolvedHandle2);
	UE::CoreUObject::Private::OnReferenceResolved(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 0, 0);

	//////////////////////////////////////////////////////////////////////////////////
	UE::CoreUObject::Private::OnReferenceLoaded(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 1, 1);

	//remove the first added read callback to verify they can be remove in any order
	RemoveObjectHandleReferenceLoadedCallback(ReferenceLoadedHandle1);

	UE::CoreUObject::Private::OnReferenceLoaded(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 1, 2);

	//add it back
	ReferenceLoadedHandle1 = AddObjectHandleReferenceLoadedCallback(ReferenceLoadedCallback1);
	UE::CoreUObject::Private::OnReferenceLoaded(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 2, 3);

	//remove it again
	RemoveObjectHandleReferenceLoadedCallback(ReferenceLoadedHandle1);
	UE::CoreUObject::Private::OnReferenceLoaded(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 2, 4);

	//remove last callback
	RemoveObjectHandleReferenceLoadedCallback(ReferenceLoadedHandle2);
	UE::CoreUObject::Private::OnReferenceLoaded(FObjectRef(), nullptr, nullptr);
	TestCounts(2, 4, 2, 4, 2, 4, 2, 4);
}

TEST_CASE("UE::CoreUObject::ObjectHandleTracking::Callbacks::Verify")
{
	using namespace UE::CoreUObject;
	using namespace UE::CoreUObject::Private;

	const FName TestPackageName("/Engine/Test/ObjectRef/Transient");
	UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
	TestPackage->AddToRoot();
	UObject* Obj1 = NewObject<UObjectPtrTestClass>(TestPackage, TEXT("DefaultSerializeObject"));
	ON_SCOPE_EXIT{
		TestPackage->RemoveFromRoot();
	};

	TObjectPtr<UObject> ObjPtr = Obj1;
	int32 ReadCount = 0;
	auto ReadCallbackHandle1 = AddObjectHandleReadCallback([&](const TArrayView<const UObject* const>& ReadObject)
		{
			++ReadCount;
			CHECK(Obj1 == ReadObject[0]);
		});
	
	CHECK(ObjPtr == Obj1);
	CHECK(ReadCount == 0);

	CHECK_FALSE(ObjPtr != Obj1);
	CHECK(ReadCount == 0);

	CHECK_FALSE(ObjPtr == nullptr);
	CHECK(ReadCount == 0);
	
	CHECK(ObjPtr != nullptr);
	CHECK(ReadCount == 0);

	bool bIsNull = !ObjPtr;
	CHECK_FALSE(bIsNull);
	CHECK(ReadCount == 0);

	if (ObjPtr)
	{
		CHECK(ReadCount == 0);
	}

	//get fires read event
	CHECK(ObjPtr.Get() != nullptr);
	CHECK(ReadCount == 1);

	RemoveObjectHandleReadCallback(ReadCallbackHandle1);
	
	FObjectRef ObjectRef(Obj1);
	int32 ClassCount = 0;
	auto ClassResolvedHandle1 = AddObjectHandleClassResolvedCallback([&](const FObjectRef& SourceRef, UPackage* ClassPackage, UClass* Class)
		{
			++ClassCount;
			CHECK(Class == UObjectPtrTestClass::StaticClass());
			CHECK(&ObjectRef == &SourceRef);
			CHECK(UObjectPtrTestClass::StaticClass()->GetPackage() == ClassPackage);
		});
	UClass* Class = ObjectRef.ResolveObjectRefClass();
	CHECK(ClassCount == 1);

	int32 ResolvedCount = 0;
	auto ReferenceResolvedHandle1 = AddObjectHandleReferenceResolvedCallback([&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++ResolvedCount;
			CHECK(Obj1 == Object);
			CHECK(&ObjectRef == &SourceRef);
			CHECK(TestPackage == ObjectPackage);

		});
	ObjectRef.Resolve();
	CHECK(ResolvedCount == 1);

	int32 LoadCount = 0;
	auto ReferenceLoadedHandle1 = AddObjectHandleReferenceLoadedCallback([&](const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)
		{
			++LoadCount;
			CHECK(Obj1 == Object);
			CHECK(&ObjectRef == &SourceRef);
			CHECK(TestPackage == ObjectPackage);
		});

	//don't have a good to test this as the object are never loaded from disk in the tests
	OnReferenceLoaded(ObjectRef, TestPackage, Obj1);
	CHECK(LoadCount == 1);

	ON_SCOPE_EXIT
	{
		RemoveObjectHandleClassResolvedCallback(ClassResolvedHandle1);
		RemoveObjectHandleReferenceResolvedCallback(ReferenceResolvedHandle1);
		RemoveObjectHandleReferenceLoadedCallback(ReferenceLoadedHandle1);
	};
	
}
#endif
#endif
