// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"

#include "CoreMinimal.h"
#include "UObject/ObjectHandle.h"
#include "Misc/ScopeLock.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
inline FObjectHandle MakeUnresolvedHandle(const UObject* Obj)
{
	UE::CoreUObject::Private::FPackedObjectRef PackedObjectRef = UE::CoreUObject::Private::MakePackedObjectRef(Obj);
	return FObjectHandle{ PackedObjectRef.EncodedRef };
}

#endif

class FObjectRefTrackingTestBase
{
public:
	uint32 GetNumResolves() const { return NumResolves; }
	uint32 GetNumFailedResolves() const { return NumFailedResolves; }
	uint32 GetNumReads() const { return NumReads; }
	uint32 GetNumReads(const UObject* const Obj)
	{
		const auto* N = NumReadsPerObject.Find(Obj);
		return N ? *N : 0;
	}
	TMap<const UObject* const, uint32>& GetNumReadsPerObject()
	{
		return NumReadsPerObject;
	}
	
	struct FSnapshotObjectRefMetrics
	{
	public:
		FSnapshotObjectRefMetrics(FObjectRefTrackingTestBase& InTest)
		: Test(InTest)
		, OriginalNumResolves(Test.GetNumResolves())
		, OriginalNumFailedResolves(Test.GetNumFailedResolves())
		, OriginalNumReads(Test.GetNumReads())
		, OriginalNumReadsPerObject(Test.GetNumReadsPerObject())
		{
			Test.InstallCallbacks();
		}

		~FSnapshotObjectRefMetrics()
		{
			Test.RemoveCallbacks();
			Test.NumReadsPerObject.Empty();
		}

		void TestNumResolves(const TCHAR* What, uint32 ExpectedDelta)
		{
#if UE_WITH_OBJECT_HANDLE_TRACKING
			TEST_EQUAL(What, OriginalNumResolves + ExpectedDelta, Test.GetNumResolves());
#endif
		}

		void TestNumFailedResolves(const TCHAR* What, uint32 ExpectedDelta)
		{
#if UE_WITH_OBJECT_HANDLE_TRACKING
			TEST_EQUAL(What, OriginalNumFailedResolves + ExpectedDelta, Test.GetNumFailedResolves());
#endif
		}

		void TestNumReads(const TCHAR* What, uint32 ExpectedDelta, bool bAllowAdditionalReads = false)
		{
#if UE_WITH_OBJECT_HANDLE_TRACKING
			bool bValue = false;
			if (bAllowAdditionalReads)
			{
				INFO(What);
				CHECK(Test.GetNumReads() >= OriginalNumReads + ExpectedDelta);
			}
			else
			{
				bValue = OriginalNumReads + ExpectedDelta == Test.GetNumReads();
				TEST_TRUE(What, bValue);
			}
#endif
		}

		void TestNumReads(const TCHAR* What, const UObject* const Obj, uint32 ExpectedDelta, bool bAllowAdditionalReads = false)
		{
#if UE_WITH_OBJECT_HANDLE_TRACKING
			bool bValue = false;
			if (bAllowAdditionalReads)
			{
				INFO(What);
				CHECK(Test.GetNumReads(Obj) >= OriginalNumReadsPerObject.FindOrAdd(Obj, 0) + ExpectedDelta);
			}
			else
			{
				bValue = OriginalNumReadsPerObject.FindOrAdd(Obj, 0) + ExpectedDelta == Test.GetNumReads(Obj);
				TEST_TRUE(What, bValue);
			}
#endif
		}
		
	private:
		FObjectRefTrackingTestBase& Test;
		uint32 OriginalNumResolves;
		uint32 OriginalNumFailedResolves;
		uint32 OriginalNumReads;
		TMap<const UObject* const, uint32> OriginalNumReadsPerObject;
	};

private:
#if UE_WITH_OBJECT_HANDLE_TRACKING
	static void OnRefResolved(const FObjectRef& ObjectRef, UPackage* Pkg, UObject* Obj)
	{
		NumResolves++;
		if (!ObjectRef.IsNull() && !Obj)
		{
			NumFailedResolves++;
		}
	}
	static void OnRefRead(const TArrayView<const UObject* const>& Objects)
	{
		NumReads++;
		for (auto Obj : Objects)
		{
			++NumReadsPerObject.FindOrAdd(Obj, 0);
		}
	}
#endif
	
	static void InstallCallbacks()
	{
#if UE_WITH_OBJECT_HANDLE_TRACKING
		ResolvedCallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback(OnRefResolved);
		HandleReadCallbackHandle = UE::CoreUObject::AddObjectHandleReadCallback(OnRefRead);
#endif
	}

	static void RemoveCallbacks()
	{
#if UE_WITH_OBJECT_HANDLE_TRACKING
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(ResolvedCallbackHandle);
		UE::CoreUObject::RemoveObjectHandleReadCallback(HandleReadCallbackHandle);
#endif
	}

#if UE_WITH_OBJECT_HANDLE_TRACKING
	static UE::CoreUObject::FObjectHandleTrackingCallbackId ResolvedCallbackHandle;
	static UE::CoreUObject::FObjectHandleTrackingCallbackId HandleReadCallbackHandle;
#endif
	static thread_local uint32 NumResolves;
	static thread_local uint32 NumFailedResolves;
	static thread_local uint32 NumReads;
	static thread_local TMap<const UObject* const, uint32> NumReadsPerObject;
};

#endif
