// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "OptionalPropertyTestObject.h"

#include "Tests/TestHarnessAdapter.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UObjectGlobals.h"

struct FOptionalTestObject
{
    UOptionalPropertyTestObject* Obj;
    FOptionalProperty* StringProperty;
    FOptionalProperty* TextProperty;
    FOptionalProperty* NameProperty;
    FOptionalProperty* IntProperty;

    FOptionalTestObject()
    {
        const FName TestPackageName("/Engine/TestPackage");
        UPackage* TestPackage = NewObject<UPackage>(nullptr, TestPackageName, RF_Transient);
        Obj = NewObject<UOptionalPropertyTestObject>(TestPackage);

        UClass* Class = Obj->GetClass();
        StringProperty = CastField<FOptionalProperty>(Class->FindPropertyByName("OptionalString"));
        REQUIRE(StringProperty != nullptr);
        TextProperty = CastField<FOptionalProperty>(Class->FindPropertyByName("OptionalText"));
        REQUIRE(TextProperty != nullptr);
        NameProperty = CastField<FOptionalProperty>(Class->FindPropertyByName("OptionalName"));
        REQUIRE(NameProperty != nullptr);
        IntProperty = CastField<FOptionalProperty>(Class->FindPropertyByName("OptionalInt"));
        REQUIRE(IntProperty != nullptr);
    }

    bool IsValid()
    {
        CHECK(StringProperty != nullptr);
        CHECK(TextProperty != nullptr);
        CHECK(NameProperty != nullptr);
        CHECK(IntProperty != nullptr);
        return true;
    }
};

TEST_CASE_NAMED(FOptionalPropertyTestSize, "UE::CoreUObject::OptionalProperty::Size", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    CHECK(TestData.StringProperty->GetSize() == sizeof(TestData.Obj->OptionalString));
    CHECK(TestData.TextProperty->GetSize() == sizeof(TestData.Obj->OptionalText));
    CHECK(TestData.NameProperty->GetSize() == sizeof(TestData.Obj->OptionalName));
    CHECK(TestData.IntProperty->GetSize() == sizeof(TestData.Obj->OptionalInt));
}

TEST_CASE_NAMED(FOptionalPropertyTestInitializeValue, "UE::CoreUObject::OptionalProperty::InitializeValue", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    uint8 OptionalStringStorage[sizeof(TOptional<FString>)];
    uint8 OptionalTextStorage[sizeof(TOptional<FText>)];
    uint8 OptionalNameStorage[sizeof(TOptional<FName>)];
    uint8 OptionalIntStorage[sizeof(TOptional<int32>)];

    TestData.StringProperty->InitializeValue(OptionalStringStorage);
    TestData.TextProperty->InitializeValue(OptionalTextStorage);
    TestData.NameProperty->InitializeValue(OptionalNameStorage);
    TestData.IntProperty->InitializeValue(OptionalIntStorage);

    CHECK_FALSE(reinterpret_cast<TOptional<FString>*>(OptionalStringStorage)->IsSet());
    CHECK_FALSE(reinterpret_cast<TOptional<FText>*>(OptionalTextStorage)->IsSet());
    CHECK_FALSE(reinterpret_cast<TOptional<FName>*>(OptionalNameStorage)->IsSet());
    CHECK_FALSE(reinterpret_cast<TOptional<int32>*>(OptionalIntStorage)->IsSet());
}

TEST_CASE_NAMED(FOptionalPropertyTestClearValue, "UE::CoreUObject::OptionalProperty::ClearValue", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    TestData.Obj->OptionalString.Emplace(TEXT("Optional"));
    TestData.Obj->OptionalText.Emplace(FText::FromStringView(TEXTVIEW("Optional")));
    TestData.Obj->OptionalName.Emplace(TEXT("Optional"));
    TestData.Obj->OptionalInt.Emplace(42);

    TestData.StringProperty->ClearValue(&TestData.Obj->OptionalString);
    CHECK_FALSE(TestData.Obj->OptionalString.IsSet());
    TestData.TextProperty->ClearValue(&TestData.Obj->OptionalText);
    CHECK_FALSE(TestData.Obj->OptionalText.IsSet());
    TestData.NameProperty->ClearValue(&TestData.Obj->OptionalName);
    CHECK_FALSE(TestData.Obj->OptionalName.IsSet());
    TestData.IntProperty->ClearValue(&TestData.Obj->OptionalInt);
    CHECK_FALSE(TestData.Obj->OptionalInt.IsSet());
}

TEST_CASE_NAMED(FOptionalPropertyTestCopyValueIn, "UE::CoreUObject::OptionalProperty::CopyValueIn", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    TOptional<FString> OptString(FString("Optional"));
    TOptional<FText> OptText(FText::FromStringView(TEXTVIEW("Optional")));
    TOptional<FName> OptName("Optional");
    TOptional<int32> OptInt(58);

    TestData.StringProperty->CopySingleValue(&TestData.Obj->OptionalString, &OptString);
    TestData.TextProperty->CopySingleValue(&TestData.Obj->OptionalText, &OptText);
    TestData.NameProperty->CopySingleValue(&TestData.Obj->OptionalName, &OptName);
    TestData.IntProperty->CopySingleValue(&TestData.Obj->OptionalInt, &OptInt);

    CHECK(OptString.IsSet());
    CHECK(TestData.Obj->OptionalString.IsSet());
    CHECK(TestData.Obj->OptionalString.Get(FString()) == OptString.GetValue());
    CHECK(OptText.IsSet());
    CHECK(TestData.Obj->OptionalText.IsSet());
    CHECK(TestData.Obj->OptionalText.Get(FText::GetEmpty()).EqualTo(OptText.GetValue()));
    CHECK(OptName.IsSet());
    CHECK(TestData.Obj->OptionalName.IsSet());
    CHECK(TestData.Obj->OptionalName.Get(FName()) == OptName.GetValue());
    CHECK(OptInt.IsSet());
    CHECK(TestData.Obj->OptionalInt.IsSet());
    CHECK(TestData.Obj->OptionalInt.Get(0) == OptInt.GetValue());
}

TEST_CASE_NAMED(FOptionalPropertyTestCopyValueOut, "UE::CoreUObject::OptionalProperty::CopyValueOut", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    TOptional<FString> OptString(FString("Optional"));
    TOptional<FText> OptText(FText::FromStringView(TEXTVIEW("Optional")));
    TOptional<FName> OptName("Optional");
    TOptional<int32> OptInt(58);

    TestData.StringProperty->CopySingleValue(&TestData.Obj->OptionalString, &OptString);
    TestData.TextProperty->CopySingleValue(&TestData.Obj->OptionalText, &OptText);
    TestData.NameProperty->CopySingleValue(&TestData.Obj->OptionalName, &OptName);
    TestData.IntProperty->CopySingleValue(&TestData.Obj->OptionalInt, &OptInt);

    CHECK(OptString.IsSet());
    CHECK(TestData.Obj->OptionalString.IsSet());
    CHECK(TestData.Obj->OptionalString.Get(FString()) == OptString.GetValue());
    CHECK(OptText.IsSet());
    CHECK(TestData.Obj->OptionalText.IsSet());
    CHECK(TestData.Obj->OptionalText.Get(FText::GetEmpty()).EqualTo(OptText.GetValue()));
    CHECK(OptName.IsSet());
    CHECK(TestData.Obj->OptionalName.IsSet());
    CHECK(TestData.Obj->OptionalName.Get(FName()) == OptName.GetValue());
    CHECK(OptInt.IsSet());
    CHECK(TestData.Obj->OptionalInt.IsSet());
    CHECK(TestData.Obj->OptionalInt.Get(0) == OptInt.GetValue());
}

TEST_CASE_NAMED(FOptionalPropertyTestIdentical, "UE::CoreUObject::OptionalProperty::Identical", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    TOptional<FString> UnsetOptString;
    TOptional<FText> UnsetOptText;
    TOptional<FName> UnsetOptName;
    TOptional<int32> UnsetOptInt;

    CHECK(TestData.StringProperty->Identical(&TestData.Obj->OptionalString, &UnsetOptString, PPF_None));
    CHECK(TestData.TextProperty->Identical(&TestData.Obj->OptionalText, &UnsetOptText, PPF_None));
    CHECK(TestData.NameProperty->Identical(&TestData.Obj->OptionalName, &UnsetOptName, PPF_None));
    CHECK(TestData.IntProperty->Identical(&TestData.Obj->OptionalInt, &UnsetOptInt, PPF_None));

    TOptional<FString> OptString(FString("Optional"));
    TOptional<FText> OptText(FText::FromStringView(TEXTVIEW("Optional")));
    TOptional<FName> OptName("Optional");
    TOptional<int32> OptInt(58);

    CHECK_FALSE(TestData.StringProperty->Identical(&TestData.Obj->OptionalString, &OptString, PPF_None));
    CHECK_FALSE(TestData.TextProperty->Identical(&TestData.Obj->OptionalText, &OptText, PPF_None));
    CHECK_FALSE(TestData.NameProperty->Identical(&TestData.Obj->OptionalName, &OptName, PPF_None));
    CHECK_FALSE(TestData.IntProperty->Identical(&TestData.Obj->OptionalInt, &OptInt, PPF_None));

    TestData.Obj->OptionalString = OptString;
    TestData.Obj->OptionalText = OptText;
    TestData.Obj->OptionalName = OptName;
    TestData.Obj->OptionalInt = OptInt;

    CHECK(TestData.StringProperty->Identical(&TestData.Obj->OptionalString, &OptString, PPF_None));
    CHECK(TestData.TextProperty->Identical(&TestData.Obj->OptionalText, &OptText, PPF_None));
    CHECK(TestData.NameProperty->Identical(&TestData.Obj->OptionalName, &OptName, PPF_None));
    CHECK(TestData.IntProperty->Identical(&TestData.Obj->OptionalInt, &OptInt, PPF_None));
    
    CHECK_FALSE(TestData.StringProperty->Identical(&TestData.Obj->OptionalString, &UnsetOptString, PPF_None));
    CHECK_FALSE(TestData.TextProperty->Identical(&TestData.Obj->OptionalText, &UnsetOptText, PPF_None));
    CHECK_FALSE(TestData.NameProperty->Identical(&TestData.Obj->OptionalName, &UnsetOptName, PPF_None));
    CHECK_FALSE(TestData.IntProperty->Identical(&TestData.Obj->OptionalInt, &UnsetOptInt, PPF_None));
}

TEST_CASE_NAMED(FOptionalPropertyTestGetValueTypeHash, "UE::CoreUObject::OptionalProperty::GetValueTypeHash", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
    REQUIRE(TestData.IsValid());

    TOptional<FString> OptString(FString("Optional"));
    // GetTypeHash is undefined for FText 
    // TOptional<FText> OptText(FText::FromStringView(TEXTVIEW("Optional")));
    TOptional<FName> OptName("Optional");
    TOptional<int32> OptInt(93);

    CHECK(GetTypeHash(OptString) == TestData.StringProperty->GetValueTypeHash(&OptString));
    // GetTypeHash is undefined for FText 
    // CHECK(GetTypeHash(OptText) == TestData.TextProperty->GetValueTypeHash(&OptText));
    CHECK_FALSE(TestData.TextProperty->HasAllPropertyFlags(CPF_HasGetValueTypeHash));
    CHECK(GetTypeHash(OptName) == TestData.NameProperty->GetValueTypeHash(&OptName));
    CHECK(GetTypeHash(OptInt) == TestData.IntProperty->GetValueTypeHash(&OptInt));
}

TEST_CASE_NAMED(FOptionalPropertyTestLayout, "UE::CoreUObject::OptionalProperty::OptionalPropertyLayout", "[Core][UObject][SmokeFilter]")
{
	FOptionalTestObject TestData = FOptionalTestObject();
	REQUIRE(TestData.IsValid());

	FOptionalPropertyLayout StringPropertyLayout(TestData.StringProperty->GetValueProperty());
	FOptionalPropertyLayout TextPropertyLayout(TestData.TextProperty->GetValueProperty());
	FOptionalPropertyLayout NamePropertyLayout(TestData.NameProperty->GetValueProperty());
	FOptionalPropertyLayout IntPropertyLayout(TestData.IntProperty->GetValueProperty());

	CHECK_FALSE(StringPropertyLayout.IsSet(&TestData.Obj->OptionalString));
	CHECK_FALSE(TextPropertyLayout.IsSet(&TestData.Obj->OptionalText));
	CHECK_FALSE(NamePropertyLayout.IsSet(&TestData.Obj->OptionalName));
	CHECK_FALSE(IntPropertyLayout.IsSet(&TestData.Obj->OptionalInt));

	FString* InnerString = (FString*)StringPropertyLayout.MarkSetAndGetInitializedValuePointerToReplace(&TestData.Obj->OptionalString);
	CHECK(TestData.Obj->OptionalString.IsSet());
	*InnerString = TEXT("Optional");
	CHECK(TestData.Obj->OptionalString.GetValue() == TEXT("Optional"));

	FText* InnerText = (FText*)TextPropertyLayout.MarkSetAndGetInitializedValuePointerToReplace(&TestData.Obj->OptionalText);
	CHECK(TestData.Obj->OptionalText.IsSet());
	*InnerText = FText::FromStringView(TEXTVIEW("Optional"));
	CHECK(TestData.Obj->OptionalText.GetValue().ToString() == TEXT("Optional"));

	FName* InnerName = (FName*)NamePropertyLayout.MarkSetAndGetInitializedValuePointerToReplace(&TestData.Obj->OptionalName);
	CHECK(TestData.Obj->OptionalName.IsSet());
	*InnerName = FName("Optional");
	CHECK(TestData.Obj->OptionalName.GetValue() == FName("Optional"));

	int32* InnerInt = (int32*)IntPropertyLayout.MarkSetAndGetInitializedValuePointerToReplace(&TestData.Obj->OptionalInt);
	CHECK(TestData.Obj->OptionalInt.IsSet());
	*InnerInt = 79;
	CHECK(TestData.Obj->OptionalInt.GetValue() == 79);

	StringPropertyLayout.MarkUnset(&TestData.Obj->OptionalString);
	TextPropertyLayout.MarkUnset(&TestData.Obj->OptionalText);
	NamePropertyLayout.MarkUnset(&TestData.Obj->OptionalName);
	IntPropertyLayout.MarkUnset(&TestData.Obj->OptionalInt);

	CHECK_FALSE(StringPropertyLayout.IsSet(&TestData.Obj->OptionalString));
	CHECK_FALSE(TextPropertyLayout.IsSet(&TestData.Obj->OptionalText));
	CHECK_FALSE(NamePropertyLayout.IsSet(&TestData.Obj->OptionalName));
	CHECK_FALSE(IntPropertyLayout.IsSet(&TestData.Obj->OptionalInt));
}

#endif