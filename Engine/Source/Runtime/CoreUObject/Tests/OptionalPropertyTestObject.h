// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "OptionalPropertyTestObject.generated.h"

#if WITH_TESTS

UCLASS()
class UOptionalPropertyTestObject : public UObject
{
    GENERATED_BODY()

public:
    UPROPERTY()
    TOptional<FString> OptionalString;

    UPROPERTY()
    TOptional<FText> OptionalText;

    UPROPERTY()
    TOptional<FName> OptionalName;

    UPROPERTY()
    TOptional<int32> OptionalInt;
};

#endif // WITH_TESTS
