// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h" //for WITH_METADATA

#if WITH_LOW_LEVEL_TESTS && WITH_METADATA

#include "TestHarness.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"

namespace UE::CoreObject::Private::Tests
{
	TEST_CASE("CoreUObject::UPackage::FMetaData", "[CoreUObject][Package][MetaData]")
	{
#if WITH_EDITORONLY_DATA
		// FMetaDataUtilities::FMoveMetadataHelperContext used to handle objects renaming with regards to metadata will only work if GIsEditor
		// is set, so we must set it here.
		TGuardValue<bool> GIsEditorGuard(GIsEditor, true);
#endif // WITH_EDITORONLY_DATA

		const TCHAR* ObjectValueKey = TEXT("ObjectValueKey");
		const TCHAR* ObjectValueValue = TEXT("ObjectValueValue");

		UPackage* NewPackage = CreatePackage(TEXT("/Temp/TestPackage"));
		TEST_NOT_NULL(TEXT("Should be able to create a new package"), NewPackage);

		UObject* NewObject = ::NewObject<UObjectRedirector>(NewPackage, MakeUniversallyUniqueObjectName(NewPackage, NAME_Object));
		TEST_NOT_NULL(TEXT("Should be able to create a new object"), NewObject);

		UObject* NewSubObject = ::NewObject<UObjectRedirector>(NewObject, MakeUniversallyUniqueObjectName(NewObject, NAME_Object));
		TEST_NOT_NULL(TEXT("Should be able to create a new sub object"), NewSubObject);

		FMetaData& NewPackageMetaData = NewPackage->GetMetaData();
		TEST_FALSE(TEXT("New object shouldn't have metadata values"), NewPackageMetaData.HasObjectValues(NewObject));
		TEST_FALSE(TEXT("New sub object shouldn't have metadata values"), NewPackageMetaData.HasObjectValues(NewSubObject));

		NewPackageMetaData.SetValue(NewObject, ObjectValueKey, ObjectValueValue);
		TEST_TRUE(TEXT("New object should have metadata values"), NewPackageMetaData.HasObjectValues(NewObject));

		NewPackageMetaData.SetValue(NewSubObject, ObjectValueKey, ObjectValueValue);
		TEST_TRUE(TEXT("New sub object should have metadata values"), NewPackageMetaData.HasObjectValues(NewObject));

		const FString* FoundObjectValue = NewPackageMetaData.FindValue(NewObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New object should have a valid metadata value"), FoundObjectValue);
		TEST_TRUE(TEXT("New object should have valid metadata value"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		FoundObjectValue = NewPackageMetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New sub object should have a valid metadata value"), FoundObjectValue);
		TEST_TRUE(TEXT("New sub object should have valid metadata value"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		const FName NewObjectName = MakeUniversallyUniqueObjectName(NewPackage, NAME_Object);
		const FString NewObjectNameString = NewObjectName.ToString();
		TEST_TRUE(TEXT("Should be able to rename new object"), NewObject->Rename(*NewObjectNameString, nullptr));

		FoundObjectValue = NewPackageMetaData.FindValue(NewObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New object should have a valid metadata value after rename"), FoundObjectValue);
		TEST_TRUE(TEXT("New object should have valid metadata value after rename"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		FoundObjectValue = NewPackageMetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New sub object should have a valid metadata value after parent rename"), FoundObjectValue);
		TEST_TRUE(TEXT("New sub object should have valid metadata value after parent rename"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		UPackage* NewPackage2 = CreatePackage(TEXT("/Temp/TestPackage1"));
		TEST_NOT_NULL(TEXT("Should be able to create a new package"), NewPackage2);
		TEST_TRUE(TEXT("Should be able to rename new object into another package"), NewObject->Rename(nullptr, NewPackage2));

		FoundObjectValue = NewPackageMetaData.FindValue(NewObject, ObjectValueKey);
		TEST_NULL(TEXT("New object should have an invalid metadata value after rename in original package metadata"), FoundObjectValue);

		FoundObjectValue = NewPackageMetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NULL(TEXT("New sub object should have a valid metadata value after parent rename in original package metadata"), FoundObjectValue);

		FMetaData& NewPackage2MetaData = NewPackage2->GetMetaData();
		FoundObjectValue = NewPackage2MetaData.FindValue(NewObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New object should have a valid metadata value in renamed package"), FoundObjectValue);
		TEST_TRUE(TEXT("New object should have valid metadata value in renamed pacjage"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		FoundObjectValue = NewPackage2MetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New sub object should have a valid metadata value in renamed package"), FoundObjectValue);
		TEST_TRUE(TEXT("New sub object should have valid metadata value in renamed package"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		TEST_TRUE(TEXT("Should be able to rename new sub object"), NewSubObject->Rename(*NewObjectNameString, nullptr));

		FoundObjectValue = NewPackage2MetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New sub object should have a valid metadata value after rename"), FoundObjectValue);
		TEST_TRUE(TEXT("New sub object should have valid metadata value after renamepackage"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);

		TEST_TRUE(TEXT("Should be able to rename new sub object"), NewSubObject->Rename(nullptr, NewPackage2));

		FoundObjectValue = NewPackage2MetaData.FindValue(NewSubObject, ObjectValueKey);
		TEST_NOT_NULL(TEXT("New object should have a valid metadata value after removing from parent"), FoundObjectValue);
		TEST_TRUE(TEXT("New object should have valid metadata value after removing from parent"), FoundObjectValue && *FoundObjectValue == ObjectValueValue);
	}
}

#endif // WITH_LOW_LEVEL_TESTS && WITH_METADATA