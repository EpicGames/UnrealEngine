// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS
#include "Interfaces/IPluginManager.h"
#include "TestHarness.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/ScopeExit.h"
#include "HAL/UnrealMemory.h"
#include "Templates/Function.h"
#include "Misc/PackageName.h"
#include "ObjectPtrTestClass.h"
#include "UObject/Package.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Class.h"
#include "Tests/WarnFilterScope.h"
#include "UObject/FastReferenceCollector.h"
#include "Containers/Ticker.h"


namespace UE::CoreUObject::Private
{
	extern bool bEnsureOnLeakedPackages;
}

namespace UE::CoreUObject::Private::Tests
{
	class MockReadOnlyFileHandle : public IFileHandle
	{
	public:

		MockReadOnlyFileHandle(const uint8* Data, int64 Size, int64 Pos)
			: Data(Data)
			, Size(Size)
			, Pos(Pos)
		{
		}

		virtual bool Seek(int64 NewPosition)
		{
			check(NewPosition < Size);
			Pos = NewPosition;
			return true;
		}

		virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0)
		{
			Pos = Size - NewPositionRelativeToEnd;
			return true;
		}

		virtual bool Read(uint8* Destination, int64 BytesToRead)
		{
			FMemory::Memcpy(Destination, Data + Pos, BytesToRead);
			return true;
		}

		virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override
		{
			FMemory::Memcpy(Destination, Data + Offset, BytesToRead);
			return true;
		}

		virtual int64 Tell() { return Pos; }
		virtual bool Write(const uint8* Source, int64 BytesToWrite) { return false; }
		virtual bool Flush(const bool bFullFlush = false){ return false; }
		virtual bool Truncate(int64 NewSize) { return false; }

		const uint8* Data;
		int64 Size;
		int64 Pos;
	};

	class MockPlatformFile : public IPlatformFile
	{
	public:

		MockPlatformFile(IPlatformFile& File)
			: PlatformFile(File)
		{
		}

		virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override { return false; }
		virtual IPlatformFile* GetLowerLevel() override { return &PlatformFile; }
		virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override { }
		virtual const TCHAR* GetName() const override { return TEXT("MockPlatformFile") ; }
		virtual bool FileExists(const TCHAR* Filename) override { return false; }
		virtual int64 FileSize(const TCHAR* Filename) override { return 0; }
		virtual bool DeleteFile(const TCHAR* Filename) override { return false; }
		virtual bool IsReadOnly(const TCHAR* Filename) override { return false; }
		virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override { return false; }
		virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override { return false; }
		virtual FDateTime GetTimeStamp(const TCHAR* Filename) override { return  FDateTime(); }
		virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override { }
		virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override { return FDateTime(); }
		virtual FString GetFilenameOnDisk(const TCHAR* Filename) override { return TEXT(""); }
		virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override { return nullptr; }
		virtual bool DirectoryExists(const TCHAR* Directory) override { return false; }
		virtual bool CreateDirectory(const TCHAR* Directory) override { return false; }
		virtual bool DeleteDirectory(const TCHAR* Directory) override { return false; }
		virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override { return FFileStatData(); }
		virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) { return false; }
		virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) { return false; }

		virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override 
		{
			TFunction<IFileHandle* ()>* Handle = Files.Find(Filename);
			if (Handle)
			{
				return (*Handle)();
			}
			return nullptr;
		}

		IPlatformFile& PlatformFile;
		TMap<FString, TFunction<IFileHandle* ()>> Files;
	};

	class FProcessor : public FSimpleReferenceProcessorBase
	{
	public:
		void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId, EGCTokenType, bool);
	};

	void FProcessor::HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId, EGCTokenType, bool)
	{
	}

	TEST_CASE("UE::CoreUObject::PluginHandler::LeakDetection")
	{
		const ANSICHAR* PluginText = R"(
		{ 
			"FileVersion": 3,
			"ExplicitlyLoaded" : true,
			"CanContainContent" : true
		 })";
		MockPlatformFile MockFile(FPlatformFileManager::Get().GetPlatformFile());
		MockFile.Files.Add(TEXT("MyTestPlugin.uplugin"), [PluginText]()
			{
				return new MockReadOnlyFileHandle((const uint8*)PluginText, FCStringAnsi::Strlen(PluginText), 0u );
			});

		FPlatformFileManager::Get().SetPlatformFile(MockFile);
		IConsoleVariable* RenamedLeakedPackages = IConsoleManager::Get().FindConsoleVariable(TEXT("PluginManager.LeakedAssetTrace.RenameLeakedPackages"));
		IConsoleVariable* LeakSeverity = IConsoleManager::Get().FindConsoleVariable(TEXT("PluginManager.LeakedAssetTrace.Severity"));
		CHECK(RenamedLeakedPackages);
		CHECK(LeakSeverity);
		bool OldRenameValue = RenamedLeakedPackages->GetBool();
		int OldSeverityValue = RenamedLeakedPackages->GetInt();
		RenamedLeakedPackages->Set(true);
		LeakSeverity->Set(0); //disabled leak logging
		ON_SCOPE_EXIT
		{
			RenamedLeakedPackages->Set(OldRenameValue);
			LeakSeverity->Set(OldSeverityValue);
			FPlatformFileManager::Get().RemovePlatformFile(&MockFile);
		};
		IPluginManager& PluginManager = IPluginManager::Get();
		CHECK(PluginManager.AddToPluginsList("MyTestPlugin.uplugin"));
		TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin("MyTestPlugin");
		CHECK(Plugin);
		CHECK(PluginManager.MountExplicitlyLoadedPlugin(TEXT("MyTestPlugin")));
		CHECK(FPackageName::MountPointExists(TEXT("/MyTestPlugin")));

		UPackage* PluginPackage = NewObject<UPackage>(nullptr, TEXT("/MyTestPlugin/MyPackage"), RF_Public);
		
		FName PluginPackageName = PluginPackage->GetFName();
		UObjectPtrTestClass* Obj = NewObject<UObjectPtrTestClass>(PluginPackage, TEXT("Obj"), RF_Public);

		UPackage* OtherPackage = NewObject<UPackage>(nullptr, TEXT("/Test/TestPackage"), RF_Public);
		UObjectPtrTestClassWithRef* RefObj = NewObject<UObjectPtrTestClassWithRef>(OtherPackage, TEXT("RefObj"), RF_Public);

		OtherPackage->AddToRoot(); 
		RefObj->AddToRoot();
		//don't root PluginPackage as rooted packages can not be renamed
		ON_SCOPE_EXIT
		{
			OtherPackage->RemoveFromRoot();
			RefObj->RemoveFromRoot();
		};
		RefObj->ObjectPtr = Obj; //add a reference

		FWarnFilterScope _([](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)
			{
				bool bFiltered = FCString::Strncmp(Message, TEXT("Marking leaking package"), FCString::Strlen(TEXT("Marking leaking package"))) == 0 && Verbosity == ELogVerbosity::Type::Warning && Category == TEXT("PluginHandlerLog");
				return bFiltered;
			});
		FText Reason;
		CHECK(PluginManager.UnmountExplicitlyLoadedPlugin(TEXT("MyTestPlugin"), &Reason));
		FTSTicker::GetCoreTicker().Tick(0.01f);
		CHECK(!FPackageName::MountPointExists(TEXT("/MyTestPlugin")));
		CHECK(PluginPackageName != PluginPackage->GetFName()); //package should have been renamed.
		CHECK(PluginPackage->HasAnyInternalFlags(EInternalObjectFlags::Garbage));
		CHECK(Obj->HasAnyInternalFlags(EInternalObjectFlags::Garbage));

		UPackage* PluginPackage2 = NewObject<UPackage>(nullptr, TEXT("/MyTestPlugin/MyPackage"), RF_Public);
		CHECK(PluginPackage != PluginPackage2);
	}

	TEST_CASE("UE::CoreUObject::PluginHandler::GarbageCollection")
	{
		const ANSICHAR* PluginText = R"(
		{ 
			"FileVersion": 3,
			"ExplicitlyLoaded" : true,
			"CanContainContent" : true
		 })";
		MockPlatformFile MockFile(FPlatformFileManager::Get().GetPlatformFile());
		MockFile.Files.Add(TEXT("MyTestPlugin.uplugin"), [PluginText]()
			{
				return new MockReadOnlyFileHandle((const uint8*)PluginText, FCStringAnsi::Strlen(PluginText), 0u);
			});

		FPlatformFileManager::Get().SetPlatformFile(MockFile);
		UE::CoreUObject::Private::bEnsureOnLeakedPackages = true;
		ON_SCOPE_EXIT
		{
			UE::CoreUObject::Private::bEnsureOnLeakedPackages = false;
			FPlatformFileManager::Get().RemovePlatformFile(&MockFile);
		};
		IPluginManager& PluginManager = IPluginManager::Get();
		CHECK(PluginManager.AddToPluginsList("MyTestPlugin.uplugin"));
		TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin("MyTestPlugin");
		CHECK(Plugin);
		CHECK(PluginManager.MountExplicitlyLoadedPlugin(TEXT("MyTestPlugin")));
		CHECK(FPackageName::MountPointExists(TEXT("/MyTestPlugin")));

		UPackage* PluginPackage = NewObject<UPackage>(nullptr, TEXT("/MyTestPlugin/MyPackage"), RF_Public);

		FName PluginPackageName = PluginPackage->GetFName();
		UObjectPtrTestClass* Obj = NewObject<UObjectPtrTestClass>(PluginPackage, TEXT("Obj"), RF_Public);
		UObjectPtrTestClassWithRef* RefObj = NewObject<UObjectPtrTestClassWithRef>(PluginPackage, TEXT("RefObj"), RF_Public);

		FTopLevelAssetPath AssetPath(Obj);
		CHECK(FindObject<UObjectPtrTestClass>(AssetPath));

		FText Reason;
		CHECK(PluginManager.UnmountExplicitlyLoadedPlugin(TEXT("MyTestPlugin"), &Reason));
		FTSTicker::GetCoreTicker().Tick(0.01f);
		CHECK(!FPackageName::MountPointExists(TEXT("/MyTestPlugin")));
		CHECK(!FindObject<UObjectPtrTestClass>(AssetPath));
	}
}
#endif
