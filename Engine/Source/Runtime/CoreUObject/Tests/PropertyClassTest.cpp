// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "ObjectPtrTestClass.h"
#include "UObject/Package.h"
#include "Misc/ScopeExit.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/ObjectHandleTracking.h"
#include "Tests/WarnFilterScope.h"
#include "ObjectRefTrackingTestBase.h"

TEST_CASE("UE::CoreUObject::FClassProperty::Identical")
{
#if UE_WITH_OBJECT_HANDLE_TRACKING
	auto CallbackHandle = UE::CoreUObject::AddObjectHandleReferenceResolvedCallback([](const FObjectRef&, UPackage*, UObject*)
		{
			FAIL("Unexpected resolve during CheckValidObject");
		});
	ON_SCOPE_EXIT
	{
		UE::CoreUObject::RemoveObjectHandleReferenceResolvedCallback(CallbackHandle);
	};
#endif
	UClass* Class = UObjectWithClassProperty::StaticClass();
	FObjectProperty* Property = CastField<FObjectProperty>(Class->FindPropertyByName(TEXT("ClassPtr")));
	REQUIRE(Property != nullptr);


	UPackage* TestPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/TestPackageName"), RF_Transient);
	TestPackage->AddToRoot();
	ON_SCOPE_EXIT
	{
		TestPackage->RemoveFromRoot();
	};
	UObjectWithClassProperty* Obj1 = NewObject<UObjectWithClassProperty>(TestPackage, TEXT("UObjectWithClassProperty1"));
	UObjectWithClassProperty* Obj2 = NewObject<UObjectWithClassProperty>(TestPackage, TEXT("UObjectWithClassProperty2"));
	Obj1->ClassPtr = UObjectWithClassProperty::StaticClass();

	const void* Value1 = Property->ContainerPtrToValuePtr<void>(Obj1);
	const void* Value2 = Property->ContainerPtrToValuePtr<void>(Obj2);
	
	//NonNull != nullptr
	CHECK(!Property->Identical(Value1, Value2, 0u));
	CHECK(!Property->Identical(Value2, Value1, 0u));

	Obj2->ClassPtr = Obj1->ClassPtr;
	CHECK(Property->Identical(Value1, Value2, 0u));
	CHECK(Property->Identical(Value2, Value1, 0u));
}


TEST_CASE("UE::CoreUObject::FClassProperty::GetCPPType")
{
	FObjectProperty* PtrProperty = CastField<FObjectProperty>(UObjectWithClassProperty::StaticClass()->FindPropertyByName(TEXT("ClassPtr")));
	FObjectProperty* RawProperty = CastField<FObjectProperty>(UObjectWithClassProperty::StaticClass()->FindPropertyByName(TEXT("ClassRaw")));
	FObjectProperty* SubClassProperty = CastField<FObjectProperty>(UObjectWithClassProperty::StaticClass()->FindPropertyByName(TEXT("SubClass")));

	CHECK(PtrProperty->GetClass() == RawProperty->GetClass());

	FString RawType = RawProperty->GetCPPType(nullptr, 0u);
	CHECK(RawType == TEXT("UClass*"));

	RawType = RawProperty->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr);
	CHECK(RawType == TEXT("UClass*"));

	FString PtrType = PtrProperty->GetCPPType(nullptr, 0u);
	CHECK(PtrType == TEXT("TObjectPtr<UClass>"));

	PtrType = PtrProperty->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr);
	CHECK(PtrType == TEXT("UClass*"));

	FString SubClassType = SubClassProperty->GetCPPType(nullptr, EPropertyExportCPPFlags::CPPF_NoTObjectPtr);
	CHECK(SubClassType == TEXT("TSubclassOf<UObjectPtrTestClass>"));

	SubClassType = SubClassProperty->GetCPPType(nullptr, 0u);
	CHECK(SubClassType == TEXT("TSubclassOf<UObjectPtrTestClass>"));
}
#endif
