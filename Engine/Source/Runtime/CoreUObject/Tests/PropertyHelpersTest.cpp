// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "Tests/WarnFilterScope.h"
#include "UObject/UnrealType.h"

// Suppress expected "ReadTerminatedToken: Bad quoted string" warnings within
// test cases that intentionally pass malformed quoted strings.
#define SUPPRESS_BAD_QUOTED_STRING_WARNING() \
	FWarnFilterScope UE_JOIN(WarnFilter, __LINE__)([](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category) \
	{ \
		return Category == TEXT("LogProperty") && Verbosity == ELogVerbosity::Warning; \
	})

TEST_CASE("UE::CoreUObject::FPropertyHelpers::ReadTerminatedToken", "[CoreUObject][Property]")
{
	// ---------------------------------------------------------------------------
	// Basic functionality
	// ---------------------------------------------------------------------------

	SECTION("Simple token reads until terminator")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("hello'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out == TEXT("hello"));
	}

	SECTION("Empty token when terminator is the first character")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out.IsEmpty());
	}

	SECTION("Returned pointer points to the terminator, not past it")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("abc|xyz");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('|'));
		REQUIRE(Result != nullptr);
		// Result should point at '|', so *(Result+1) == 'x'
		CHECK(*Result == TEXT('|'));
		CHECK(*(Result + 1) == TEXT('x'));
	}

	SECTION("Appends to existing Out content rather than replacing it")
	{
		FString Out = TEXT("prefix_");
		const TCHAR* Buffer = TEXT("suffix'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("prefix_suffix"));
	}

	SECTION("Works with a non-quote, non-apostrophe terminator")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("foo/bar|");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('|'));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('|'));
		CHECK(Out == TEXT("foo/bar"));
	}

	// ---------------------------------------------------------------------------
	// Quoted string handling
	// ---------------------------------------------------------------------------

	SECTION("Quoted string content is unquoted into the output")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("\"hello world\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("hello world"));
	}

	SECTION("Quoted string with escaped double quote")
	{
		FString Out;
		// Produces the literal buffer: "hello\"world"'
		const TCHAR* Buffer = TEXT("\"hello\\\"world\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("hello\"world"));
	}

	SECTION("Quoted string with escaped backslash")
	{
		FString Out;
		// Produces the literal buffer: "hello\\world"'
		const TCHAR* Buffer = TEXT("\"hello\\\\world\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("hello\\world"));
	}

	SECTION("Quoted string with escape sequences (newline, tab)")
	{
		FString Out;
		// Produces: "a\nb\tc"'
		const TCHAR* Buffer = TEXT("\"a\\nb\\tc\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("a\nb\tc"));
	}

	SECTION("Multiple consecutive quoted strings are all appended")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("\"foo\"\"bar\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("foobar"));
	}

	SECTION("Unquoted text before and after a quoted string")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("pre\"mid\"post'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("premidpost"));
	}

	// ---------------------------------------------------------------------------
	// Nested quoted strings: terminator inside quotes must not stop parsing
	// ---------------------------------------------------------------------------

	SECTION("Terminator inside a quoted string does not end the token")
	{
		FString Out;
		// Buffer: "hello'world"'
		// The ' inside the quotes is part of the string, not the terminator.
		const TCHAR* Buffer = TEXT("\"hello'world\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out == TEXT("hello'world"));
	}

	SECTION("Multiple terminators inside a quoted string, all treated as content")
	{
		FString Out;
		// Buffer: "a'b'c"'
		const TCHAR* Buffer = TEXT("\"a'b'c\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("a'b'c"));
	}

	SECTION("Unquoted text, then quoted string with terminators, then more text")
	{
		FString Out;
		// Buffer: before"a'b'c"after'
		const TCHAR* Buffer = TEXT("before\"a'b'c\"after'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out == TEXT("beforea'b'cafter"));
	}

	SECTION("Quoted string containing escaped terminator and real terminator after")
	{
		FString Out;
		// Buffer: "hello\'"'
		// \" inside is an escaped single quote; the last ' is the real terminator.
		const TCHAR* Buffer = TEXT("\"hello\\'\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out == TEXT("hello'"));
	}

	// ---------------------------------------------------------------------------
	// Failure cases
	// ---------------------------------------------------------------------------

	SECTION("Returns null when no terminator is found")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("hello");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		CHECK(Result == nullptr);
	}

	SECTION("Returns null when MaxTokenLen is exceeded before the terminator is found")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("hello'");
		int32 MaxLen = 3;
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''), &MaxLen);
		CHECK(Result == nullptr);
	}

	SECTION("Succeeds when token length equals MaxTokenLen exactly")
	{
		FString Out;
		const TCHAR* Buffer = TEXT("hello'");
		int32 MaxLen = 5; // "hello" is exactly 5 characters
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''), &MaxLen);
		REQUIRE(Result != nullptr);
		CHECK(Out == TEXT("hello"));
	}

	SECTION("Returns null for a malformed quoted string (missing closing quote)")
	{
		FString Out;
		// Buffer: "unclosed'  — no closing quote before the null terminator
		const TCHAR* Buffer = TEXT("\"unclosed'");
		SUPPRESS_BAD_QUOTED_STRING_WARNING();
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		CHECK(Result == nullptr);
	}

	SECTION("Returns null for a quoted string broken by a newline (no closing quote)")
	{
		FString Out;
		// A raw newline inside a quoted string is a stop character that prevents
		// finding the closing quote, so QuotedString fails.
		const TCHAR* Buffer = TEXT("\"line1\nline2\"'");
		SUPPRESS_BAD_QUOTED_STRING_WARNING();
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		CHECK(Result == nullptr);
	}

	// ---------------------------------------------------------------------------
	// FStringBuilderBase overload
	// ---------------------------------------------------------------------------

	SECTION("FStringBuilderBase overload - basic token")
	{
		TStringBuilder<64> Out;
		const TCHAR* Buffer = TEXT("hello'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(*Result == TEXT('\''));
		CHECK(Out.ToView() == TEXTVIEW("hello"));
	}

	SECTION("FStringBuilderBase overload - terminator inside quoted string does not end token")
	{
		TStringBuilder<64> Out;
		const TCHAR* Buffer = TEXT("\"a'b\"'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out.ToView() == TEXTVIEW("a'b"));
	}

	SECTION("FStringBuilderBase overload - returns null when no terminator found")
	{
		TStringBuilder<64> Out;
		const TCHAR* Buffer = TEXT("hello");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		CHECK(Result == nullptr);
	}

	SECTION("FStringBuilderBase overload - appends to existing builder content")
	{
		TStringBuilder<64> Out;
		Out << TEXT("existing_");
		const TCHAR* Buffer = TEXT("appended'");
		const TCHAR* Result = FPropertyHelpers::ReadTerminatedToken(Buffer, Out, TEXT('\''));
		REQUIRE(Result != nullptr);
		CHECK(Out.ToView() == TEXTVIEW("existing_appended"));
	}
}

#undef SUPPRESS_BAD_QUOTED_STRING_WARNING

#endif // WITH_LOW_LEVEL_TESTS
