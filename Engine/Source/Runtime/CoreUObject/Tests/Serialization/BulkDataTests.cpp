// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "IO/IoDispatcher.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/BulkData.h"
#include "Serialization/BulkDataScopedLock.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include <catch2/generators/catch_generators.hpp>

namespace UE
{

static FUniqueBuffer CreatePayload(uint64 Size, uint64 Seed = 0)
{
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Size);
	TArrayView<uint64> Values(static_cast<uint64*>(Buffer.GetData()), IntCastChecked<int32>(Size / sizeof(uint64)));

	uint64 Index = Seed;
	for (uint64& Value : Values)
	{
		Value = ++Index;
	}

	return Buffer;
};

static void CopyPayload(FBulkData& BulkData, FMemoryView Src)
{
	BulkData.Lock(LOCK_READ_WRITE);
	FMutableMemoryView Dst(BulkData.Realloc(Src.GetSize(), 1), Src.GetSize());
	Dst.CopyFrom(Src);
	BulkData.Unlock();
}

template<typename T>
bool TestBulkDataFlags(FBulkData& BulkData, uint32 Flags, T&& IsSet)
{
	bool bOk = false;

	if (IsSet() == false)
	{
		BulkData.SetBulkDataFlags(Flags);
		bOk = IsSet();
	}
	BulkData.ResetBulkDataFlags(0);

	return bOk;
}

class FIoDispatcherTestScope
{
	bool bIoDispatcherInitialized = false;
public:
	FIoDispatcherTestScope()
	{
		if (!FIoDispatcher::IsInitialized())
		{
			FIoDispatcher::Initialize();
			bIoDispatcherInitialized = true;
		}
		FIoDispatcher::InitializePostSettings();
	}
	
	~FIoDispatcherTestScope()
	{
		if (bIoDispatcherInitialized)
		{
			FIoDispatcher::Shutdown();
		}
	}
};

TEST_CASE("CoreUObject::Serialization::FBulkData::Basic", "[CoreUObject][Serialization]")
{
	FIoDispatcherTestScope _;

	SECTION("Default construction")
	{
		FBulkData BulkData;
		CHECK(BulkData.GetBulkDataFlags() == 0);
		CHECK(BulkData.GetBulkDataSize() == 0);
		CHECK(BulkData.GetBulkDataOffsetInFile() == -1);
		CHECK(BulkData.CanLoadFromDisk() == false);
		CHECK(BulkData.DoesExist() == false);
		CHECK(BulkData.IsBulkDataLoaded() == false);
	}
	
	SECTION("Lock empty")
	{
		FBulkData BulkData;
		CHECK(nullptr == BulkData.Lock(LOCK_READ_ONLY));
		BulkData.Unlock();
		CHECK(nullptr == BulkData.Lock(LOCK_READ_WRITE));
		BulkData.Unlock();
		CHECK(nullptr == BulkData.LockReadOnly());
		BulkData.Unlock();
	}
	
	SECTION("Create payload")
	{
		const int64 ExpectedPayloadSize = 1024;

		FBulkData BulkData;
		void* Payload = BulkData.Lock(LOCK_READ_WRITE);
		CHECK(Payload == nullptr);
		
		Payload = BulkData.Realloc(ExpectedPayloadSize, 1);
		CHECK(Payload != nullptr);
		BulkData.Unlock();
		
		CHECK(BulkData.GetBulkDataFlags() == 0);
		CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);
		CHECK(BulkData.GetBulkDataOffsetInFile() == -1);
	}
	
	SECTION("Get copy")
	{
		// Empty
		{
			void* Dst = nullptr;
			bool bDiscardInternalCopy = true;
			
			FBulkData BulkData;
			BulkData.GetCopy(&Dst, bDiscardInternalCopy); 

			CHECK(Dst == nullptr);
		}
		
		// Non empty
		{
			const int64 ExpectedPayloadSize = 64;

			FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
			FBulkData BulkData;
			CopyPayload(BulkData, Payload.GetView());
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			void* Dst = nullptr;
			bool bDiscardInternalCopy = false;
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			FMemoryView DstView(Dst, ExpectedPayloadSize);
			CHECK(DstView.EqualBytes(Payload.GetView()));
		}

		// Get copy and discard with non-allocatated destination buffer 
		{
			const int64 ExpectedPayloadSize = 2048;

			FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
			FBulkData BulkData;
			CopyPayload(BulkData, Payload.GetView());
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			void* Dst = nullptr;
			bool bDiscardInternalCopy = true;
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);

			CHECK(BulkData.IsBulkDataLoaded()); // Still loaded (not discardable)
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			FMemoryView DstView(Dst, ExpectedPayloadSize);
			CHECK(DstView.EqualBytes(Payload.GetView()));
			
			bDiscardInternalCopy = true;
			BulkData.SetBulkDataFlags(BULKDATA_AlwaysAllowDiscard);
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);
			
			CHECK(BulkData.IsBulkDataLoaded() == false);
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			DstView = FMemoryView(Dst, ExpectedPayloadSize);
			CHECK(DstView.EqualBytes(Payload.GetView()));
			FMemory::Free(Dst);
		}
		
		// Get copy and discard with allocatated destination buffer 
		{
			const int64 ExpectedPayloadSize = 512;
			
			FBulkData BulkData;
			{
				FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
				CopyPayload(BulkData, Payload.GetView());
			}

			FUniqueBuffer Payload = FUniqueBuffer::Alloc(ExpectedPayloadSize);
			
			bool bDiscardInternalCopy = true;
			void* Dst = Payload.GetData();
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);
			
			CHECK(BulkData.IsBulkDataLoaded());
			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);

			FMemoryView BulkDataView(BulkData.LockReadOnly(), BulkData.GetBulkDataSize());
			CHECK(BulkDataView.EqualBytes(Payload.GetView()));
			BulkData.Unlock();
		
			bDiscardInternalCopy = false;
			BulkData.SetBulkDataFlags(BULKDATA_AlwaysAllowDiscard);
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);
			
			CHECK(BulkData.IsBulkDataLoaded());
			
			bDiscardInternalCopy = true;
			BulkData.GetCopy(&Dst, bDiscardInternalCopy);
			
			CHECK(BulkData.IsBulkDataLoaded() == false);
		}
	}
	
	SECTION("Remove payload")
	{
		const int64 ExpectedPayloadSize = 256;

		FBulkData BulkData;
		FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
		CopyPayload(BulkData, Payload);
		CHECK(BulkData.IsBulkDataLoaded());

		BulkData.RemoveBulkData();
		CHECK(BulkData.IsBulkDataLoaded() == false);
	}
	
	SECTION("Flags")
	{
		FBulkData BulkData;
		
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_PayloadAtEndOfFile, [&BulkData]() { return !BulkData.IsInlined(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_PayloadInSeparateFile, [&BulkData]() { return BulkData.IsInSeparateFile(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_OptionalPayload, [&BulkData]() { return BulkData.IsOptional(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_DuplicateNonOptionalPayload, [&BulkData]() { return BulkData.IsDuplicateNonOptional(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_DataIsMemoryMapped, [&BulkData]() { return BulkData.IsDataMemoryMapped(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_SingleUse, [&BulkData]() { return BulkData.IsSingleUse(); }));
		CHECK(TestBulkDataFlags(BulkData, BULKDATA_UsesIoDispatcher, [&BulkData]() { return BulkData.IsUsingIODispatcher(); }));
	}
}

#if WITH_EDITORONLY_DATA

TEST_CASE("CoreUObject::Serialization::FBulkData::Serialize", "[CoreUObject][Serialization]")
{
	FIoDispatcherTestScope _;

	SECTION("Serialize to memory archive")
	{
		const int64 ExpectedPayloadSize = 128;
		
		FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
		FLargeMemoryWriter Ar;
		Ar.SetIsPersistent(true);

		{
			FBulkData BulkData;
			CopyPayload(BulkData, Payload);
			BulkData.Serialize(Ar, nullptr, false, 1, EFileRegionType::None); 
			CHECK(Ar.TotalSize() > BulkData.GetBulkDataSize()); // Bulk meta data + payload
		}

		{
			FMemoryReaderView ReaderAr(Ar.GetView());
			ReaderAr.SetIsPersistent(true);

			FBulkData BulkData;
			BulkData.Serialize(ReaderAr, nullptr, false, 1, EFileRegionType::None); 
			FMemoryView PayloadView(BulkData.LockReadOnly(), BulkData.GetBulkDataSize());

			CHECK(BulkData.GetBulkDataSize() == ExpectedPayloadSize);
			CHECK(BulkData.GetBulkDataOffsetInFile() == ReaderAr.Tell() - ExpectedPayloadSize);
			CHECK(BulkData.GetBulkDataFlags() == 0);
			CHECK(BulkData.IsInlined());
			CHECK(PayloadView.EqualBytes(Payload.GetView()));

			BulkData.Unlock();
		}
	}
	
	SECTION("Serialize to memory archive does not serialize invalid flags")
	{
		const int64 ExpectedPayloadSize = 1024;
		const uint32 InvalidFlags = uint32(BULKDATA_PayloadAtEndOfFile | BULKDATA_PayloadInSeparateFile | BULKDATA_WorkspaceDomainPayload);

		FUniqueBuffer Payload = CreatePayload(ExpectedPayloadSize);
		FLargeMemoryWriter Ar;
		Ar.SetIsPersistent(true);
		
		{
			FBulkData BulkData;
			BulkData.SetBulkDataFlags(InvalidFlags);
			BulkData.Serialize(Ar, nullptr, false, 1, EFileRegionType::None); 
			CHECK(BulkData.GetBulkDataFlags() == InvalidFlags);
			CHECK(BulkData.GetBulkDataOffsetInFile() == -1);
		}

		{
			FMemoryReaderView ReaderAr(Ar.GetView());
			ReaderAr.SetIsPersistent(true);

			FBulkData BulkData;
			BulkData.Serialize(Ar, nullptr, false, 1, EFileRegionType::None); 
			CHECK(BulkData.GetBulkDataFlags() == 0);
			CHECK(BulkData.IsInlined());
		}
	}
	
	SECTION("Serialize many to memory archive")
	{
		const int64 ExpectedPayloadSizes[] = {8, 16, 32, 64, 128, 256, 512, 1024, 2048};
		TArrayView<const int64> PayloadSizes = MakeArrayView<const int64>(ExpectedPayloadSizes, 9); 
		TArray<FUniqueBuffer> ExpectedPayloads;

		for (int32 Idx = 0; Idx < PayloadSizes.Num(); ++Idx)
		{
			ExpectedPayloads.Emplace(CreatePayload(PayloadSizes[Idx], Idx)); 
		}
		
		FLargeMemoryWriter Ar;
		Ar.SetIsPersistent(true);

		for (FUniqueBuffer& Payload : ExpectedPayloads)
		{
			FBulkData BulkData;
			CopyPayload(BulkData, Payload);
			BulkData.Serialize(Ar, nullptr, false, 1, EFileRegionType::None); 
			CHECK(BulkData.GetBulkDataOffsetInFile() == -1);
		}

		{
			FMemoryReaderView ReaderAr(Ar.GetView());
			ReaderAr.SetIsPersistent(true);

			for (const FUniqueBuffer& ExpectedPayload : ExpectedPayloads)
			{
				FBulkData BulkData;
				BulkData.Serialize(ReaderAr, nullptr, false, 1, EFileRegionType::None); 
				FMemoryView PayloadView(BulkData.LockReadOnly(), BulkData.GetBulkDataSize());

				CHECK(BulkData.GetBulkDataOffsetInFile() == (ReaderAr.Tell() - BulkData.GetBulkDataSize()));
				CHECK(BulkData.GetBulkDataFlags() == 0);
				CHECK(BulkData.IsInlined());
				CHECK(PayloadView.EqualBytes(ExpectedPayload.GetView()));

				BulkData.Unlock();
			}
		}
	}
	
	SECTION("Serialize compressed")
	{
		const int64 UncompressedPayloadSize = 4 << 20;
		FUniqueBuffer Payload = CreatePayload(UncompressedPayloadSize);
		
		FLargeMemoryWriter Ar;
		Ar.SetIsPersistent(true);

		{
			FBulkData BulkData;
			CopyPayload(BulkData, Payload);
			BulkData.SetBulkDataFlags(BULKDATA_SerializeCompressedZLIB);
			BulkData.Serialize(Ar, nullptr, false, 1, EFileRegionType::None);
		}

		{
			FMemoryReaderView ReaderAr(Ar.GetView());
			ReaderAr.SetIsPersistent(true);
			
			FBulkData BulkData;
			BulkData.Serialize(ReaderAr, nullptr, false, 1, EFileRegionType::None);

			CHECK(BulkData.IsBulkDataLoaded());
			CHECK(BulkData.IsStoredCompressedOnDisk());
			CHECK(BulkData.GetBulkDataFlags() == BULKDATA_SerializeCompressedZLIB);
			CHECK(BulkData.GetBulkDataSize() == UncompressedPayloadSize );
		}
	}

	SECTION("Reuse FBulkData for serialization")
	{
		// Zero size was chosen for the second payload, because there was a bug where the bulk data
		// state wouldn't be initialized correctly if the payload was empty, leaving stale state
		// from a previous serialization.
		//
		// This acts as a regression test for that bug.
		const int64 FullPayloadSize = 128;
		const int64 ZeroPayloadSize = 0;

		FUniqueBuffer Payload = CreatePayload(FullPayloadSize);

		FLargeMemoryWriter ArFullPayload;
		FLargeMemoryWriter ArZeroPayload;
		ArFullPayload.SetIsPersistent(true);
		ArZeroPayload.SetIsPersistent(true);

		// Write two different payloads with the same FBulkData
		{
			FBulkData BulkData;
			CopyPayload(BulkData, Payload);
			BulkData.Serialize(ArFullPayload, nullptr, false, 1, EFileRegionType::None);
			CHECK(ArFullPayload.TotalSize() > BulkData.GetBulkDataSize()); // Bulk meta data + payload

			// Remove the payload and zero ElementCount
			BulkData.RemoveBulkData();

			BulkData.Serialize(ArZeroPayload, nullptr, false, 1, EFileRegionType::None);
			CHECK(ArZeroPayload.TotalSize() > BulkData.GetBulkDataSize()); // Bulk meta data + payload
		}

		// Read the two payloads with the same FBulkData
		{
			FBulkData BulkData;

			{
				FMemoryReaderView ReaderArFullPayload(ArFullPayload.GetView());
				ReaderArFullPayload.SetIsPersistent(true);

				BulkData.Serialize(ReaderArFullPayload, nullptr, false, 1, EFileRegionType::None);
				FMemoryView PayloadView(BulkData.LockReadOnly(), BulkData.GetBulkDataSize());

				CHECK(BulkData.GetBulkDataSize() == FullPayloadSize);
				CHECK(BulkData.GetBulkDataOffsetInFile() == ReaderArFullPayload.Tell() - FullPayloadSize);
				CHECK(BulkData.GetBulkDataFlags() == 0);
				CHECK(BulkData.IsInlined());
				CHECK(PayloadView.EqualBytes(Payload.GetView()));

				BulkData.Unlock();
			}

			{
				FMemoryReaderView ReaderArZeroPayload(ArZeroPayload.GetView());
				ReaderArZeroPayload.SetIsPersistent(true);

				BulkData.Serialize(ReaderArZeroPayload, nullptr, false, 1, EFileRegionType::None);

				// Should be able to access these without locking
				CHECK(BulkData.GetBulkDataSize() == ZeroPayloadSize);
				CHECK(BulkData.GetBulkDataOffsetInFile() == ReaderArZeroPayload.Tell() - ZeroPayloadSize);
				CHECK(BulkData.GetBulkDataFlags() == 0);
				CHECK(BulkData.IsInlined());
			}
		}
	}
}

#endif // WITH_EDITORONLY_DATA

TEST_CASE("CoreUObject::Serialization::FBulkData::LockScope", "[CoreUObject][Serialization]")
{
	const int32 NumElements = 32;
	FByteBulkData BulkData;

	// Lock for write
	{
		BulkData.Lock(LOCK_READ_WRITE);
		BulkData.Realloc(32);
		BulkData.Unlock();

		{
			TBulkDataScopedWriteLock WriteLock(BulkData);
			REQUIRE(BulkData.IsLocked());

			uint8* DataPtr = WriteLock.GetData();

			REQUIRE(DataPtr != nullptr);

			REQUIRE(WriteLock.Num() == NumElements);
			REQUIRE(WriteLock.Num() == BulkData.GetElementCount());
			REQUIRE(WriteLock.GetAllocatedSize() == BulkData.GetBulkDataSize());

			REQUIRE(WriteLock.GetView().GetData() != nullptr);
			REQUIRE(WriteLock.GetView().GetData() == DataPtr);
			REQUIRE(WriteLock.GetView().Num() == BulkData.GetElementCount());

			for (int32 Index = 0; Index < NumElements; ++Index)
			{
				DataPtr[Index] = static_cast<uint8>(Index);
			}
		}

		REQUIRE_FALSE(BulkData.IsLocked());
	}

	// Lock for read
	{
		{
			TBulkDataScopedReadLock ReadLock(BulkData);
			REQUIRE(BulkData.IsLocked());

			const uint8* DataPtr = ReadLock.GetData();

			REQUIRE(DataPtr != nullptr);

			REQUIRE(ReadLock.Num() == NumElements);
			REQUIRE(ReadLock.Num() == BulkData.GetElementCount());
			REQUIRE(ReadLock.GetAllocatedSize() == BulkData.GetBulkDataSize());

			REQUIRE(ReadLock.GetView().GetData() != nullptr);
			REQUIRE(ReadLock.GetView().GetData() == DataPtr);
			REQUIRE(ReadLock.GetView().Num() == BulkData.GetElementCount());

			for (int32 Index = 0; Index < NumElements; ++Index)
			{
				REQUIRE(DataPtr[Index] == Index);
			}
		}

		REQUIRE_FALSE(BulkData.IsLocked());
	}
}

} // namespace UE

#endif // WITH_LOW_LEVEL_TESTS
