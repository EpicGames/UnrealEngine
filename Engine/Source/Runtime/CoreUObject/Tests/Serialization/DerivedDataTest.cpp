// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Serialization/DerivedData.h"

#include "Algo/Compare.h"
#include "HAL/Event.h"
#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/StringBuilder.h"
#include "TestHarness.h"

#if WITH_EDITORONLY_DATA
#include "Containers/SharedString.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"
#endif

namespace UE::DerivedDataTest::Private
{

static FSharedBuffer MakeSharedBuffer(uint64& Counter, uint64 Size)
{
	FUniqueBuffer Unique = FUniqueBuffer::Alloc(Size);
	TArrayView<uint64> Values(static_cast<uint64*>(Unique.GetData()), IntCastChecked<int32>(Size / sizeof(uint64)));
	uint64 Index = 0;
	for (uint64& Value : Values)
	{
		Value = ++Index * Size + Counter;
	}
	FMemory::Memset(Values.GetData() + Values.Num(), 0, Size - Values.Num() * sizeof(uint64));
	++Counter;
	return Unique.MoveToShared();
}

} // UE::DerivedDataTest::Private

namespace UE
{

TEST_CASE("CoreUObject::Serialization::DerivedData", "[CoreUObject][Serialization]")
{
	using namespace DerivedData;
	using namespace DerivedDataTest::Private;
	uint64 Counter = 0;

	SECTION("Null Request")
	{
		FDerivedDataIoRequest Request;
		CHECK(Request.IsNull());
		CHECK_FALSE(Request);
	}

	SECTION("Null")
	{
		const FDerivedData DerivedData;

		CHECK(DerivedData.IsNull());
		CHECK_FALSE(DerivedData);
		CHECK(DerivedData.GetFlags() == EDerivedDataFlags::None);
		CHECK(WriteToString<64>(DerivedData) == TEXTVIEW("Null"));
		CHECK(DerivedData.ReferenceEquals(FDerivedData::Null));
		CHECK(DerivedData.ReferenceHash() == FDerivedData::Null.ReferenceHash());
	#if WITH_EDITORONLY_DATA
		CHECK(DerivedData.GetName().IsEmpty());
	#endif

		FEventRef Event;

		FDerivedDataIoResponse Response;
		CHECK(Response.IsNull());
		CHECK_FALSE(Response);

		TArray<FDerivedDataIoRequest, TInlineAllocator<4>> Requests;
		{
			FDerivedDataIoBatch Batch;
			CHECK(Batch.IsEmpty());
			Requests.Add(Batch.Read(DerivedData, {}));
			Requests.Add(Batch.Cache(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, {}));
		#if WITH_EDITORONLY_DATA
			Requests.Add(Batch.Compress(DerivedData));
		#endif
			CHECK_FALSE(Batch.IsEmpty());
			Batch.Dispatch(Response, {}, [&Event] { Event->Trigger(); });
			CHECK(Batch.IsEmpty());
		}

		Event->Wait();

		CHECK_FALSE(Response.IsNull());
		CHECK(Response);

		CHECK(Response.Poll());
		CHECK(Response.Cancel());
		CHECK(Response.GetOverallStatus() == EDerivedDataIoStatus::Error);

		for (const FDerivedDataIoRequest& Request : Requests)
		{
			CHECK(Response.GetStatus(Request) == EDerivedDataIoStatus::Error);
			CHECK(Response.GetData(Request).IsNull());
			CHECK(Response.GetSize(Request) == 0);
		#if WITH_EDITORONLY_DATA
			CHECK(Response.GetHash(Request) == nullptr);
			CHECK(Response.GetCacheKey(Request) == nullptr);
			CHECK(Response.GetCacheValueId(Request) == nullptr);
			CHECK(Response.GetCompressedData(Request) == nullptr);
		#endif
		}
	}

#if WITH_EDITORONLY_DATA
	SECTION("CompositeBuffer")
	{
		constexpr uint64 HalfSize = 32 * 1024;
		const FCompositeBuffer Source(MakeSharedBuffer(Counter, HalfSize), MakeSharedBuffer(Counter, HalfSize));
		const FIoHash SourceHash = FIoHash::HashBuffer(Source);
		const FDerivedData DerivedData(TEXTVIEW("CompositeBuffer"), Source);
		const FDerivedData DerivedDataCopy(TEXTVIEW("CompositeBufferCopy"), Source.ToShared());

		CHECK_FALSE(DerivedData.IsNull());
		CHECK(DerivedData);
		CHECK(DerivedData.GetFlags() == EDerivedDataFlags::Required);
		CHECK(WriteToString<128>(DerivedData).ToView() == WriteToString<128>(TEXTVIEW("Buffer: Size "),
			HalfSize * 2, TEXTVIEW(" Hash "), SourceHash, TEXTVIEW(" for CompositeBuffer")));
		CHECK(DerivedData.ReferenceEquals(DerivedDataCopy));
		CHECK(DerivedData.ReferenceHash() == DerivedDataCopy.ReferenceHash());
		CHECK(DerivedData.GetName() == TEXTVIEW("CompositeBuffer"));

		FEventRef Event;

		FUniqueBuffer Target = FUniqueBuffer::Alloc(HalfSize);
		FDerivedDataIoResponse Response;
		TArray<FDerivedDataIoRequest, TInlineAllocator<7>> Requests;
		{
			FDerivedDataIoBatch Batch;
			Requests.Add(Batch.Read(DerivedData, {}));
			Requests.Add(Batch.Cache(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, FDerivedDataIoOptions(HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(HalfSize, HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(Target, HalfSize / 2)));
		#if WITH_EDITORONLY_DATA
			Requests.Add(Batch.Compress(DerivedData));
		#endif
			Batch.Dispatch(Response, {}, [&Event] { Event->Trigger(); });
		}
		Event->Wait();

		CHECK(Response.Poll());
		CHECK(Response.Cancel());
		CHECK(Response.GetOverallStatus() == EDerivedDataIoStatus::Ok);

		for (const FDerivedDataIoRequest& Request : Requests)
		{
			CHECK(Response.GetStatus(Request) == EDerivedDataIoStatus::Ok);
			const FIoHash* RequestHash = Response.GetHash(Request);
			CHECKED_IF(RequestHash)
			{
				CHECK(*RequestHash == SourceHash);
			}
			CHECK(Response.GetCacheKey(Request) == nullptr);
			CHECK(Response.GetCacheValueId(Request) == nullptr);
		}

		CHECK(Response.GetData(Requests[0]).GetSize() == HalfSize * 2);
		CHECK(Response.GetData(Requests[1]).IsNull());
		CHECK(Response.GetData(Requests[2]).IsNull());
		CHECK(Response.GetData(Requests[3]).IsNull());
		CHECK(Response.GetData(Requests[4]) == Source.GetSegments()[1]);
		CHECK(FCompositeBuffer(Response.GetData(Requests[5])).EqualBytes(Source.Mid(HalfSize / 2, HalfSize)));
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetData(Requests[6]).IsNull());
	#endif

		CHECK(Response.GetSize(Requests[0]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[1]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[2]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[3]) == HalfSize);
		CHECK(Response.GetSize(Requests[4]) == HalfSize);
		CHECK(Response.GetSize(Requests[5]) == HalfSize);
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetSize(Requests[6]) == HalfSize * 2);
	#endif

	#if WITH_EDITORONLY_DATA
		const FCompressedBuffer* Compressed = Response.GetCompressedData(Requests[6]);
		CHECKED_IF(Compressed)
		{
			CHECK(Compressed->GetRawHash() == SourceHash);
			CHECK(Compressed->GetRawSize() == HalfSize * 2);
		}
	#endif

		Response.Reset();
		CHECK(Response.IsNull());
		CHECK(Response.GetStatus(Requests[0]) == EDerivedDataIoStatus::Error);
		CHECK(Response.GetData(Requests[0]).IsNull());
	}

	SECTION("CompressedBuffer")
	{
		constexpr uint64 HalfSize = 16 * 1024;
		const FSharedBuffer RawSource = MakeSharedBuffer(Counter, HalfSize * 2);
		const FCompressedBuffer Source = FValue::Compress(RawSource, HalfSize).GetData();
		const FDerivedData DerivedData(TEXTVIEW("CompressedBuffer"), Source);
		const FDerivedData DerivedDataCopy(TEXTVIEW("CompressedBufferCopy"), Source);

		CHECK_FALSE(DerivedData.IsNull());
		CHECK(DerivedData);
		CHECK(DerivedData.GetFlags() == EDerivedDataFlags::Required);
		CHECK(WriteToString<128>(DerivedData).ToView() == WriteToString<128>(TEXTVIEW("Buffer: CompressedSize "),
			Source.GetCompressedSize(), TEXTVIEW(" Size "), Source.GetRawSize(), TEXTVIEW(" Hash "),
			Source.GetRawHash(), TEXTVIEW(" for CompressedBuffer")));
		CHECK(DerivedData.ReferenceEquals(DerivedDataCopy));
		CHECK(DerivedData.ReferenceHash() == DerivedDataCopy.ReferenceHash());
		CHECK(DerivedData.GetName() == TEXTVIEW("CompressedBuffer"));

		FEventRef Event;

		FUniqueBuffer Target = FUniqueBuffer::Alloc(HalfSize);
		FDerivedDataIoResponse Response;
		TArray<FDerivedDataIoRequest, TInlineAllocator<7>> Requests;
		{
			FDerivedDataIoBatch Batch;
			Requests.Add(Batch.Read(DerivedData, {}));
			Requests.Add(Batch.Cache(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, FDerivedDataIoOptions(HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(HalfSize, HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(Target, HalfSize / 2)));
		#if WITH_EDITORONLY_DATA
			Requests.Add(Batch.Compress(DerivedData));
		#endif
			Batch.Dispatch(Response, {}, [&Event] { Event->Trigger(); });
		}
		Event->Wait();

		CHECK(Response.Poll());
		CHECK(Response.Cancel());
		CHECK(Response.GetOverallStatus() == EDerivedDataIoStatus::Ok);

		for (const FDerivedDataIoRequest& Request : Requests)
		{
			CHECK(Response.GetStatus(Request) == EDerivedDataIoStatus::Ok);
			const FIoHash* RequestHash = Response.GetHash(Request);
			CHECKED_IF(RequestHash)
			{
				CHECK(*RequestHash == Source.GetRawHash());
			}
			CHECK(Response.GetCacheKey(Request) == nullptr);
			CHECK(Response.GetCacheValueId(Request) == nullptr);
		}

		CHECK(Response.GetData(Requests[0]).GetSize() == HalfSize * 2);
		CHECK(Response.GetData(Requests[1]).IsNull());
		CHECK(Response.GetData(Requests[2]).IsNull());
		CHECK(Response.GetData(Requests[3]).IsNull());
		CHECK(Response.GetData(Requests[4]).GetView().EqualBytes(RawSource.GetView().Right(HalfSize)));
		CHECK(Response.GetData(Requests[5]).GetView().EqualBytes(RawSource.GetView().Mid(HalfSize / 2, HalfSize)));
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetData(Requests[6]).IsNull());
	#endif

		CHECK(Response.GetSize(Requests[0]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[1]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[2]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[3]) == HalfSize);
		CHECK(Response.GetSize(Requests[4]) == HalfSize);
		CHECK(Response.GetSize(Requests[5]) == HalfSize);
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetSize(Requests[6]) == HalfSize * 2);
	#endif

	#if WITH_EDITORONLY_DATA
		const FCompressedBuffer* Compressed = Response.GetCompressedData(Requests[6]);
		CHECKED_IF(Compressed)
		{
			CHECK(Compressed->GetRawHash() == Source.GetRawHash());
			CHECK(Compressed->GetRawSize() == Source.GetRawSize());
			CHECK(Compressed->GetCompressedSize() == Source.GetCompressedSize());
			CHECK(Algo::Compare(Compressed->GetCompressed().GetSegments(), Source.GetCompressed().GetSegments()));
		}
	#endif
	}

	SECTION("CacheRecord")
	{
		constexpr uint64 HalfSize = 8 * 1024;
		const FSharedBuffer RawValue = MakeSharedBuffer(Counter, HalfSize * 2);
		const FValue Value = FValue::Compress(RawValue, HalfSize);
		const FCacheKey Key{FCacheBucket(ANSITEXTVIEW("Test")), Value.GetRawHash()};
		const FValueId ValueId = FValueId::FromName(ANSITEXTVIEW("Value"));

		FCacheRecordBuilder RecordBuilder(Key);
		RecordBuilder.AddValue(ValueId, Value);

		FRequestOwner PutOwner(EPriority::Blocking);
		GetCache().Put({{{TEXTVIEW("CacheRecord")}, RecordBuilder.Build()}}, PutOwner);
		PutOwner.Wait();

		const FDerivedData DerivedData(TEXTVIEW("CacheRecord"), Key, ValueId);
		const FDerivedData DerivedDataCopy(TEXTVIEW("CacheRecordCopy"), Key, ValueId);

		CHECK_FALSE(DerivedData.IsNull());
		CHECK(DerivedData);
		CHECK(DerivedData.GetFlags() == EDerivedDataFlags::Required);
		CHECK(WriteToString<128>(DerivedData).ToView() == WriteToString<128>(TEXTVIEW("Cache: Key "), Key,
			TEXTVIEW(" ID "), ValueId, TEXTVIEW(" for CacheRecord")));
		CHECK(DerivedData.ReferenceEquals(DerivedDataCopy));
		CHECK(DerivedData.ReferenceHash() == DerivedDataCopy.ReferenceHash());
		CHECK(DerivedData.GetName() == TEXTVIEW("CacheRecord"));

		FEventRef Event;

		FUniqueBuffer Target = FUniqueBuffer::Alloc(HalfSize);
		FDerivedDataIoResponse Response;
		TArray<FDerivedDataIoRequest, TInlineAllocator<7>> Requests;
		{
			FDerivedDataIoBatch Batch;
			Requests.Add(Batch.Read(DerivedData, {}));
			Requests.Add(Batch.Cache(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, FDerivedDataIoOptions(HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(HalfSize, HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(Target, HalfSize / 2)));
		#if WITH_EDITORONLY_DATA
			Requests.Add(Batch.Compress(DerivedData));
		#endif
			Batch.Dispatch(Response, {}, [&Event] { Event->Trigger(); });
		}
		Event->Wait();

		CHECK(Response.Poll());
		CHECK(Response.Cancel());
		CHECK(Response.GetOverallStatus() == EDerivedDataIoStatus::Ok);

		for (const FDerivedDataIoRequest& Request : Requests)
		{
			CHECK(Response.GetStatus(Request) == EDerivedDataIoStatus::Ok);

			const FIoHash* RequestHash = Response.GetHash(Request);
			CHECKED_IF(RequestHash)
			{
				CHECK(*RequestHash == Value.GetRawHash());
			}

			const FCacheKey* CacheKey = Response.GetCacheKey(Request);
			CHECKED_IF(CacheKey)
			{
				CHECK(*CacheKey == Key);
			}

			const FValueId* CacheValueId = Response.GetCacheValueId(Request);
			CHECKED_IF(CacheValueId)
			{
				CHECK(*CacheValueId == ValueId);
			}
		}

		CHECK(Response.GetData(Requests[0]).GetSize() == HalfSize * 2);
		CHECK(Response.GetData(Requests[1]).IsNull());
		CHECK(Response.GetData(Requests[2]).IsNull());
		CHECK(Response.GetData(Requests[3]).IsNull());
		CHECK(Response.GetData(Requests[4]).GetView().EqualBytes(RawValue.GetView().Right(HalfSize)));
		CHECK(Response.GetData(Requests[5]).GetView().EqualBytes(RawValue.GetView().Mid(HalfSize / 2, HalfSize)));
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetData(Requests[6]).IsNull());
	#endif

		CHECK(Response.GetSize(Requests[0]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[1]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[2]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[3]) == HalfSize);
		CHECK(Response.GetSize(Requests[4]) == HalfSize);
		CHECK(Response.GetSize(Requests[5]) == HalfSize);
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetSize(Requests[6]) == HalfSize * 2);
	#endif

	#if WITH_EDITORONLY_DATA
		const FCompressedBuffer* Compressed = Response.GetCompressedData(Requests[6]);
		CHECKED_IF(Compressed)
		{
			CHECK(Compressed->GetRawHash() == Value.GetRawHash());
			CHECK(Compressed->GetRawSize() == Value.GetRawSize());
		}
	#endif
	}

	SECTION("CacheValue")
	{
		constexpr uint64 HalfSize = 4 * 1024;
		const FSharedBuffer RawValue = MakeSharedBuffer(Counter, HalfSize * 2);
		const FValue Value = FValue::Compress(RawValue, HalfSize);
		const FCacheKey Key{FCacheBucket(ANSITEXTVIEW("Test")), Value.GetRawHash()};

		FRequestOwner PutOwner(EPriority::Blocking);
		GetCache().PutValue({{{TEXTVIEW("CacheValue")}, Key, Value}}, PutOwner);
		PutOwner.Wait();

		const FDerivedData DerivedData(TEXTVIEW("CacheValue"), Key);
		const FDerivedData DerivedDataCopy(TEXTVIEW("CacheValueCopy"), Key);

		CHECK_FALSE(DerivedData.IsNull());
		CHECK(DerivedData);
		CHECK(DerivedData.GetFlags() == EDerivedDataFlags::Required);
		CHECK(WriteToString<128>(DerivedData).ToView() == WriteToString<128>(TEXTVIEW("Cache: Key "), Key,
			TEXTVIEW(" for CacheValue")));
		CHECK(DerivedData.ReferenceEquals(DerivedDataCopy));
		CHECK(DerivedData.ReferenceHash() == DerivedDataCopy.ReferenceHash());
		CHECK(DerivedData.GetName() == TEXTVIEW("CacheValue"));

		FEventRef Event;

		FUniqueBuffer Target = FUniqueBuffer::Alloc(HalfSize);
		FDerivedDataIoResponse Response;
		TArray<FDerivedDataIoRequest, TInlineAllocator<7>> Requests;
		{
			FDerivedDataIoBatch Batch;
			Requests.Add(Batch.Read(DerivedData, {}));
			Requests.Add(Batch.Cache(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, {}));
			Requests.Add(Batch.Exists(DerivedData, FDerivedDataIoOptions(HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(HalfSize, HalfSize)));
			Requests.Add(Batch.Read(DerivedData, FDerivedDataIoOptions(Target, HalfSize / 2)));
		#if WITH_EDITORONLY_DATA
			Requests.Add(Batch.Compress(DerivedData));
		#endif
			Batch.Dispatch(Response, {}, [&Event] { Event->Trigger(); });
		}
		Event->Wait();

		CHECK(Response.Poll());
		CHECK(Response.Cancel());
		CHECK(Response.GetOverallStatus() == EDerivedDataIoStatus::Ok);

		for (const FDerivedDataIoRequest& Request : Requests)
		{
			CHECK(Response.GetStatus(Request) == EDerivedDataIoStatus::Ok);

			const FIoHash* RequestHash = Response.GetHash(Request);
			CHECKED_IF(RequestHash)
			{
				CHECK(*RequestHash == Value.GetRawHash());
			}

			const FCacheKey* CacheKey = Response.GetCacheKey(Request);
			CHECKED_IF(CacheKey)
			{
				CHECK(*CacheKey == Key);
			}

			CHECK(Response.GetCacheValueId(Request) == nullptr);
		}

		CHECK(Response.GetData(Requests[0]).GetSize() == HalfSize * 2);
		CHECK(Response.GetData(Requests[1]).IsNull());
		CHECK(Response.GetData(Requests[2]).IsNull());
		CHECK(Response.GetData(Requests[3]).IsNull());
		CHECK(Response.GetData(Requests[4]).GetView().EqualBytes(RawValue.GetView().Right(HalfSize)));
		CHECK(Response.GetData(Requests[5]).GetView().EqualBytes(RawValue.GetView().Mid(HalfSize / 2, HalfSize)));
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetData(Requests[6]).IsNull());
	#endif

		CHECK(Response.GetSize(Requests[0]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[1]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[2]) == HalfSize * 2);
		CHECK(Response.GetSize(Requests[3]) == HalfSize);
		CHECK(Response.GetSize(Requests[4]) == HalfSize);
		CHECK(Response.GetSize(Requests[5]) == HalfSize);
	#if WITH_EDITORONLY_DATA
		CHECK(Response.GetSize(Requests[6]) == HalfSize * 2);
	#endif

	#if WITH_EDITORONLY_DATA
		const FCompressedBuffer* Compressed = Response.GetCompressedData(Requests[6]);
		CHECKED_IF(Compressed)
		{
			CHECK(Compressed->GetRawHash() == Value.GetRawHash());
			CHECK(Compressed->GetRawSize() == Value.GetRawSize());
		}
	#endif
	}
#endif // WITH_EDITORONLY_DATA
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
