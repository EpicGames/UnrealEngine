// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS
#include "Misc/CoreMiscDefines.h" // WITH_EDITORONLY_DATA
#endif

#if WITH_LOW_LEVEL_TESTS && WITH_EDITORONLY_DATA

#include "Algo/AllOf.h"
#include "Async/ParallelFor.h"
#include "Compression/OodleDataCompression.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/BulkDataRegistry.h"
#include "Serialization/EditorBulkData.h"
#include "Serialization/EditorBulkDataReader.h"
#include "Serialization/EditorBulkDataWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "TestCommon/Expectations.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"

#include <catch2/generators/catch_generators.hpp>

namespace UE::Serialization
{

bool IsBulkDataRegistryEnabled()
{
#if WITH_EDITOR
	return IBulkDataRegistry::IsEnabled();
#else
	return false;
#endif //WITH_EDITOR 
}

/** Creates a buffer full of random data to make it easy to have something to test against. */
TUniquePtr<uint8[]> CreateRandomData(uint64 BufferSize)
{
	TUniquePtr<uint8[]> Buffer = MakeUnique<uint8[]>(BufferSize);

	for (uint64 Index = 0; Index < BufferSize; ++Index)
	{
		Buffer[Index] = (uint8)(FMath::Rand() % 255);
	}

	return Buffer;
}

/** Creates a FSharedBuffer full of random data to make it easy to have something to test against. */
FSharedBuffer CreateRandomPayload(int64 BufferSize)
{
	TUniquePtr<uint8[]> Data = CreateRandomData(BufferSize);

	return FSharedBuffer::TakeOwnership(Data.Release(), BufferSize, [](void* Ptr, uint64) { delete[](uint8*)Ptr; });
}

/** Creates a FSharedBuffer with semi random data. */
FSharedBuffer CreatePayload(int64 BufferSize, int64 Stride)
{
	TUniquePtr<uint8[]> Data = CreateRandomData(BufferSize);

	uint8 Value = (uint8)(FMath::Rand() % 255);
	int64 SpanCount = 0;

	for (int64 Index = 0; Index < BufferSize; ++Index)
	{
		Data[Index] = Value;

		if (++SpanCount >= Stride)
		{
			Value = (uint8)(FMath::Rand() % 255);
			SpanCount = 0;
		}
	}

	return FSharedBuffer::TakeOwnership(Data.Release(), BufferSize, [](void* Ptr, uint64) { delete[](uint8*)Ptr; });
}

bool CompareSharedBufferContents(const FSharedBuffer& LHS, const FSharedBuffer& RHS)
{
	if (LHS.GetSize() != RHS.GetSize())
	{
		return false;
	}

	return FMemory::Memcmp(LHS.GetData(), RHS.GetData(), LHS.GetSize()) == 0;
}

/**
 * This test creates a very basic FEditorBulkData object with in memory payload and validates that we are able to retrieve the
 * payload via both TFuture and callback methods. It then creates copies of the object and makes sure that we can get the payload from
 * the copies, even when the original source object has been reset.
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Basic", "[CoreUObject][Serialization]")
{
	const uint64 BufferSize = 1024;
	TUniquePtr<uint8[]> SourceBuffer = CreateRandomData(BufferSize);

	auto ValidateBulkData = [&SourceBuffer, BufferSize](const FEditorBulkData& BulkDataToValidate, const TCHAR* Label)
	{
		FSharedBuffer RetrievedBuffer = BulkDataToValidate.GetPayload().Get();
		CHECK((int64)RetrievedBuffer.GetSize() == BufferSize);
		CHECK(FMemory::Memcmp(SourceBuffer.Get(), RetrievedBuffer.GetData(), BufferSize) == 0);
	};

	// Create a basic bulkdata (but retain ownership of the buffer!)
	FEditorBulkData BulkData;
	BulkData.UpdatePayload(FSharedBuffer::MakeView(SourceBuffer.Get(), BufferSize));

	// Test accessing the data from the bulkdata object
	ValidateBulkData(BulkData, TEXT("Retrieved"));

	// Create a new bulkdata object via the copy constructor
	FEditorBulkData BulkDataCopy(BulkData);

	// Create a new bulkdata object and copy by assignment (note we assign some junk data that will get overwritten)
	FEditorBulkData BulkDataAssignment;
	BulkDataAssignment.UpdatePayload(FUniqueBuffer::Alloc(128).MoveToShared());
	BulkDataAssignment = BulkData;

	// Test both bulkdata objects
	ValidateBulkData(BulkDataCopy, TEXT("Copy Constructor"));
	ValidateBulkData(BulkDataAssignment, TEXT("Copy Assignment"));

	// Should not affect BulkDataAssignment/BulkDataCopy 
	BulkData.Reset();

	// Test both bulkdata objects again now that we reset the data
	ValidateBulkData(BulkDataCopy, TEXT("Copy Constructor (after data reset)"));
	ValidateBulkData(BulkDataAssignment, TEXT("Copy Assignment (after data reset)"));
}

/**
 * This test will validate how FEditorBulkData behaves when it has no associated payload and make sure
 * that our assumptions are correct.
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Empty", "[CoreUObject][Serialization]")
{
	auto Validate = [](const TCHAR* Id, const FEditorBulkData& BulkData)
	{
		// Validate the general accessors
		CHECK_EQUALS(FString::Printf(TEXT("(%s) Return value of ::GetBulkDataSize()"), Id), BulkData.GetPayloadSize(), (int64)0);
		CHECK_MESSAGE(FString::Printf(TEXT("(%s) Payload key is invalid"), Id), BulkData.GetPayloadId().IsZero());
		CHECK_FALSE_MESSAGE(FString::Printf(TEXT("(%s) Return value of ::DoesPayloadNeedLoading()"), Id), BulkData.DoesPayloadNeedLoading());

		// Validate the payload accessors
		FSharedBuffer Payload = BulkData.GetPayload().Get();
		CHECK_MESSAGE(FString::Printf(TEXT("(%s) The payload from the GetPayload TFuture is null"), Id), Payload.IsNull());

		FCompressedBuffer CompressedPayload = BulkData.GetCompressedPayload().Get();
		CHECK_MESSAGE(FString::Printf(TEXT("(%s) The payload from the GetCompressedPayload TFuture is null"), Id), Payload.IsNull());
	};

	FEditorBulkData DefaultBulkData;
	Validate(TEXT("DefaultBulkData"), DefaultBulkData);

	FEditorBulkData NullPayloadBulkData;
	NullPayloadBulkData.UpdatePayload(FSharedBuffer());
	Validate(TEXT("NullPayloadBulkData"), NullPayloadBulkData);

	FEditorBulkData ZeroLengthPayloadBulkData;
	ZeroLengthPayloadBulkData.UpdatePayload(FUniqueBuffer::Alloc(0).MoveToShared());
	Validate(TEXT("ZeroLengthPayloadBulkData"), ZeroLengthPayloadBulkData);
}

/**
 * Test the various methods for updating the payload that a FEditorBulkData owns via FSharedBuffer
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::UpdatePayloadSharedBuffer", "[CoreUObject][Serialization]")
{
	// Create a memory buffer of all zeros
	const int64 BufferSize = 1024;
	TUniquePtr<uint8[]> OriginalData = MakeUnique<uint8[]>(BufferSize);
	FMemory::Memzero(OriginalData.Get(), BufferSize);

	// Pass the buffer to to bulkdata but retain ownership
	FEditorBulkData BulkData;
	BulkData.UpdatePayload(FSharedBuffer::MakeView(OriginalData.Get(), BufferSize));

	// Access the payload, edit it and push it back into the bulkdata object
	{
		// The payload should be the same size and same contents as the original buffer but a different
		// memory address since we retained ownership in the TUniquePtr, so the bulkdata object should 
		// have created it's own copy.
		FSharedBuffer Payload = BulkData.GetPayload().Get();
		CHECK_EQUAL((int64)Payload.GetSize(), BufferSize);
		CHECK_NOT_EQUAL((uint8*)OriginalData.Get(), (uint8*)Payload.GetData());
		CHECK(FMemory::Memcmp(OriginalData.Get(), Payload.GetData(), Payload.GetSize()) == 0);

		// Make a copy of the payload that we can edit
		const uint8 NewValue = 255;
		FSharedBuffer EditedPayload;
		{
			FUniqueBuffer EditablePayload = FUniqueBuffer::Clone(Payload);
			FMemory::Memset(EditablePayload.GetData(), NewValue, EditablePayload.GetSize());
			EditedPayload = EditablePayload.MoveToShared();
		}

		// Update the bulkdata object with the new edited payload
		BulkData.UpdatePayload(EditedPayload);

		Payload = BulkData.GetPayload().Get();
		CHECK_EQUAL((int64)Payload.GetSize(), BufferSize);
		CHECK_EQUAL((uint8*)Payload.GetData(), (uint8*)EditedPayload.GetData());

		const bool bAllElementsCorrect = Algo::AllOf(TArrayView64<uint8>((uint8*)Payload.GetData(), (int64)Payload.GetSize()), [NewValue](uint8 Val)
			{ 
				return Val == NewValue; 
			});

		CHECK(bAllElementsCorrect);
	}

	{
		// Store the original data pointer address so we can test against it later, we should not actually use
		// this pointer though as once we pass it to the bulkdata object we cannot be sure what happens to it.
		uint8* OriginalDataPtr = OriginalData.Get();

		// Update the bulkdata object with the original data but this time we give ownership of the buffer to
		// the bulkdata object.
		BulkData.UpdatePayload(FSharedBuffer::TakeOwnership(OriginalData.Release(), BufferSize, [](void* Ptr, uint64) { delete[] (uint8*)Ptr; }));

		FSharedBuffer Payload = BulkData.GetPayload().Get();
		CHECK_EQUAL((int64)Payload.GetSize(), BufferSize);
		CHECK_EQUAL((uint8*)Payload.GetData(), OriginalDataPtr);

		// The original data was all zeros, so we can test for that to make sure that the contents are correct.
		const bool bAllElementsCorrect = Algo::AllOf(TArrayView64<uint8>((uint8*)Payload.GetData(), (int64)Payload.GetSize()), [](uint8 Val)
			{
				return Val == 0;
			});

		CHECK(bAllElementsCorrect);
	}
}

/** This test covers updating FEditorBulkData with a payload stored in FCompressedBuffer format */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::UpdatePayloadCompressedBuffer", "[CoreUObject][Serialization]")
{
	// Create a memory buffer
	const int64 BufferSize = 1024;
	const int64 BufferStride = 32;

	FSharedBuffer InitialPayload = CreatePayload(BufferSize, BufferStride);
	
	{
		FCompressedBuffer UncompressedPayload = FCompressedBuffer::Compress(InitialPayload, ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);

		FEditorBulkData BulkData;
		BulkData.UpdatePayload(UncompressedPayload, nullptr);

		FSharedBuffer BulkDataPayload = BulkData.GetPayload().Get();

		CHECK_EQUAL( UncompressedPayload.GetRawHash(), BulkData.GetPayloadId());
		CHECK(CompareSharedBufferContents(BulkDataPayload, BulkDataPayload));

		// Since the data was never compressed there is no reason that the data needed to be copied at any point
		// so the returned payload should be a reference to the original FSharedBuffer
		CHECK_EQUAL(BulkDataPayload.GetData(), InitialPayload.GetData());
	}

	{
		FCompressedBuffer CompressedPayload = FCompressedBuffer::Compress(InitialPayload, ECompressedBufferCompressor::Kraken, ECompressedBufferCompressionLevel::Fast);

		FEditorBulkData BulkData;
		BulkData.UpdatePayload(CompressedPayload, nullptr);

		FSharedBuffer BulkDataPayload = BulkData.GetPayload().Get();

		CHECK_EQUAL(CompressedPayload.GetRawHash(), BulkData.GetPayloadId());
		CHECK(CompareSharedBufferContents(BulkDataPayload, BulkDataPayload));

		// Since the data was compressed we will not have a reference to the original FSharedBuffer and should
		// expect different memory addresses.
		CHECK_NOT_EQUAL(BulkDataPayload.GetData(), InitialPayload.GetData());
	}
}

/**
 * This test will create a buffer, then serialize it to a FEditorBulkData object via FEditorBulkDataWriter.
 * Then we will serialize the FEditorBulkData object back to a second buffer and compare the results.
 * If the reader and writers are working then the ReplicatedBuffer should be the same as the original SourceBuffer.
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Reader/Writer", "[CoreUObject][Serialization]")
{
	const int64 BufferSize = 1024;

	TUniquePtr<uint8[]> SourceBuffer = CreateRandomData(BufferSize);
	TUniquePtr<uint8[]> ReplicatedBuffer = MakeUnique<uint8[]>(BufferSize);

	FEditorBulkData BulkData;

	// Serialize the SourceBuffer to BulkData 
	{
		FEditorBulkDataWriter WriterAr(BulkData);
		WriterAr.Serialize(SourceBuffer.Get(), BufferSize);
	}

	// Serialize BulkData back to ReplicatedBuffer
	{
		FEditorBulkDataReader ReaderAr(BulkData);
		ReaderAr.Serialize(ReplicatedBuffer.Get(), BufferSize);
	}

	// Now test that the buffer was restored to the original values
	const bool bMemCmpResult = FMemory::Memcmp(SourceBuffer.Get(), ReplicatedBuffer.Get(), BufferSize) == 0;
	CHECK(bMemCmpResult);

	// Test writing nothing to an empty bulkdata object and then reading that bulkdata object
	// to make sure that we deal with null buffers properly.
	{
		FEditorBulkData EmptyBulkData;
		FEditorBulkDataWriter WriterAr(EmptyBulkData);
		FEditorBulkDataReader ReaderAr(EmptyBulkData);
	}
}

/** 
 * This test will serialize a number of empty and valid bulkdata objects to a memory buffer and then serialize them back 
 * as bulkdata objects again. This should help find problems where the data saved and loaded are mismatched.
 * Note that this does not check the package saving code paths, only direct serialization to buffers.
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::SerializationToMemory", "[CoreUObject][Serialization]")
{
	const bool bIsArPersistent = true;
	const int64 BufferSize = 1024;

	TUniquePtr<uint8[]> SourceBuffer = CreateRandomData(BufferSize);
	
	TArray<uint8> MemoryBuffer;
	FGuid ValidBulkDataId;
	FEditorBulkData EmptyBulkData;

	{
		FEditorBulkData ValidBulkData;
		ValidBulkData.UpdatePayload(FSharedBuffer::Clone(SourceBuffer.Get(), BufferSize));
		ValidBulkDataId = ValidBulkData.GetIdentifier();

		// Write out
		{
			FMemoryWriter WriterAr(MemoryBuffer, bIsArPersistent);

			ValidBulkData.Serialize(WriterAr, nullptr);
			EmptyBulkData.Serialize(WriterAr, nullptr);
			ValidBulkData.Serialize(WriterAr, nullptr);
			EmptyBulkData.Serialize(WriterAr, nullptr);
		}

		// First test we test reading the editor bulkdata while ValidBulkData is still registered and in scope
		// in this case if the BulkDataRegistry is enabled then SerializedBulkData and ValidBulkData should
		// have different identifiers, if the BulkDataRegistry is disabled then they should have the same identifiers
		FEditorBulkData SerializedBulkData;

		FMemoryReader ReaderAr(MemoryBuffer, bIsArPersistent);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		if (IsBulkDataRegistryEnabled())
		{
			CHECK_NOT_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkData.GetIdentifier());
		}
		else
		{
			CHECK_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkData.GetIdentifier());
		}
		
		CHECK(FMemory::Memcmp(SourceBuffer.Get(), SerializedBulkData.GetPayload().Get().GetData(), BufferSize) == 0);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_FALSE(SerializedBulkData.GetIdentifier().IsValid());
		CHECK(SerializedBulkData.GetPayload().Get().IsNull());

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		if (IsBulkDataRegistryEnabled())
		{
			CHECK_NOT_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkData.GetIdentifier());
		}
		else
		{
			CHECK_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkData.GetIdentifier());
		}

		CHECK(FMemory::Memcmp(SourceBuffer.Get(), SerializedBulkData.GetPayload().Get().GetData(), BufferSize) == 0);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_FALSE(SerializedBulkData.GetIdentifier().IsValid());
		CHECK(SerializedBulkData.GetPayload().Get().IsNull());
	}

	// Now test the serialization when ValidBulkData is no longer in scope and has unregistered itself. In this
	// case it doesn't matter if the BulkDataRegistry is enabled or not. In both cases we should get the original
	// bulkdata identifier.
	{
		FEditorBulkData SerializedBulkData;

		FMemoryReader ReaderAr(MemoryBuffer, bIsArPersistent);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkDataId);
		CHECK(FMemory::Memcmp(SourceBuffer.Get(), SerializedBulkData.GetPayload().Get().GetData(), BufferSize) == 0);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_EQUAL(SerializedBulkData.GetIdentifier(), EmptyBulkData.GetIdentifier());
		CHECK(SerializedBulkData.GetPayload().Get().IsNull());

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_EQUAL(SerializedBulkData.GetIdentifier(), ValidBulkDataId);
		CHECK(FMemory::Memcmp(SourceBuffer.Get(), SerializedBulkData.GetPayload().Get().GetData(), BufferSize) == 0);

		SerializedBulkData.Serialize(ReaderAr, nullptr);
		CHECK_EQUAL(SerializedBulkData.GetIdentifier(), EmptyBulkData.GetIdentifier());
		CHECK(SerializedBulkData.GetPayload().Get().IsNull());
	}
}

/**
 * This set of tests validate that the BulkData's identifier works how we expect it too. It should remain unique in all cases except
 * move semantics.
 */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Identifiers", "[CoreUObject][Serialization]")
{
	// Some basic tests with an invalid id
	{
		FEditorBulkData BulkData;
		CHECK_FALSE(BulkData.GetIdentifier().IsValid());

		FEditorBulkData CopiedBulkData(BulkData);
		CHECK_FALSE(CopiedBulkData.GetIdentifier().IsValid());

		FEditorBulkData AssignedBulkData;
		AssignedBulkData = BulkData;
		CHECK_FALSE(AssignedBulkData.GetIdentifier().IsValid());
		
		// Check that we did not change the initial object at any point
		CHECK_FALSE(BulkData.GetIdentifier().IsValid());
	}

	// Some basic tests with a valid id
	{
		FEditorBulkData BulkData;
		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared()); // Assigning this payload should cause BulkData to gain an identifier
		CHECK(BulkData.GetIdentifier().IsValid());

		const FGuid OriginalGuid = BulkData.GetIdentifier();

		FEditorBulkData CopiedBulkData(BulkData);
		CHECK_NOT_EQUAL(BulkData.GetIdentifier(), CopiedBulkData.GetIdentifier());

		FEditorBulkData AssignedBulkData;
		AssignedBulkData = BulkData;
		CHECK_NOT_EQUAL( BulkData.GetIdentifier(), AssignedBulkData.GetIdentifier());

		// Check that we did not change the initial object at any point
		CHECK_EQUAL(BulkData.GetIdentifier(), OriginalGuid);

		// Now that AssignedBulkData has a valid identifier, make sure that it is not changed if we assign something else to it.
		const FGuid OriginalAssignedGuid = AssignedBulkData.GetIdentifier();
		AssignedBulkData = CopiedBulkData;
		CHECK_EQUAL(AssignedBulkData.GetIdentifier(), OriginalAssignedGuid);
	}

	// Test move constructor
	{	
		FEditorBulkData BulkData;
		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());

		FGuid OriginalGuid = BulkData.GetIdentifier();

		FEditorBulkData MovedBulkData = MoveTemp(BulkData);

		CHECK_EQUAL(MovedBulkData.GetIdentifier(), OriginalGuid);
	}

	// Test move assignment
	{
		FEditorBulkData BulkData;
		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());

		FGuid OriginalGuid = BulkData.GetIdentifier();

		FEditorBulkData MovedBulkData;
		MovedBulkData = MoveTemp(BulkData);

		CHECK_EQUAL(MovedBulkData.GetIdentifier(), OriginalGuid);
	}

	// Check that resizing an array will not change the internals
	{
		const uint32 NumToTest = 10;

		TArray<FEditorBulkData> BulkDataArray;
		TArray<FGuid> GuidArray;

		for (uint32 Index = 0; Index < NumToTest; ++Index)
		{
			BulkDataArray.Add(FEditorBulkData());

			// Leave some with invalid ids and some with valid ones
			if (Index % 2 == 0)
			{
				BulkDataArray[Index].UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());
			}

			GuidArray.Add(BulkDataArray[Index].GetIdentifier());
		}

		// Force an internal reallocation and make sure that the identifiers are unchanged.
		// Note that it is possible that the allocation is just resized and not reallocated.
		BulkDataArray.Reserve(BulkDataArray.Max() * 4);

		for (uint32 Index = 0; Index < NumToTest; ++Index)
		{
			CHECK_EQUAL( BulkDataArray[Index].GetIdentifier(), GuidArray[Index]);
		}

		// Now insert a new item, moving all of the existing entries and make sure that
		// the identifiers are unchanged.
		BulkDataArray.Insert(FEditorBulkData(), 0);

		for (uint32 Index = 0; Index < NumToTest; ++Index)
		{
			CHECK_EQUAL(BulkDataArray[Index+1].GetIdentifier(), GuidArray[Index]);
		}
	}

	// Test that adding a payload to a reset bulkdata object or one that has had a zero length payload applied
	// will correctly show the original id once it has a valid payload.
	{
		FEditorBulkData BulkData;
		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());
		
		const FGuid OriginalGuid = BulkData.GetIdentifier();

		BulkData.Reset();
		CHECK_EQUAL(BulkData.GetIdentifier(), OriginalGuid);

		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());
		CHECK_EQUAL(BulkData.GetIdentifier(), OriginalGuid);

		BulkData.UpdatePayload(FUniqueBuffer::Alloc(0).MoveToShared());
		CHECK_EQUAL(BulkData.GetIdentifier(), OriginalGuid);
		
		BulkData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());
		CHECK_EQUAL(BulkData.GetIdentifier(), OriginalGuid);
	}

	// Test that serialization does not change the identifier (in this case serializing to and from a memory buffer) 
	{
		FEditorBulkData SrcData;
		SrcData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());

		TArray<uint8> MemoryBuffer;
		const bool bIsArPersistent = true;

		FEditorBulkData DstData;

		// Serialize the SourceBuffer to BulkData 
		{
			FMemoryWriter WriterAr(MemoryBuffer, bIsArPersistent);
			SrcData.Serialize(WriterAr, nullptr);
		}

		// Serialize BulkData back to ReplicatedBuffer
		{
			FMemoryReader ReaderAr(MemoryBuffer, bIsArPersistent);
			DstData.Serialize(ReaderAr, nullptr);
		}

		if (IsBulkDataRegistryEnabled())
		{
			CHECK_NOT_EQUAL(SrcData.GetIdentifier(), DstData.GetIdentifier());
		}
		else
		{
			CHECK_EQUAL(SrcData.GetIdentifier(), DstData.GetIdentifier());
		}	
	}

	// Test that serialization does not change the identifier (in this case serializing to and from a memory buffer) 
	{
		FEditorBulkData SrcData;
		SrcData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());

		const FGuid OriginalIdentifier = SrcData.GetIdentifier();
		SrcData.UpdatePayload(FUniqueBuffer::Alloc(0).MoveToShared());

		TArray<uint8> MemoryBuffer;
		const bool bIsArPersistent = true;

		FEditorBulkData DstData;

		// Serialize the SourceBuffer to BulkData 
		{
			FMemoryWriter WriterAr(MemoryBuffer, bIsArPersistent);
			SrcData.Serialize(WriterAr, nullptr);
		}

		// Serialize BulkData back to ReplicatedBuffer
		{
			FMemoryReader ReaderAr(MemoryBuffer, bIsArPersistent);
			DstData.Serialize(ReaderAr, nullptr);
		}

		CHECK_EQUAL(DstData.GetIdentifier(), OriginalIdentifier);

		DstData.UpdatePayload(FUniqueBuffer::Alloc(32).MoveToShared());
		CHECK_EQUAL(DstData.GetIdentifier(), OriginalIdentifier);
	}
}

TEST_CASE("CoreUObject::Serialization::FEditorBulkData::ZeroSizedAllocs", "[CoreUObject][Serialization]")
{
	{
		FEditorBulkData Empty;
		CHECK(Empty.GetPayload().Get().GetData() == nullptr);
		CHECK(Empty.GetCompressedPayload().Get().Decompress().GetData() == nullptr);

		CHECK_FALSE(Empty.HasPayloadData());
		CHECK_FALSE(Empty.DoesPayloadNeedLoading());
	}

	{
		FEditorBulkData EmptySrc;
		
		FLargeMemoryWriter ArWrite(0, true);
		EmptySrc.Serialize(ArWrite, nullptr);

		FEditorBulkData EmptyDst;

		FLargeMemoryReader ArRead(ArWrite.GetData(), ArWrite.TotalSize(), ELargeMemoryReaderFlags::Persistent);
		EmptyDst.Serialize(ArRead, nullptr);

		CHECK(EmptyDst.GetPayload().Get().GetData() == nullptr);
		CHECK(EmptyDst.GetCompressedPayload().Get().Decompress().GetData() == nullptr);

		CHECK_FALSE(EmptyDst.HasPayloadData());
		CHECK_FALSE(EmptyDst.DoesPayloadNeedLoading());
	}

	{
		FEditorBulkData ZeroAlloc;
		ZeroAlloc.UpdatePayload(FUniqueBuffer::Alloc(0).MoveToShared());

		CHECK(ZeroAlloc.GetPayload().Get().GetData() == nullptr);
		CHECK(ZeroAlloc.GetCompressedPayload().Get().Decompress().GetData() == nullptr);

		CHECK_FALSE(ZeroAlloc.HasPayloadData());
		CHECK_FALSE(ZeroAlloc.DoesPayloadNeedLoading());
	}
}

/** This tests the function UE::Serialization::IoHashToGuid which is closely used with the FEditorBulkData system */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::IoHashToGuid", "[CoreUObject][Serialization]")
{
	// Test that an empty hash will give an invalid FGuid
	FIoHash DefaultHash;
	FGuid InvalidGuid = IoHashToGuid(DefaultHash);

	CHECK_FALSE(InvalidGuid.IsValid());
	
	// Test that finding the FGuid of a known hash results in the FGuid we expect. If not then the generation algorithm has
	// changed. The failing test should remind whoever changed the algorithm to double check that changing the results 
	// will not have knock on effects.
	const uint8 KnownHashData[20] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18 , 19};

	FIoHash KnownHash(KnownHashData);
	FGuid KnownGuid = IoHashToGuid(KnownHash);
	FGuid KnownResult(TEXT("04030201-0807-0605-0C0B-0A09100F0E0D"));

	CHECK_EQUAL(KnownGuid, KnownResult);
}

/** This tests that updating a payload via a FSharedBufferWithID will have the same results as if the payload was applied to the bulkdata object directly */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::SharedBufferWithID", "[CoreUObject][Serialization]")
{
	// Test FSharedBufferWithID/FEditorBulkData in their default states
	{
		FEditorBulkData::FSharedBufferWithID SharedId;
		FEditorBulkData BulkDataFromSharedId;
		BulkDataFromSharedId.UpdatePayload(MoveTemp(SharedId));

		FEditorBulkData BulkData;

		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadId(), BulkData.GetPayloadId());
		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadSize(), BulkData.GetPayloadSize());
	}

	// Test FSharedBufferWithID/FEditorBulkData updated with a null FSharedBuffer
	{
		FSharedBuffer NullBuffer;

		FEditorBulkData::FSharedBufferWithID SharedId(NullBuffer);
		FEditorBulkData BulkDataFromSharedId;
		BulkDataFromSharedId.UpdatePayload(MoveTemp(SharedId));

		FEditorBulkData BulkData;
		BulkData.UpdatePayload(NullBuffer);

		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadId(), BulkData.GetPayloadId());
		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadSize(), BulkData.GetPayloadSize());
	}

	// Test FSharedBufferWithID/FEditorBulkData updated with a zero length FSharedBuffer
	{
		FSharedBuffer ZeroLengthBuffer = FUniqueBuffer::Alloc(0).MoveToShared();

		FEditorBulkData::FSharedBufferWithID SharedId(ZeroLengthBuffer);
		FEditorBulkData BulkDataFromSharedId;
		BulkDataFromSharedId.UpdatePayload(MoveTemp(SharedId));

		FEditorBulkData BulkData;
		BulkData.UpdatePayload(ZeroLengthBuffer);

		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadId(), BulkData.GetPayloadId());
		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadSize(), BulkData.GetPayloadSize());
	}

	// Test FSharedBufferWithID/FEditorBulkData updated with a random set of data
	{
		const int64 BufferSize = 1024;
		TUniquePtr<uint8[]> SourceBuffer = CreateRandomData(BufferSize);
		FSharedBuffer RandomData = FSharedBuffer::MakeView(SourceBuffer.Get(), BufferSize);

		FEditorBulkData::FSharedBufferWithID SharedId(RandomData);
		FEditorBulkData BulkDataFromSharedId;
		BulkDataFromSharedId.UpdatePayload(MoveTemp(SharedId));

		FEditorBulkData BulkData;
		BulkData.UpdatePayload(RandomData);

		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadId(), BulkData.GetPayloadId());
		CHECK_EQUAL(BulkDataFromSharedId.GetPayloadSize(), BulkData.GetPayloadSize());
	}
}

/** Test a number of threads all updating a FEditorBulkData object at the same time */
TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Threading_Basic", "[CoreUObject][Serialization]")
{
	// Before thread safety was added the following number of tests/payload tended to result in broken data.
	// Although trying to induce threading issues is not an exact science.
	const int32 NumTests = 128;
	const int32 NumPayloadsToTest = 16;

	for (int32 TestIndex = 0; TestIndex < NumTests; ++TestIndex)
	{
		// Create a number of randomly sized payloads
		TArray<FSharedBuffer> Payloads;
		Payloads.Reserve(NumPayloadsToTest);

		for (int32 Index = 0; Index < NumPayloadsToTest; ++Index)
		{
			const int64 BufferSize = FMath::RandRange(512, 12 * 1024);
			Payloads.Add(CreateRandomPayload(BufferSize));
		}

		FEditorBulkData BulkData;

		ParallelFor(NumPayloadsToTest, [&Payloads, &BulkData](int32 Index)
			{
				BulkData.UpdatePayload(Payloads[Index]);
			});

		const FIoHash FinalId = BulkData.GetPayloadId();
		const uint64 FinalSize = BulkData.GetPayloadSize();

		const FSharedBuffer FinalPayload = BulkData.GetPayload().Get();

		// Make sure that the size of the payload matches the value stored in the object
		REQUIRE_EQUAL(FinalPayload.GetSize(), FinalSize);

		// Make sure that the hash of the payload matches the value stored in the object
		REQUIRE_EQUAL(FIoHash::HashBuffer(FinalPayload), FinalId);
	}
}

TEST_CASE("CoreUObject::Serialization::FEditorBulkData::Threading_Assignment", "[CoreUObject][Serialization]")
{
	const int32 NumThreads = 8;
	const int32 NumBulkDataToTest = 128;
	const int32 NumAssignments = 16 * 1024;

	// Create a FEditorBulkData of randomly sized payloads
	TArray<FEditorBulkData> BulkDatas;
	BulkDatas.SetNum(NumBulkDataToTest);

	for (int32 Index = 0; Index < NumBulkDataToTest; ++Index)
	{
		const int32 PayloadSize = FMath::RandRange(512, 1024);
		BulkDatas[Index].UpdatePayload(CreateRandomPayload(PayloadSize));
	}

	TArray<UE::Tasks::FTask> CompletionEvents;
	CompletionEvents.Reserve(NumThreads);

	for (int Index = 0; Index < NumThreads; ++Index)
	{
		UE::Tasks::FTask Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&BulkDatas, NumAssignments]()
			{
				for (int32 Index = 0; Index < NumAssignments; ++Index)
				{
					const int32 DstIndex = FMath::RandRange(0, BulkDatas.Num() - 1);
					const int32 SrcIndex = FMath::RandRange(0, BulkDatas.Num() - 1);

					BulkDatas[DstIndex] = BulkDatas[SrcIndex];
				}
			});

		CompletionEvents.Emplace(MoveTemp(Task));
	}

	UE::Tasks::Wait(CompletionEvents);

	for (const FEditorBulkData& BulkData : BulkDatas)
	{
		const FSharedBuffer Payload = BulkData.GetPayload().Get();

		// Make sure that the size of the payload matches the value stored in the object
		REQUIRE_EQUAL(BulkData.GetPayloadSize(), (int64)Payload.GetSize());

		// Make sure that the hash of the payload matches the value stored in the object
		REQUIRE_EQUAL(BulkData.GetPayloadId(), FIoHash::HashBuffer(Payload));
	}
}

} // namespace UE::Serialization

#endif // WITH_LOW_LEVEL_TESTS && WITH_EDITORONLY_DATA
