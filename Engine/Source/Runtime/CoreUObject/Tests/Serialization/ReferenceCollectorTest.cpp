// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS
#include "TestHarness.h"
#include "UObject/Package.h"
#include "../ObjectPtrTestClass.h"
#include "Misc/ScopeExit.h"
#include "../ObjectRefTrackingTestBase.h"


class FTestReferencecollector : public FReferenceCollector
{
public:
	virtual bool IsIgnoringArchetypeRef() const override
	{
		return true;
	}

	virtual bool IsIgnoringTransient() const override
	{
		return true;
	}

	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		++Count;
	}

	int Count = 0;
};

DISABLED_TEST_CASE("CoreUObject::ReferenceCollector::TestUnResolved", "[.]")
{
	const FName TestPackage1Name("/Engine/Test/RefCollector/Transient");
	UPackage* TestPackage1 = NewObject<UPackage>(nullptr, TestPackage1Name, RF_Transient);
	TestPackage1->AddToRoot();
	UObjectPtrTestClassWithRef* TestObject1 = NewObject<UObjectPtrTestClassWithRef>(TestPackage1, TEXT("TestObject1"));
	UObjectPtrTestClass* Inner = NewObject<UObjectPtrTestClass>(TestObject1, TEXT("TestObject1"));
	ON_SCOPE_EXIT
	{
		TestPackage1->RemoveFromRoot();
	};

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	{
		FObjectPtr ObjPtr(MakeUnresolvedHandle(Inner));
		TObjectPtr<UObjectPtrTestClass> Ptr = *reinterpret_cast<TObjectPtr<UObjectPtrTestClass>*>(&ObjPtr);
		REQUIRE(!Ptr.IsResolved());
		TestObject1->ObjectPtr = Ptr;
		REQUIRE(!TestObject1->ObjectPtr.IsResolved());
	}
#endif

	FTestReferencecollector Collector;
	FReferenceCollectorArchive Ar(nullptr, Collector);

	TestObject1->Serialize(Ar);

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	CHECK(Collector.Count == 1);
#else
	CHECK(Collector.Count == 2);
#endif
	
}
#endif