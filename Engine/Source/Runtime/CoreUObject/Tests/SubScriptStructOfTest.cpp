// Copyright Epic Games, Inc. All Rights Reserved.


#include "SubScriptStructOfTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubScriptStructOfTest)

#if WITH_TESTS

#include "Templates/SubScriptStructOf.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FSubScriptStructOfTest, "CoreUObject::SubScriptStructOf", "[Core][UObject][EngineFilter]")
{
	// operator=
	{
		TSubScriptStructOf<FSubScriptStructOfTest_ChildA> SubOf_ChildA;
		SubOf_ChildA = FSubScriptStructOfTest_ChildA::StaticStruct();
		CHECK(*SubOf_ChildA == FSubScriptStructOfTest_ChildA::StaticStruct());
		SubOf_ChildA = FSubScriptStructOfTest_GrandChildA::StaticStruct();
		CHECK(*SubOf_ChildA == FSubScriptStructOfTest_GrandChildA::StaticStruct());
		SubOf_ChildA = TBaseStructure<FVector>::Get();
		CHECK(*SubOf_ChildA == nullptr);
	}

	// constructor && operator
	{
		TSubScriptStructOf<FSubScriptStructOfTest_Base> SubOf_Base1 = FSubScriptStructOfTest_ChildA::StaticStruct();
		CHECK(SubOf_Base1 == FSubScriptStructOfTest_ChildA::StaticStruct());
		TSubScriptStructOf<FSubScriptStructOfTest_Base> SubOf_Base2 = SubOf_Base1;
		CHECK(SubOf_Base1 == SubOf_Base2);
		SubOf_Base1 = SubOf_Base2;
		CHECK(SubOf_Base1 == SubOf_Base2);
		CHECK(SubOf_Base1.Get() == *SubOf_Base2);
	}

	// conversion
	{
		TNotNull<UScriptStruct*> NotNullPtr = FSubScriptStructOfTest_ChildA::StaticStruct();
		TSubScriptStructOf<FSubScriptStructOfTest_ChildA> SubOf_ChildA = NotNullPtr;
		CHECK(SubOf_ChildA == FSubScriptStructOfTest_ChildA::StaticStruct());
	}	
}

} // UE
#endif
