// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubScriptStructOf.h"
#include "UObject/Object.h"

#include "SubScriptStructOfTest.generated.h"

#if WITH_TESTS

USTRUCT()
struct FSubScriptStructOfTest_Base
{
	GENERATED_BODY()
};

USTRUCT()
struct FSubScriptStructOfTest_ChildA : public FSubScriptStructOfTest_Base
{
	GENERATED_BODY()
};

USTRUCT()
struct FSubScriptStructOfTest_ChildB : public FSubScriptStructOfTest_Base
{
	GENERATED_BODY()
};

USTRUCT()
struct FSubScriptStructOfTest_GrandChildA : public FSubScriptStructOfTest_ChildA
{
	GENERATED_BODY()
};

UCLASS()
class USubScriptStructOfTest : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Properties")
	TSubScriptStructOf<FSubScriptStructOfTest_ChildA> ChildA;

	UPROPERTY(EditAnywhere, Category = "Properties", meta=(MetaStruct="/Script/CoreUObject.SubScriptStructOfTest_ChildB"))
	FSubScriptStructOf ChildB;
};

#endif