// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/AssertionMacros.h"
#include "SubobjectInstancingTest.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UnrealType.h"

namespace UE
{

TEST_CASE_NAMED(FInstancedPropertiesTest, "CoreUObject::InstancedProperties", "[CoreUObject][EngineFilter]")
{
	SECTION("Instanced object property")
	{
		const FName InstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObject);
		const FProperty* InstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(InstancedObjectPropertyName);

		REQUIRE(InstancedObjectProperty != nullptr);
		CHECK(InstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(InstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(InstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Non-instanced object property")
	{
		const FName NonInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, ExternalObject);
		const FProperty* NonInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(NonInstancedObjectPropertyName);

		REQUIRE(NonInstancedObjectProperty != nullptr);
		CHECK(NonInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK_FALSE(NonInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(NonInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(NonInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK_FALSE(NonInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("DefaultToInstanced object property")
	{
		const FName DefaultToInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectFromType);
		const FProperty* DefaultToInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(DefaultToInstancedObjectPropertyName);

		REQUIRE(DefaultToInstancedObjectProperty != nullptr);
		CHECK(DefaultToInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(DefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: DefaultToInstanced also implies 'EditInline' (via UHT)
		CHECK(DefaultToInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Transient instanced object property")
	{
		const FName TransientInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientInnerObject);
		const FProperty* TransientInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(TransientInstancedObjectPropertyName);

		REQUIRE(TransientInstancedObjectProperty != nullptr);
		CHECK(TransientInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(TransientInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(TransientInstancedObjectProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("DuplicateTransient instanced object property")
	{
		const FName DuplicateTransientInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, DuplicateTransientInnerObjectUsingNew);
		const FProperty* DuplicateTransientInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(DuplicateTransientInstancedObjectPropertyName);

		REQUIRE(DuplicateTransientInstancedObjectProperty != nullptr);
		CHECK(DuplicateTransientInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(DuplicateTransientInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(DuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK(DuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_DuplicateTransient));
		CHECK(DuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(DuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(DuplicateTransientInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("NonPIEDuplicateTransient instanced object property")
	{
		const FName NonPIEDuplicateTransientInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, NonPIEDuplicateTransientInnerObjectUsingNew);
		const FProperty* NonPIEDuplicateTransientInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(NonPIEDuplicateTransientInstancedObjectPropertyName);

		REQUIRE(NonPIEDuplicateTransientInstancedObjectProperty != nullptr);
		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(NonPIEDuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK_FALSE(NonPIEDuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_DuplicateTransient));
		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_NonPIEDuplicateTransient));
		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(NonPIEDuplicateTransientInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Transient type instanced object property")
	{
		const FName TransientTypeInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientInnerObjectFromType);
		const FProperty* TransientTypeInstancedObjectProperty = USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(TransientTypeInstancedObjectPropertyName);

		REQUIRE(TransientTypeInstancedObjectProperty != nullptr);
		CHECK(TransientTypeInstancedObjectProperty->IsA<FObjectProperty>());
		
		const FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(TransientTypeInstancedObjectProperty);
		{
			REQUIRE(ObjectProperty->PropertyClass != nullptr);
			CHECK(ObjectProperty->PropertyClass->HasAllClassFlags(CLASS_Transient));
		}

		CHECK(TransientTypeInstancedObjectProperty->ContainsInstancedObjectProperty());
		// Note: CLASS_Transient imposes RF_Transient on all non-archetype allocations (see StaticAllocateObject), which means the referenced object will not be exported on save.
		// It does NOT impose CPF_Transient - the property value is instead serialized as NULL on save if the referenced object is not also exported.
		CHECK_FALSE(TransientTypeInstancedObjectProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientTypeInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientTypeInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientTypeInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Instanced object array property")
	{
		const FName InstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectArray);
		const FArrayProperty* InstancedObjectArrayProperty = CastField<FArrayProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(InstancedObjectArrayPropertyName));

		REQUIRE(InstancedObjectArrayProperty != nullptr);
		CHECK(InstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(InstancedObjectArrayProperty->Inner != nullptr);
		CHECK(InstancedObjectArrayProperty->Inner->IsA<FObjectProperty>());

		CHECK(InstancedObjectArrayProperty->Inner->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectArrayProperty->Inner->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("DefaultToInstanced object array property")
	{
		const FName DefaultToInstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectFromTypeArray);
		const FArrayProperty* DefaultToInstancedObjectArrayProperty = CastField<FArrayProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(DefaultToInstancedObjectArrayPropertyName));

		REQUIRE(DefaultToInstancedObjectArrayProperty != nullptr);
		CHECK(DefaultToInstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(DefaultToInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: DefaultToInstanced also implies 'EditInline' (via UHT). 
		CHECK(DefaultToInstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(DefaultToInstancedObjectArrayProperty->Inner != nullptr);
		CHECK(DefaultToInstancedObjectArrayProperty->Inner->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectArrayProperty->Inner->ContainsInstancedObjectProperty());
		CHECK_FALSE(DefaultToInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(DefaultToInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Unlike explicitly-instanced array properties, the Inner is not also flagged as 'EditInline'.
		CHECK_FALSE(DefaultToInstancedObjectArrayProperty->Inner->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Transient instanced object array property")
	{
		const FName TransientInstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientInnerObjectArray);
		const FArrayProperty* TransientInstancedObjectArrayProperty = CastField<FArrayProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(TransientInstancedObjectArrayPropertyName));

		REQUIRE(TransientInstancedObjectArrayProperty != nullptr);
		CHECK(TransientInstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(TransientInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(TransientInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK_FALSE(TransientInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(TransientInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(TransientInstancedObjectArrayProperty->Inner != nullptr);
		CHECK(TransientInstancedObjectArrayProperty->Inner->IsA<FObjectProperty>());

		CHECK(TransientInstancedObjectArrayProperty->Inner->ContainsInstancedObjectProperty());
		CHECK_FALSE(TransientInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(TransientInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientInstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectArrayProperty->Inner->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Instanced object set container property")
	{
		const FName InstancedObjectSetPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectSet);
		const FSetProperty* InstancedObjectSetProperty = CastField<FSetProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(InstancedObjectSetPropertyName));

		REQUIRE(InstancedObjectSetProperty != nullptr);
		CHECK(InstancedObjectSetProperty->ContainsInstancedObjectProperty());
		CHECK(InstancedObjectSetProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(InstancedObjectSetProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(InstancedObjectSetProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectSetProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(InstancedObjectSetProperty->ElementProp != nullptr);
		CHECK(InstancedObjectSetProperty->ElementProp->IsA<FObjectProperty>());

		CHECK(InstancedObjectSetProperty->ElementProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectSetProperty->ElementProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("DefaultToInstanced object set container property")
	{
		const FName DefaultToInstancedObjectSetPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectFromTypeSet);
		const FSetProperty* DefaultToInstancedObjectSetProperty = CastField<FSetProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(DefaultToInstancedObjectSetPropertyName));

		REQUIRE(DefaultToInstancedObjectSetProperty != nullptr);
		CHECK(DefaultToInstancedObjectSetProperty->ContainsInstancedObjectProperty());
		CHECK(DefaultToInstancedObjectSetProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectSetProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectSetProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: DefaultToInstanced also implies 'EditInline' (via UHT).
		CHECK(DefaultToInstancedObjectSetProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(DefaultToInstancedObjectSetProperty->ElementProp != nullptr);
		CHECK(DefaultToInstancedObjectSetProperty->ElementProp->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectSetProperty->ElementProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(DefaultToInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(DefaultToInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Unlike explicitly-instanced set properties, the ElementProp is not also flagged as 'EditInline'.
		CHECK_FALSE(DefaultToInstancedObjectSetProperty->ElementProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Transient instanced object set container property")
	{
		const FName TransientInstancedObjectSetPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientInnerObjectSet);
		const FSetProperty* TransientInstancedObjectSetProperty = CastField<FSetProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(TransientInstancedObjectSetPropertyName));

		REQUIRE(TransientInstancedObjectSetProperty != nullptr);
		CHECK(TransientInstancedObjectSetProperty->ContainsInstancedObjectProperty());
		CHECK(TransientInstancedObjectSetProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(TransientInstancedObjectSetProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK_FALSE(TransientInstancedObjectSetProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(TransientInstancedObjectSetProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectSetProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(TransientInstancedObjectSetProperty->ElementProp != nullptr);
		CHECK(TransientInstancedObjectSetProperty->ElementProp->IsA<FObjectProperty>());

		CHECK(TransientInstancedObjectSetProperty->ElementProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(TransientInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(TransientInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientInstancedObjectSetProperty->ElementProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectSetProperty->ElementProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Instanced object map container property")
	{
		const FName InstancedObjectMapPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectMap);
		const FMapProperty* InstancedObjectMapProperty = CastField<FMapProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(InstancedObjectMapPropertyName));

		REQUIRE(InstancedObjectMapProperty != nullptr);
		CHECK(InstancedObjectMapProperty->ContainsInstancedObjectProperty());
		CHECK(InstancedObjectMapProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(InstancedObjectMapProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(InstancedObjectMapProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: 'EditInline' is set via UHT.
		CHECK(InstancedObjectMapProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(InstancedObjectMapProperty->KeyProp != nullptr);
		CHECK(InstancedObjectMapProperty->KeyProp->IsA<FObjectProperty>());

		CHECK(InstancedObjectMapProperty->KeyProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Currently the KeyProp is not also flagged as 'EditInline' when the map is declared as 'Instanced'.
		CHECK_FALSE(InstancedObjectMapProperty->KeyProp->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(InstancedObjectMapProperty->ValueProp != nullptr);
		CHECK(InstancedObjectMapProperty->ValueProp->IsA<FObjectProperty>());

		CHECK(InstancedObjectMapProperty->ValueProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectMapProperty->ValueProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("DefaultToInstanced object map container property")
	{
		const FName DefaultToInstancedObjectMapPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObjectFromTypeMap);
		const FMapProperty* DefaultToInstancedObjectMapProperty = CastField<FMapProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(DefaultToInstancedObjectMapPropertyName));

		REQUIRE(DefaultToInstancedObjectMapProperty != nullptr);
		CHECK(DefaultToInstancedObjectMapProperty->ContainsInstancedObjectProperty());
		CHECK(DefaultToInstancedObjectMapProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: DefaultToInstanced also implies 'EditInline' (via UHT).
		CHECK(DefaultToInstancedObjectMapProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(DefaultToInstancedObjectMapProperty->KeyProp != nullptr);
		CHECK(DefaultToInstancedObjectMapProperty->KeyProp->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectMapProperty->KeyProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(DefaultToInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Unlike explicitly-instanced map properties, the Key/ValueProp is not also flagged as 'EditInline'.
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->KeyProp->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(DefaultToInstancedObjectMapProperty->ValueProp != nullptr);
		CHECK(DefaultToInstancedObjectMapProperty->ValueProp->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectMapProperty->ValueProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(DefaultToInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Unlike explicitly-instanced map properties, the Key/ValueProp is not also flagged as 'EditInline'.
		CHECK_FALSE(DefaultToInstancedObjectMapProperty->ValueProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Transient instanced object map container property")
	{
		const FName TransientInstancedObjectMapPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientInnerObjectMap);
		const FMapProperty* TransientInstancedObjectMapProperty = CastField<FMapProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(TransientInstancedObjectMapPropertyName));

		REQUIRE(TransientInstancedObjectMapProperty != nullptr);
		CHECK(TransientInstancedObjectMapProperty->ContainsInstancedObjectProperty());
		CHECK(TransientInstancedObjectMapProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(TransientInstancedObjectMapProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK_FALSE(TransientInstancedObjectMapProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(TransientInstancedObjectMapProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: 'EditInline' is set via UHT.
		CHECK(TransientInstancedObjectMapProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(TransientInstancedObjectMapProperty->KeyProp != nullptr);
		CHECK(TransientInstancedObjectMapProperty->KeyProp->IsA<FObjectProperty>());

		CHECK(TransientInstancedObjectMapProperty->KeyProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(TransientInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(TransientInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientInstancedObjectMapProperty->KeyProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Currently the KeyProp is not also flagged as 'EditInline' when the map is declared as 'Instanced'.
		CHECK_FALSE(TransientInstancedObjectMapProperty->KeyProp->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(TransientInstancedObjectMapProperty->ValueProp != nullptr);
		CHECK(TransientInstancedObjectMapProperty->ValueProp->IsA<FObjectProperty>());

		CHECK(TransientInstancedObjectMapProperty->ValueProp->ContainsInstancedObjectProperty());
		CHECK_FALSE(TransientInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(TransientInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_Transient));
		CHECK(TransientInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(TransientInstancedObjectMapProperty->ValueProp->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(TransientInstancedObjectMapProperty->ValueProp->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Struct with instanced object property")
	{
		const FName StructWithInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, StructWithInnerObjects);
		const FStructProperty* StructWithInstancedObjectProperty = CastField<FStructProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(StructWithInstancedObjectPropertyName));

		REQUIRE(StructWithInstancedObjectProperty != nullptr);
		CHECK(StructWithInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK_FALSE(StructWithInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(StructWithInstancedObjectProperty->Struct != nullptr);
		const FName InstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(FSubobjectInstancingTestStructType, InnerObject);
		const FProperty* InstancedObjectProperty = StructWithInstancedObjectProperty->Struct->FindPropertyByName(InstancedObjectPropertyName);

		REQUIRE(InstancedObjectProperty != nullptr);
		CHECK(InstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(InstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Struct with DefaultToInstanced object property")
	{
		const FName StructWithInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, StructWithInnerObjects);
		const FStructProperty* StructWithInstancedObjectProperty = CastField<FStructProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(StructWithInstancedObjectPropertyName));

		REQUIRE(StructWithInstancedObjectProperty != nullptr);
		CHECK(StructWithInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(StructWithInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK_FALSE(StructWithInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(StructWithInstancedObjectProperty->Struct != nullptr);
		const FName DefaultToInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(FSubobjectInstancingTestStructType, InnerObjectFromType);
		const FProperty* DefaultToInstancedObjectProperty = StructWithInstancedObjectProperty->Struct->FindPropertyByName(DefaultToInstancedObjectPropertyName);

		REQUIRE(DefaultToInstancedObjectProperty != nullptr);
		CHECK(DefaultToInstancedObjectProperty->IsA<FObjectProperty>());

		CHECK(DefaultToInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(DefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(DefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(DefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(DefaultToInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Struct with instanced object array property")
	{
		const FName StructWithInstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, StructWithInnerObjects);
		const FStructProperty* StructWithInstancedObjectArrayProperty = CastField<FStructProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(StructWithInstancedObjectArrayPropertyName));

		REQUIRE(StructWithInstancedObjectArrayProperty != nullptr);
		CHECK(StructWithInstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(StructWithInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(StructWithInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(StructWithInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK_FALSE(StructWithInstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(StructWithInstancedObjectArrayProperty->Struct != nullptr);
		const FName InstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(FSubobjectInstancingTestStructType, InnerObjectArray);
		const FArrayProperty* InstancedObjectArrayProperty = CastField<FArrayProperty>(StructWithInstancedObjectArrayProperty->Struct->FindPropertyByName(InstancedObjectArrayPropertyName));

		REQUIRE(InstancedObjectArrayProperty != nullptr);
		CHECK(InstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(InstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(InstancedObjectArrayProperty->Inner != nullptr);
		CHECK(InstancedObjectArrayProperty->Inner->IsA<FObjectProperty>());

		CHECK(InstancedObjectArrayProperty->Inner->ContainsInstancedObjectProperty());
		CHECK_FALSE(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(InstancedObjectArrayProperty->Inner->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(InstancedObjectArrayProperty->Inner->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Optional instanced object property")
	{
		const FName OptionalInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, OptionalInnerObject);
		const FOptionalProperty* OptionalInstancedObjectProperty = CastField<FOptionalProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(OptionalInstancedObjectPropertyName));

		REQUIRE(OptionalInstancedObjectProperty != nullptr);
		CHECK(OptionalInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(OptionalInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(OptionalInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(OptionalInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		const FProperty* OptionalInstancedObjectValueProperty = OptionalInstancedObjectProperty->GetValueProperty();
		REQUIRE(OptionalInstancedObjectValueProperty != nullptr);
		CHECK(OptionalInstancedObjectValueProperty->IsA<FObjectProperty>());

		CHECK(OptionalInstancedObjectValueProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(OptionalInstancedObjectValueProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(OptionalInstancedObjectValueProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(OptionalInstancedObjectValueProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalInstancedObjectValueProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Optional DefaultToInstanced object property")
	{
		const FName OptionalDefaultToInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, OptionalInnerObjectFromType);
		const FOptionalProperty* OptionalDefaultToInstancedObjectProperty = CastField<FOptionalProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(OptionalDefaultToInstancedObjectPropertyName));

		REQUIRE(OptionalDefaultToInstancedObjectProperty != nullptr);
		CHECK(OptionalDefaultToInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(OptionalDefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(OptionalDefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(OptionalDefaultToInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: DefaultToInstanced also implies 'EditInline' (via UHT).
		CHECK(OptionalDefaultToInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		const FProperty* OptionalDefaultToInstancedObjectValueProperty = OptionalDefaultToInstancedObjectProperty->GetValueProperty();
		REQUIRE(OptionalDefaultToInstancedObjectValueProperty != nullptr);
		CHECK(OptionalDefaultToInstancedObjectValueProperty->IsA<FObjectProperty>());

		CHECK(OptionalDefaultToInstancedObjectValueProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(OptionalDefaultToInstancedObjectValueProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(OptionalDefaultToInstancedObjectValueProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(OptionalDefaultToInstancedObjectValueProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Unlike explicitly-instanced optional properties, the value property is not also flagged as 'EditInline'.
		CHECK_FALSE(OptionalDefaultToInstancedObjectValueProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Optional transient instanced object property")
	{
		const FName OptionalTransientInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, OptionalTransientInnerObject);
		const FOptionalProperty* OptionalTransientInstancedObjectProperty = CastField<FOptionalProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(OptionalTransientInstancedObjectPropertyName));

		REQUIRE(OptionalTransientInstancedObjectProperty != nullptr);
		CHECK(OptionalTransientInstancedObjectProperty->ContainsInstancedObjectProperty());
		CHECK(OptionalTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(OptionalTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK_FALSE(OptionalTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(OptionalTransientInstancedObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalTransientInstancedObjectProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		const FProperty* OptionalTransientInstancedObjectValueProperty = OptionalTransientInstancedObjectProperty->GetValueProperty();
		REQUIRE(OptionalTransientInstancedObjectValueProperty != nullptr);
		CHECK(OptionalTransientInstancedObjectValueProperty->IsA<FObjectProperty>());

		CHECK(OptionalTransientInstancedObjectValueProperty->ContainsInstancedObjectProperty());
		CHECK_FALSE(OptionalTransientInstancedObjectValueProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(OptionalTransientInstancedObjectValueProperty->HasAllPropertyFlags(CPF_Transient));
		CHECK(OptionalTransientInstancedObjectValueProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(OptionalTransientInstancedObjectValueProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalTransientInstancedObjectValueProperty->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Optional instanced object array property")
	{
		const FName OptionalInstancedObjectArrayPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, OptionalInnerObjectArray);
		const FOptionalProperty* OptionalInstancedObjectArrayProperty = CastField<FOptionalProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(OptionalInstancedObjectArrayPropertyName));

		REQUIRE(OptionalInstancedObjectArrayProperty != nullptr);
		CHECK(OptionalInstancedObjectArrayProperty->ContainsInstancedObjectProperty());
		CHECK(OptionalInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK_FALSE(OptionalInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK_FALSE(OptionalInstancedObjectArrayProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalInstancedObjectArrayProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		FArrayProperty* OptionalInstancedObjectArrayValueProperty = CastField<FArrayProperty>(OptionalInstancedObjectArrayProperty->GetValueProperty());
		REQUIRE(OptionalInstancedObjectArrayValueProperty != nullptr);
		CHECK(OptionalInstancedObjectArrayValueProperty->ContainsInstancedObjectProperty());
		CHECK(OptionalInstancedObjectArrayValueProperty->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(OptionalInstancedObjectArrayValueProperty->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(OptionalInstancedObjectArrayValueProperty->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		CHECK(OptionalInstancedObjectArrayValueProperty->GetBoolMetaData(TEXT("EditInline")));
#endif

		REQUIRE(OptionalInstancedObjectArrayValueProperty->Inner != nullptr);
		CHECK(OptionalInstancedObjectArrayValueProperty->Inner->IsA<FObjectProperty>());

		CHECK(OptionalInstancedObjectArrayValueProperty->Inner->ContainsInstancedObjectProperty());
		CHECK_FALSE(OptionalInstancedObjectArrayValueProperty->Inner->HasAllPropertyFlags(CPF_ContainsInstancedReference));
		CHECK(OptionalInstancedObjectArrayValueProperty->Inner->HasAllPropertyFlags(CPF_InstancedReference));
		CHECK(OptionalInstancedObjectArrayValueProperty->Inner->HasAllPropertyFlags(CPF_PersistentInstance));
#if WITH_METADATA
		// Note: Currently 'EditInline' does not propagate to the array's inner property.
		CHECK_FALSE(OptionalInstancedObjectArrayValueProperty->Inner->GetBoolMetaData(TEXT("EditInline")));
#endif
	}

	SECTION("Optional instanced object property - unset semantics")
	{
		const FName OptionalNullableInstancedObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, OptionalNullableInnerObjectUnset);
		const FOptionalProperty* OptionalNullableInstancedObjectProperty = CastField<FOptionalProperty>(USubobjectInstancingTestOuterObject::StaticClass()->FindPropertyByName(OptionalNullableInstancedObjectPropertyName));

		REQUIRE(OptionalNullableInstancedObjectProperty != nullptr);

		const FProperty* ValueProperty = OptionalNullableInstancedObjectProperty->GetValueProperty();
		REQUIRE(ValueProperty != nullptr);

		CHECK(ValueProperty->IsA<FObjectProperty>());
		CHECK_FALSE(ValueProperty->HasIntrusiveUnsetOptionalState());
		CHECK_FALSE(ValueProperty->HasAnyPropertyFlags(CPF_NonNullable));
		CHECK(ValueProperty->HasAnyPropertyFlags(CPF_TObjectPtrWrapper));
	}
}

}

#endif	// WITH_TESTS