// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "SubobjectInstancingTest.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

namespace UE
{

TEST_CASE_NAMED(FObjectInstancingGraphTest, "CoreUObject::FObjectInstancingGraph", "[CoreUObject][EngineFilter]")
{
	SECTION("Default construction")
	{
		FObjectInstancingGraph InstanceGraph;
		CHECK_FALSE(InstanceGraph.HasDestinationRoot());
		CHECK(InstanceGraph.IsSubobjectInstancingEnabled());
	}

	SECTION("Construct with options")
	{
		FObjectInstancingGraph InstanceGraph(EObjectInstancingGraphOptions::DisableInstancing);
		CHECK_FALSE(InstanceGraph.HasDestinationRoot());
		CHECK_FALSE(InstanceGraph.IsSubobjectInstancingEnabled());
	}

	SECTION("Construct with root object")
	{
		USubobjectInstancingTestObject* RootObject = NewObject<USubobjectInstancingTestObject>();

		FObjectInstancingGraph InstanceGraph(RootObject);
		CHECK(InstanceGraph.HasDestinationRoot());
		CHECK(InstanceGraph.GetDestinationRoot() == RootObject);
		CHECK(InstanceGraph.GetDestinationObject(RootObject->GetArchetype()) == RootObject);
	}

	SECTION("Set destination root on first add")
	{
		USubobjectInstancingTestObject* RootObject = NewObject<USubobjectInstancingTestObject>();

		FObjectInstancingGraph InstanceGraph;
		InstanceGraph.AddNewObject(RootObject);
		CHECK(InstanceGraph.HasDestinationRoot());
		CHECK(InstanceGraph.GetDestinationRoot() == RootObject);
		CHECK(InstanceGraph.GetDestinationObject(RootObject->GetArchetype()) == RootObject);
	}

	SECTION("Exclude native class property from instancing when initialized from CDO")
	{
		FObjectInstancingGraph InstanceGraph;

		UClass* TestClass = USubobjectInstancingTestOuterObject::StaticClass();
		const FName InnerObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObject);
		const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(TestClass->FindPropertyByName(InnerObjectPropertyName));

		REQUIRE(InnerObjectProperty != nullptr);
		InstanceGraph.AddPropertyToSubobjectExclusionList(InnerObjectProperty);

		FStaticConstructObjectParameters Params(TestClass);
		Params.Outer = GetTransientPackage();
		Params.InstanceGraph = &InstanceGraph;
		USubobjectInstancingTestOuterObject* OuterObject = CastChecked<USubobjectInstancingTestOuterObject>(StaticConstructObject_Internal(Params));

		UObject* InnerObject = InnerObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

		// Note: Because the class/property is native and we are initializing from the CDO, excluding it will have no effect and you are left with the constructed (instanced) value
		// This differs from the non-native (e.g. Blueprint) case, which will always initialize non-transient object properties from the CDO, and you are left with the default value
		// @see FObjectInitializer::InitProperties()
		REQUIRE(InnerObject != nullptr);
		CHECK(InnerObject == OuterObject->InnerObject);
		CHECK(InnerObject->IsInOuter(OuterObject));
		CHECK_FALSE(InnerObject->IsTemplate());
		CHECK(InnerObject->IsDefaultSubobject());
		CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
	}

	SECTION("Exclude native class property from instancing when initialized from template")
	{
		FObjectInstancingGraph InstanceGraph;

		UClass* TestClass = USubobjectInstancingTestOuterObject::StaticClass();
		const FName InnerObjectPropertyName = GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, InnerObject);
		const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(TestClass->FindPropertyByName(InnerObjectPropertyName));

		REQUIRE(InnerObjectProperty != nullptr);
		InstanceGraph.AddPropertyToSubobjectExclusionList(InnerObjectProperty);

		FStaticConstructObjectParameters Params(TestClass);
		Params.Outer = GetTransientPackage();
		Params.Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), NAME_None, RF_ArchetypeObject);
		Params.InstanceGraph = &InstanceGraph;
		USubobjectInstancingTestOuterObject* OuterObject = CastChecked<USubobjectInstancingTestOuterObject>(StaticConstructObject_Internal(Params));

		UObject* InnerObject = InnerObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

		// Note: The expectation here is that we are left with the default (initialized) value
		REQUIRE(InnerObject != nullptr);
		CHECK(InnerObject == OuterObject->InnerObject);
		CHECK_FALSE(InnerObject->IsInOuter(OuterObject));
		CHECK(InnerObject->IsTemplate());
		CHECK(InnerObject->IsDefaultSubobject());
		CHECK(InnerObject->IsTemplateForSubobjects());
	}

	SECTION("Exclude non-native class property from instancing when initialized from CDO")
	{
		FObjectInstancingGraph InstanceGraph;

		UClass* ParentClass = USubobjectInstancingTestOuterObject::StaticClass();
		UClass* TestClass = FSubobjectInstancingTestUtils::CreateNonNativeInstancingTestClass(ParentClass);
		const FName NonNativeInnerObjectPropertyName = FSubobjectInstancingTestUtils::GetNonNativeInnerObjectPropertyName();
		const FObjectProperty* NonNativeInnerObjectProperty = CastField<FObjectProperty>(TestClass->FindPropertyByName(NonNativeInnerObjectPropertyName));

		REQUIRE(NonNativeInnerObjectProperty != nullptr);
		InstanceGraph.AddPropertyToSubobjectExclusionList(NonNativeInnerObjectProperty);

		FStaticConstructObjectParameters Params(TestClass);
		Params.Outer = GetTransientPackage();
		Params.InstanceGraph = &InstanceGraph;
		USubobjectInstancingTestOuterObject* OuterObject = CastChecked<USubobjectInstancingTestOuterObject>(StaticConstructObject_Internal(Params));

		UObject* NonNativeInnerObject = NonNativeInnerObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

		REQUIRE(NonNativeInnerObject != nullptr);
		CHECK_FALSE(NonNativeInnerObject->IsInOuter(OuterObject));
		CHECK(NonNativeInnerObject->IsTemplate());
		CHECK(NonNativeInnerObject->IsDefaultSubobject());
		CHECK(NonNativeInnerObject->IsTemplateForSubobjects());
	}

	SECTION("Exclude non-native class property from instancing when initialized from template")
	{
		FObjectInstancingGraph InstanceGraph;

		UClass* ParentClass = USubobjectInstancingTestOuterObject::StaticClass();
		UClass* TestClass = FSubobjectInstancingTestUtils::CreateNonNativeInstancingTestClass(ParentClass);
		const FName NonNativeInnerObjectPropertyName = FSubobjectInstancingTestUtils::GetNonNativeInnerObjectPropertyName();
		const FObjectProperty* NonNativeInnerObjectProperty = CastField<FObjectProperty>(TestClass->FindPropertyByName(NonNativeInnerObjectPropertyName));

		REQUIRE(NonNativeInnerObjectProperty != nullptr);
		InstanceGraph.AddPropertyToSubobjectExclusionList(NonNativeInnerObjectProperty);

		FStaticConstructObjectParameters Params(TestClass);
		Params.Outer = GetTransientPackage();
		Params.Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), TestClass, NAME_None, RF_ArchetypeObject);
		Params.InstanceGraph = &InstanceGraph;
		USubobjectInstancingTestOuterObject* OuterObject = CastChecked<USubobjectInstancingTestOuterObject>(StaticConstructObject_Internal(Params));

		UObject* NonNativeInnerObject = NonNativeInnerObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

		REQUIRE(NonNativeInnerObject != nullptr);
		CHECK_FALSE(NonNativeInnerObject->IsInOuter(OuterObject));
		CHECK(NonNativeInnerObject->IsTemplate());
		CHECK(NonNativeInnerObject->IsDefaultSubobject());
		CHECK(NonNativeInnerObject->IsTemplateForSubobjects(RF_ArchetypeObject));
	}
}

}

#endif	// WITH_TESTS
