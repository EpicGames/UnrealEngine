// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPathFunctionsTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyPathFunctionsTest)

#if WITH_TESTS

#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/PropertyPathFunctions.h"
#include "UObject/PropertyPathName.h"

namespace UE
{

TEST_CASE_NAMED(FPropertyPathFunctionsFindPropertyTest, "CoreUObject::PropertyPathFunctions::FindProperty", "[Core][UObject][SmokeFilter]")
{
	SECTION("Found")
	{
		const UStruct* TestType = UTestPropertyPathFunctionsClass::StaticClass();
		for (const FProperty* Property = TestType->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			CHECK(Property == FindPropertyByNameAndTypeName(TestType, Property->GetFName(), FPropertyTypeName(Property)));
		}
	}

	SECTION("Missing")
	{
		const UStruct* TestType = UTestPropertyPathFunctionsClass::StaticClass();
		const FPropertyTypeName FloatType = []
		{
			FPropertyTypeNameBuilder TypeName;
			TypeName.AddName(NAME_FloatProperty);
			return TypeName.Build();
		}();
		for (const FProperty* Property = TestType->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			CHECK_FALSE(FindPropertyByNameAndTypeName(TestType, Property->GetFName(), FloatType));
		}
	}
}

TEST_CASE_NAMED(FPropertyPathFunctionsTryResolveTest, "CoreUObject::PropertyPathFunctions::TryResolve", "[Core][UObject][SmokeFilter]")
{
	UTestPropertyPathFunctionsClass* TestObject = NewObject<UTestPropertyPathFunctionsClass>();
	const UClass* TestClass = StaticClass<UTestPropertyPathFunctionsClass>();
	const UStruct* TestStruct = StaticStruct<FTestPropertyPathFunctionsStruct>();
	const UStruct* TestStructKey = StaticStruct<FTestPropertyPathFunctionsStructKey>();

	FTestPropertyPathFunctionsStruct& StructRef = TestObject->StructStaticArray[3];
	StructRef.Int32 = 0x100;
	StructRef.Int32StaticArray[5] = 0x101;
	StructRef.Int32Array = {0x102, 0x103, 0x104, 0x105};
	StructRef.Int32Set = {0x106, 0x107, 0x108, 0x109};
	StructRef.Int32Map = {{0x10a, 0x10b}, {0x10c, 0x10d}, {0x10e, 0x10f}};
	StructRef.Int32Optional = 0x110;
	StructRef.StructOptional = FTestPropertyPathFunctionsStructKey{ 0x111 };

	TestObject->StructArray.AddDefaulted(3);
	TestObject->StructArray.Add(StructRef);
	TestObject->StructSet = {{0x100}, {0x101}, {0x102}, {0x103}};
	TestObject->StructMap = {{{0x104}, {}}, {{0x105}, {}}, {{0x106}, {}}, {{0x107}, StructRef}};
	TestObject->StructOptional = StructRef;

	struct FPropertyReference
	{
		const UStruct* Struct = nullptr;
		const ANSICHAR* PropertyName = nullptr;
		int32 ArrayIndex = INDEX_NONE;
	};

	const auto MakePath = [](std::initializer_list<FPropertyReference> Segments) -> FPropertyPathName
	{
		FPropertyPathName PathName;
		for (const FPropertyReference& Segment : Segments)
		{
			FName PropertyName = Segment.PropertyName;
			FPropertyTypeName TypeName;
			if (Segment.Struct)
			{
				FProperty* Property = Segment.Struct->FindPropertyByName(PropertyName);
				CAPTURE(Segment.PropertyName);
				CHECK(Property);
				if (Property)
				{
					TypeName = FPropertyTypeName(Property);
				}
			}
			PathName.Push({PropertyName, TypeName, Segment.ArrayIndex});
		}
		return PathName;
	};

	FPropertyValueInContainer Value;

	// Resolve Empty Path
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({}), TestObject)));

	// Resolve Null Object
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructArray"}}), nullptr)));

	// Resolve Int32
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestStruct, "Int32"}}), TestObject)));

	// Resolve StructStaticArray
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray"}}), TestObject)));

	// Resolve StructStaticArray[3]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}}), TestObject)));
	CHECK(Value.ArrayIndex == 3);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructStaticArray");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructStaticArray[3]);

	// Resolve StructStaticArray[3] -> Int32
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32);
	CHECK(*Value.GetValuePtr<int32>() == 0x100);

	// Resolve StructStaticArray[3] -> Int32StaticArray[5]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32StaticArray", 5}}), TestObject)));
	CHECK(Value.ArrayIndex == 5);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32StaticArray");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32StaticArray[5]);
	CHECK(*Value.GetValuePtr<int32>() == 0x101);

	// Resolve StructStaticArray[3] -> Int32Array[1]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Array", 1}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "Int32Array");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32Array[1]);
	CHECK(*Value.GetValuePtr<int32>() == 0x103);

	// Resolve StructStaticArray[3] -> Int32Set[1]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Set", 1}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "Int32Set");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32Set[FSetElementId::FromInteger(1)]);
	CHECK(*Value.GetValuePtr<int32>() == 0x107);

	// Resolve StructStaticArray[3] -> Int32Map[1]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Map", 1}}), TestObject)));

	// Resolve StructStaticArray[3] -> Int32Map[1] -> Key
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Map", 1}, {nullptr, "Key"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "Int32Map_Key");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32Map.Get(FSetElementId::FromInteger(1)).Key);
	CHECK(*Value.GetValuePtr<int32>() == 0x10c);

	// Resolve StructStaticArray[3] -> Int32Map[1] -> Value
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Map", 1}, {nullptr, "Value"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "Int32Map");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructStaticArray[3].Int32Map.Get(FSetElementId::FromInteger(1)).Value);
	CHECK(*Value.GetValuePtr<int32>() == 0x10d);

	// Resolve StructStaticArray[3] -> Int32Optional
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32Optional"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32Optional");
	CHECK(Value.GetValuePtr<TOptional<int32>>() == &TestObject->StructStaticArray[3].Int32Optional);
	CHECK(*Value.GetValuePtr<TOptional<int32>>() == 0x110);

	// Resolve StructStaticArray[3] -> Int32[0]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 3}, {TestStruct, "Int32", 0}}), TestObject)));

	// Resolve StructStaticArray[9]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructStaticArray", 9}}), TestObject)));

	// Resolve StructArray
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructArray"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructArray");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructArray);

	// Resolve StructArray[3]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructArray", 3}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "StructArray");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructArray[3]);

	// Resolve StructArray[3] -> Int32StaticArray[5]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructArray", 3}, {TestStruct, "Int32StaticArray", 5}}), TestObject)));
	CHECK(Value.ArrayIndex == 5);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32StaticArray");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructArray[3].Int32StaticArray[5]);
	CHECK(*Value.GetValuePtr<int32>() == 0x101);

	// Resolve StructArray[4]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructArray", 4}}), TestObject)));

	// Resolve StructSet
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructSet"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructSet");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructSet);

	// Resolve StructSet[3]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructSet", 3}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "StructSet");
	CHECK(Value.GetValuePtr<FTestPropertyPathFunctionsStructKey>() == &TestObject->StructSet[FSetElementId::FromInteger(3)]);
	CHECK(Value.GetValuePtr<FTestPropertyPathFunctionsStructKey>()->Key == 0x103);

	// Resolve StructSet[3] -> Key
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructSet", 3}, {TestStructKey, "Key"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStructKey);
	CHECK(Value.Property->GetFName() == "Key");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructSet[FSetElementId::FromInteger(3)].Key);
	CHECK(*Value.GetValuePtr<int32>() == 0x103);

	// Resolve StructSet[4]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructSet", 4}}), TestObject)));

	// Resolve StructMap
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructMap");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructMap);

	// Resolve StructMap[3]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}}), TestObject)));

	// Resolve StructMap[3] -> Int32
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Int32"}}), TestObject)));

	// Resolve StructMap[3] -> Int32(IntProperty)
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {TestStruct, "Int32"}}), TestObject)));

	// Resolve StructMap[3] -> Int32StaticArray[5]
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Int32StaticArray", 5}}), TestObject)));

	// Resolve StructMap[3] -> Key
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Key"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "StructMap_Key");
	CHECK(Value.GetValuePtr<FTestPropertyPathFunctionsStructKey>() == &TestObject->StructMap.Get(FSetElementId::FromInteger(3)).Key);
	CHECK(Value.GetValuePtr<FTestPropertyPathFunctionsStructKey>()->Key == 0x107);

	// Resolve StructMap[3] -> Key -> Key
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Key"}, {TestStructKey, "Key"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStructKey);
	CHECK(Value.Property->GetFName() == "Key");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructMap.Get(FSetElementId::FromInteger(3)).Key.Key);
	CHECK(*Value.GetValuePtr<int32>() == 0x107);

	// Resolve StructMap[3] -> Value
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Value"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == nullptr);
	CHECK(Value.Property->GetFName() == "StructMap");
	CHECK(Value.GetValuePtr<FTestPropertyPathFunctionsStruct>() == &TestObject->StructMap.Get(FSetElementId::FromInteger(3)).Value);

	// Resolve StructMap[3] -> Value -> Int32StaticArray[5]
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Value"}, {TestStruct, "Int32StaticArray", 5}}), TestObject)));
	CHECK(Value.ArrayIndex == 5);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32StaticArray");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructMap.Get(FSetElementId::FromInteger(3)).Value.Int32StaticArray[5]);
	CHECK(*Value.GetValuePtr<int32>() == 0x101);

	// Resolve StructMap[3] -> Int32
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 3}, {nullptr, "Value"}, {TestStruct, "Int32"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructMap.Get(FSetElementId::FromInteger(3)).Value.Int32);
	CHECK(*Value.GetValuePtr<int32>() == 0x100);

	// Resolve StructMap[4] -> Key
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructMap", 4}, {nullptr, "Key"}}), TestObject)));

	// Resolve StructOptional
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructOptional"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructOptional");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructOptional);

	// Resolve StructOptional -> Int32
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructOptional"}, {TestStruct, "Int32"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestStruct);
	CHECK(Value.Property->GetFName() == "Int32");
	CHECK(Value.GetValuePtr<int32>() == &TestObject->StructOptional->Int32);
	CHECK(*Value.GetValuePtr<int32>() == 0x100);

	TestObject->StructOptional.Reset();

	// Resolve unset StructOptional
	REQUIRE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructOptional"}}), TestObject)));
	CHECK(Value.ArrayIndex == 0);
	CHECK(Value.Struct == TestClass);
	CHECK(Value.Property->GetFName() == "StructOptional");
	CHECK(Value.GetValuePtr<void>() == &TestObject->StructOptional);

	// Resolve unset StructOptional -> Int32
	CHECK_FALSE((Value = TryResolvePropertyPath(MakePath({{TestClass, "StructOptional"}, {TestStruct, "Int32"}}), TestObject)));


	// WILDCARDS
	constexpr int32 StructStaticArraySize = UE_ARRAY_COUNT(UTestPropertyPathFunctionsClass::StructStaticArray);
	constexpr int32 Int32StaticArraySize = UE_ARRAY_COUNT(FTestPropertyPathFunctionsStruct::Int32StaticArray);
	
	// Resolve StructStaticArray[*] -> Int32Array
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructStaticArray", INDEX_NONE}, {TestStruct, "Int32Array"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == TestStruct);
			CHECK(Resolver.Value.Property->GetFName() == "Int32Array");
			CHECK(Resolver.Value.GetValuePtr<void>() == &TestObject->StructStaticArray[FoundCount].Int32Array);
			++FoundCount;
			Resolver.Next();
		}
		CHECK(FoundCount == StructStaticArraySize);
	}

	// Resolve StructStaticArray[*] -> Int32Map[*] -> Key
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructStaticArray", INDEX_NONE}, {TestStruct, "Int32Map", INDEX_NONE}, {nullptr, "Key"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		TMap<int32, int32>::TConstIterator ExpectedDataIterator = StructRef.Int32Map.CreateConstIterator();
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == nullptr);
			CHECK(Resolver.Value.Property->GetFName() == "Int32Map_Key");
			CHECK(*Resolver.Value.GetValuePtr<int32>() == ExpectedDataIterator->Key);
			++FoundCount;
			++ExpectedDataIterator;
			Resolver.Next();
		}
		CHECK(FoundCount == 1 * StructRef.Int32Map.Num()); // only StructStaticArray[3].Int32Map was given elements
	}

	// Resolve StructStaticArray[*] -> Int32Map[*] -> Value
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructStaticArray", INDEX_NONE}, {TestStruct, "Int32Map", INDEX_NONE}, {nullptr, "Value"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		TMap<int32, int32>::TConstIterator ExpectedDataIterator = StructRef.Int32Map.CreateConstIterator();
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == nullptr);
			CHECK(Resolver.Value.Property->GetFName() == "Int32Map");
			CHECK(*Resolver.Value.GetValuePtr<int32>() == ExpectedDataIterator->Value);
			++FoundCount;
			++ExpectedDataIterator;
			Resolver.Next();
		}
		CHECK(FoundCount == 1 * StructRef.Int32Map.Num()); // only StructStaticArray[3].Int32Map was given elements
	}

	// Resolve StructStaticArray[*] -> Int32Optional
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructStaticArray", INDEX_NONE}, {TestStruct, "Int32Optional"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == TestStruct);
			CHECK(Resolver.Value.Property->GetFName() == "Int32Optional");
			CHECK(Resolver.Value.GetValuePtr<TOptional<int32>>() == &TestObject->StructStaticArray[FoundCount].Int32Optional);
			++FoundCount;
			Resolver.Next();
		}
		CHECK(FoundCount == StructStaticArraySize);
	}

	// Resolve StructStaticArray[*] -> StructOptional -> Key
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructStaticArray", INDEX_NONE}, {TestStruct, "StructOptional"}, {TestStructKey, "Key"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == TestStructKey);
			CHECK(Resolver.Value.Property->GetFName() == "Key");
			CHECK(Resolver.Value.GetValuePtr<int32>() == &StructRef.StructOptional->Key);
			++FoundCount;
			Resolver.Next();
		}
		CHECK(FoundCount == 1); // only StructStaticArray[3].StructOptional was given a value
	}

	// Resolve StructSet[*] -> Key
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructSet", INDEX_NONE}, {TestStructKey, "Key"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		TSet<FTestPropertyPathFunctionsStructKey>::TConstIterator ExpectedDataIterator = TestObject->StructSet.CreateConstIterator();
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == TestStructKey);
			CHECK(Resolver.Value.Property->GetFName() == "Key");
			CHECK(Resolver.Value.GetValuePtr<int32>() == &ExpectedDataIterator->Key);
			CHECK(*Resolver.Value.GetValuePtr<int32>() == ExpectedDataIterator->Key);
			++FoundCount;
			++ExpectedDataIterator;
			Resolver.Next();
		}
		CHECK(FoundCount == TestObject->StructSet.Num());
	}

	// Resolve StructMap[*] -> Key -> Key
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructMap", INDEX_NONE}, {nullptr, "Key"}, {TestStructKey, "Key"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		TMap<FTestPropertyPathFunctionsStructKey, FTestPropertyPathFunctionsStruct>::TConstIterator ExpectedDataIterator = TestObject->StructMap.CreateConstIterator();
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == TestStructKey);
			CHECK(Resolver.Value.Property->GetFName() == "Key");
			CHECK(Resolver.Value.GetValuePtr<int32>() == &ExpectedDataIterator->Key.Key);
			CHECK(*Resolver.Value.GetValuePtr<int32>() == ExpectedDataIterator->Key.Key);
			++FoundCount;
			++ExpectedDataIterator;
			Resolver.Next();
		}
		CHECK(FoundCount == TestObject->StructMap.Num());
	}

	// Resolve StructMap[*] -> Value
	{
		FPropertyPathName Path = MakePath({ {TestClass, "StructMap", INDEX_NONE}, {nullptr, "Value"} });
		FPropertyPathNameResolver Resolver(Path, TestObject);
		REQUIRE(Resolver);
		int32 FoundCount = 0;
		TMap<FTestPropertyPathFunctionsStructKey, FTestPropertyPathFunctionsStruct>::TConstIterator ExpectedDataIterator = TestObject->StructMap.CreateConstIterator();
		while (Resolver)
		{
			CHECK(Resolver.Value.ArrayIndex == 0);
			CHECK(Resolver.Value.Struct == nullptr);
			CHECK(Resolver.Value.Property->GetFName() == "StructMap");
			CHECK(Resolver.Value.GetValuePtr<void>() == &ExpectedDataIterator->Value);
			++FoundCount;
			++ExpectedDataIterator;
			Resolver.Next();
		}
		CHECK(FoundCount == TestObject->StructMap.Num());
	}
}

} // UE

#endif // WITH_TESTS
