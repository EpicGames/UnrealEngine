// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/Optional.h"
#include "Templates/TypeHash.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "PropertyPathFunctionsTest.generated.h"

#if WITH_TESTS

USTRUCT()
struct FTestPropertyPathFunctionsStructKey
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 Unused = -1;

	UPROPERTY()
	int32 Key = 0;

	FTestPropertyPathFunctionsStructKey() = default;

	inline FTestPropertyPathFunctionsStructKey(int32 InKey)
		: Key(InKey)
	{
	}

	inline bool operator==(const FTestPropertyPathFunctionsStructKey& Other) const
	{
		return Key == Other.Key;
	}

	friend inline uint32 GetTypeHash(const FTestPropertyPathFunctionsStructKey& Value)
	{
		return GetTypeHashHelper(Value.Key);
	}
};

USTRUCT()
struct FTestPropertyPathFunctionsStruct
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 Unused = -1;

	UPROPERTY()
	int32 Int32 = 0;

	UPROPERTY()
	int32 Int32StaticArray[8]{};

	UPROPERTY()
	TArray<int32> Int32Array;

	UPROPERTY()
	TSet<int32> Int32Set;

	UPROPERTY()
	TMap<int32, int32> Int32Map;

	UPROPERTY()
	TOptional<int32> Int32Optional;

	UPROPERTY()
	TOptional<FTestPropertyPathFunctionsStructKey> StructOptional;
};

UCLASS()
class UTestPropertyPathFunctionsClass : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FTestPropertyPathFunctionsStruct StructStaticArray[8];

	UPROPERTY()
	TArray<FTestPropertyPathFunctionsStruct> StructArray;

	UPROPERTY()
	TSet<FTestPropertyPathFunctionsStructKey> StructSet;

	UPROPERTY()
	TMap<FTestPropertyPathFunctionsStructKey, FTestPropertyPathFunctionsStruct> StructMap;

	UPROPERTY()
	TOptional<FTestPropertyPathFunctionsStruct> StructOptional;
};

#endif // WITH_TESTS
