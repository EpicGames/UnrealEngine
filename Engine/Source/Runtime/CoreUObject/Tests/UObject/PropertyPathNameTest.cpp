// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyPathName.h"

#if WITH_TESTS

#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FPropertyPathNameTest, "CoreUObject::PropertyPathName", "[Core][UObject][SmokeFilter]")
{
	const FName CountName(TEXTVIEW("Count"));

	FPropertyTypeNameBuilder TypeBuilder;
	TypeBuilder.AddName(NAME_IntProperty);
	const FPropertyTypeName IntType = TypeBuilder.Build();

	SECTION("Empty")
	{
		FPropertyPathName PathName;
		CHECK(PathName.IsEmpty());
		CHECK(PathName.GetSegmentCount() == 0);
		CHECK(WriteToString<16>(PathName).Len() == 0);
	}

	SECTION("Name")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName});
		CHECK(!PathName.IsEmpty());
		CHECK(PathName.GetSegmentCount() == 1);
		CHECK(PathName.GetSegment(0).Name == CountName);
		CHECK(PathName.GetSegment(0).Type.IsEmpty());
		CHECK(PathName.GetSegment(0).Index == INDEX_NONE);
		CHECK(TEXTVIEW("Count").Equals(WriteToString<16>(PathName)));
	}

	SECTION("NameType")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName, IntType});
		CHECK(!PathName.IsEmpty());
		CHECK(PathName.GetSegmentCount() == 1);
		CHECK(PathName.GetSegment(0).Name == CountName);
		CHECK(PathName.GetSegment(0).Type == IntType);
		CHECK(PathName.GetSegment(0).Index == INDEX_NONE);
		CHECK(TEXTVIEW("Count (IntProperty)").Equals(WriteToString<16>(PathName)));
	}

	SECTION("NameTypeIndex")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName, IntType, 7});
		CHECK(!PathName.IsEmpty());
		CHECK(PathName.GetSegmentCount() == 1);
		CHECK(PathName.GetSegment(0).Name == CountName);
		CHECK(PathName.GetSegment(0).Type == IntType);
		CHECK(PathName.GetSegment(0).Index == 7);
		CHECK(TEXTVIEW("Count[7] (IntProperty)").Equals(WriteToString<16>(PathName)));
	}

	SECTION("MultipleSegments")
	{
		FPropertyTypeNameBuilder ArrayTypeBuilder;
		ArrayTypeBuilder.AddName(NAME_ArrayProperty);
		ArrayTypeBuilder.BeginParameters();
		ArrayTypeBuilder.AddName(NAME_StructProperty);
		ArrayTypeBuilder.BeginParameters();
		ArrayTypeBuilder.AddName(TEXT("TestType"));
		ArrayTypeBuilder.EndParameters();
		ArrayTypeBuilder.EndParameters();

		FPropertyTypeNameBuilder VectorTypeBuilder;
		VectorTypeBuilder.AddName(NAME_StructProperty);
		VectorTypeBuilder.BeginParameters();
		VectorTypeBuilder.AddName(TEXT("IntVector"));
		VectorTypeBuilder.EndParameters();

		FPropertyPathName PathName;
		PathName.Push({TEXT("TestArray"), ArrayTypeBuilder.Build(), 7});
		PathName.Push({TEXT("Position"), VectorTypeBuilder.Build()});
		PathName.Push({TEXT("X"), IntType});

		CHECK(PathName.GetSegmentCount() == 3);
		CHECK(TEXTVIEW("TestArray[7] (ArrayProperty(StructProperty(TestType))) -> Position (StructProperty(IntVector)) -> X (IntProperty)").Equals(WriteToString<128>(PathName)));
	}

	SECTION("SetType/SetIndex")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName});
		PathName.Push({CountName});

		PathName.SetIndex(7);
		CHECK(PathName.GetSegment(0).Index == INDEX_NONE);
		CHECK(PathName.GetSegment(1).Index == 7);
	}

	SECTION("Pop/Empty/Reset")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName, IntType, 0});
		PathName.Push({CountName, IntType, 1});
		PathName.Push({CountName, IntType, 2});
		CHECK(PathName.GetSegmentCount() == 3);
		CHECK(PathName.GetSegment(2).Index == 2);

		SECTION("Pop")
		{
			PathName.Pop();
			CHECK(PathName.GetSegmentCount() == 2);
			CHECK(PathName.GetSegment(1).Index == 1);
			PathName.Pop();
			CHECK(PathName.GetSegmentCount() == 1);
			CHECK(PathName.GetSegment(0).Index == 0);
			PathName.Pop();
			CHECK(PathName.GetSegmentCount() == 0);
			CHECK(PathName.IsEmpty());
		}

		SECTION("Empty")
		{
			PathName.Empty();
			CHECK(PathName.GetSegmentCount() == 0);
			CHECK(PathName.IsEmpty());
		}

		SECTION("Reset")
		{
			PathName.Reset();
			CHECK(PathName.GetSegmentCount() == 0);
			CHECK(PathName.IsEmpty());
		}
	}

	SECTION("Equals+Less")
	{
		const FName DepthName(TEXTVIEW("Depth"));
		
		TypeBuilder.Reset();
		TypeBuilder.AddName(NAME_BoolProperty);
		const FPropertyTypeName BoolType = TypeBuilder.Build();

		FPropertyPathName CountBool;
		CountBool.Push({CountName, BoolType});
		FPropertyPathName CountInt;
		CountInt.Push({CountName, IntType});
		FPropertyPathName DepthBool;
		DepthBool.Push({DepthName, BoolType});
		FPropertyPathName DepthInt;
		DepthInt.Push({DepthName, IntType});

		CHECK(CountBool == CountBool);
		CHECK_FALSE(CountBool == CountInt);
		CHECK_FALSE(CountBool == DepthBool);
		CHECK_FALSE(CountBool == DepthInt);

		CHECK_FALSE(CountBool < CountBool);
		CHECK(CountBool < CountInt);
		CHECK(CountBool < DepthBool);
		CHECK(CountBool < DepthInt);

		CHECK_FALSE(CountInt < CountBool);
		CHECK_FALSE(CountInt < CountInt);
		CHECK(CountInt < DepthBool);
		CHECK(CountInt < DepthInt);

		CHECK_FALSE(DepthBool < CountBool);
		CHECK_FALSE(DepthBool < CountInt);
		CHECK_FALSE(DepthBool < DepthBool);
		CHECK(DepthBool < DepthInt);

		CHECK_FALSE(DepthInt < CountBool);
		CHECK_FALSE(DepthInt < CountInt);
		CHECK_FALSE(DepthInt < DepthBool);
		CHECK_FALSE(DepthInt < DepthInt);
	}

	SECTION("GetTypeHash")
	{
		FPropertyPathName Empty, Name, NameType, NameTypeIndex;
		Name.Push({CountName});
		NameType.Push({CountName, IntType});
		NameTypeIndex.Push({CountName, IntType, 7});

		const uint32 EmptyHash = GetTypeHash(Empty);
		const uint32 NameHash = GetTypeHash(Name);
		const uint32 NameTypeHash = GetTypeHash(NameType);
		const uint32 NameTypeIndexHash = GetTypeHash(NameTypeIndex);

		CHECK(EmptyHash == 0);
		CHECK(EmptyHash != NameHash);
		CHECK(EmptyHash != NameTypeHash);
		CHECK(EmptyHash != NameTypeIndexHash);
		CHECK(NameHash != NameTypeHash);
		CHECK(NameTypeHash != NameTypeIndexHash);

		NameTypeIndex.SetIndex(8);
		CHECK(NameTypeIndexHash != GetTypeHash(NameTypeIndex));
	}
}

} // UE

#endif // WITH_TESTS
