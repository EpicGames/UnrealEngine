// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyPathNameTree.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FPropertyPathNameTreeTest, "CoreUObject::PropertyPathNameTree", "[Core][UObject][SmokeFilter]")
{
	const FName CountName(TEXTVIEW("Count"));
	const FName SizeName(TEXTVIEW("Size"));

	FPropertyTypeNameBuilder TypeBuilder;
	TypeBuilder.AddName(NAME_IntProperty);
	const FPropertyTypeName IntType = TypeBuilder.Build();

	TypeBuilder.Reset();
	TypeBuilder.AddName(NAME_FloatProperty);
	const FPropertyTypeName FloatType = TypeBuilder.Build();

	TypeBuilder.Reset();
	TypeBuilder.AddName(NAME_StructProperty);
	TypeBuilder.BeginParameters();
	TypeBuilder.AddName(NAME_Vector);
	TypeBuilder.EndParameters();
	const FPropertyTypeName VectorType = TypeBuilder.Build();

	SECTION("Empty")
	{
		FPropertyPathNameTree Tree;
		CHECK(Tree.IsEmpty());

		FPropertyPathName PathName;
		PathName.Push({CountName});
		Tree.Add(PathName);
		CHECK_FALSE(Tree.IsEmpty());

		Tree.Empty();
		CHECK(Tree.IsEmpty());
	}

	SECTION("Name")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName});
		FPropertyPathNameTree Tree;
		Tree.Add(PathName);
		FPropertyPathNameTree::FNode Node = Tree.Find(PathName);
		CHECK(Node);
		CHECK_FALSE(Node.GetSubTree());
		const FPropertyPathNameTree::FConstIterator First = Tree.CreateConstIterator();
		FPropertyPathNameTree::FConstIterator It = Tree.CreateConstIterator();
		CHECK(First == It);
		CHECK_FALSE(First != It);
		CHECKED_IF(It)
		{
			CHECK(It.GetName() == CountName);
			CHECK(It.GetType().IsEmpty());
			CHECK(It.GetNode());
			CHECK_FALSE(It.GetNode().GetSubTree());
			CHECK_FALSE(It.GetNode().GetTag());
			++It;
			CHECK_FALSE(It);
			CHECK_FALSE(First == It);
			CHECK(First != It);
		}
	}

	SECTION("NameType")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName, IntType});
		FPropertyPathNameTree Tree;
		Tree.Add(PathName);
		FPropertyPathNameTree::FNode Node = Tree.Find(PathName);
		CHECK(Node);
		CHECK_FALSE(Node.GetSubTree());
		const FPropertyPathNameTree::FConstIterator First = Tree.CreateConstIterator();
		FPropertyPathNameTree::FConstIterator It = Tree.CreateConstIterator();
		CHECK(First == It);
		CHECK_FALSE(First != It);
		CHECKED_IF(It)
		{
			CHECK(It.GetName() == CountName);
			CHECK(It.GetType() == IntType);
			CHECK(It.GetNode());
			CHECK_FALSE(It.GetNode().GetSubTree());
			++It;
			CHECK_FALSE(It);
			CHECK_FALSE(First == It);
			CHECK(First != It);
		}
	}

	SECTION("NameTypeIndex")
	{
		FPropertyPathName PathName;
		PathName.Push({CountName, IntType, 7});
		FPropertyPathNameTree Tree;
		Tree.Add(PathName);
		CHECK(Tree.Find(PathName));
		PathName.SetIndex(5);
		CHECK(Tree.Find(PathName));
		PathName.SetIndex(3);
		Tree.Add(PathName);
		PathName.SetIndex(INDEX_NONE);
		FPropertyPathNameTree::FConstNode Node = Tree.Find(PathName);
		CHECK(Node);
		CHECK_FALSE(Node.GetSubTree());
		const FPropertyPathNameTree::FConstIterator First = Tree.CreateConstIterator();
		FPropertyPathNameTree::FConstIterator It = Tree.CreateConstIterator();
		CHECK(First == It);
		CHECK_FALSE(First != It);
		CHECKED_IF(It)
		{
			CHECK(It.GetName() == CountName);
			CHECK(It.GetType() == IntType);
			CHECK(It.GetNode());
			CHECK_FALSE(It.GetNode().GetSubTree());
			++It;
			CHECK_FALSE(It);
			CHECK_FALSE(First == It);
			CHECK(First != It);
		}
	}

	SECTION("SameNameDiffType")
	{
		FPropertyPathName PathNameInt;
		PathNameInt.Push({CountName, IntType});
		FPropertyPathName PathNameFloat;
		PathNameFloat.Push({CountName, FloatType});
		FPropertyPathNameTree Tree;
		Tree.Add(PathNameInt);
		Tree.Add(PathNameFloat);
		CHECK(Tree.Find(PathNameInt));
		CHECK(Tree.Find(PathNameFloat));

		FPropertyPathNameTree::FConstIterator It = Tree.CreateConstIterator();
		CHECKED_IF(It)
		{
			CHECK(It.GetType() == IntType);
			CHECK(It.GetNode());
			CHECK_FALSE(It.GetNode().GetSubTree());
			++It;
			CHECKED_IF(It)
			{
				CHECK(It.GetType() == FloatType);
				CHECK(It.GetNode());
				CHECK_FALSE(It.GetNode().GetSubTree());
				++It;
				CHECK_FALSE(It);
			}
		}
	}

	SECTION("DiffNameSameType")
	{
		FPropertyPathName PathNameCount;
		PathNameCount.Push({CountName, IntType});
		FPropertyPathName PathNameSize;
		PathNameSize.Push({SizeName, IntType});
		FPropertyPathNameTree Tree;
		Tree.Add(PathNameCount);
		Tree.Add(PathNameSize);
		CHECK(Tree.Find(PathNameCount));
		CHECK(Tree.Find(PathNameSize));

		FPropertyPathNameTree::FConstIterator It = Tree.CreateConstIterator();
		CHECKED_IF(It)
		{
			CHECK(It.GetName() == CountName);
			CHECK(It.GetNode());
			CHECK_FALSE(It.GetNode().GetSubTree());
			++It;
			CHECKED_IF(It)
			{
				CHECK(It.GetName() == SizeName);
				CHECK(It.GetNode());
				CHECK_FALSE(It.GetNode().GetSubTree());
				++It;
				CHECK_FALSE(It);
			}
		}
	}

	SECTION("Tree")
	{
		FPropertyPathName ParentPathName;
		ParentPathName.Push({NAME_Vector, VectorType});
		FPropertyPathName PathNameA = ParentPathName;
		PathNameA.Push({CountName, IntType});
		FPropertyPathName PathNameB = ParentPathName;
		PathNameB.Push({SizeName, FloatType});
		FPropertyPathNameTree Tree;
		Tree.Add(PathNameA);
		Tree.Add(PathNameB);
		CHECK(Tree.Find(ParentPathName));
		CHECK(Tree.Find(PathNameA));
		CHECK(Tree.Find(PathNameB));

		FPropertyPathNameTree::FConstIterator ParentIt = Tree.CreateConstIterator();
		CHECKED_IF(ParentIt)
		{
			CHECK(ParentIt.GetName() == NAME_Vector);
			CHECK(ParentIt.GetType() == VectorType);
			const FPropertyPathNameTree* ChildTree = ParentIt.GetNode().GetSubTree();
			++ParentIt;
			CHECK_FALSE(ParentIt);
			CHECKED_IF(ChildTree)
			{
				FPropertyPathNameTree::FConstIterator ChildIt = ChildTree->CreateConstIterator();
				CHECKED_IF(ChildIt)
				{
					CHECK(ChildIt.GetName() == CountName);
					CHECK(ChildIt.GetNode());
					CHECK_FALSE(ChildIt.GetNode().GetSubTree());
					++ChildIt;
					CHECKED_IF(ChildIt)
					{
						CHECK(ChildIt.GetName() == SizeName);
						CHECK(ChildIt.GetNode());
						CHECK_FALSE(ChildIt.GetNode().GetSubTree());
						++ChildIt;
						CHECK_FALSE(ChildIt);
					}
				}
			}
		}

		FPropertyPathNameTree::FNode ChildNode = Tree.Find(ParentPathName);
		CHECK(ChildNode);
		const FPropertyPathNameTree* ChildTree = ChildNode.GetSubTree();
		CHECKED_IF(ChildTree)
		{
			CHECK(ChildTree->Find(PathNameA, 1));
			CHECK(ChildTree->Find(PathNameB, 1));
		}

		CHECK_FALSE(Tree.Find(PathNameA, 1));
	}

	SECTION("Remove")
	{
		FPropertyPathName ParentPathName;
		ParentPathName.Push({ NAME_Vector, VectorType });
		FPropertyPathName PathNameA = ParentPathName;
		PathNameA.Push({ CountName, IntType });
		FPropertyPathName PathNameB = ParentPathName;
		PathNameB.Push({ SizeName, FloatType });
		FPropertyPathNameTree Tree;
		Tree.Add(PathNameA);
		Tree.Add(PathNameB);
		CHECKED_IF(Tree.Remove(PathNameA))
		{
			// Because B is also child of Parent, it shouldn'tve been removed yet.
			CHECK(Tree.Find(ParentPathName));

			CHECKED_IF(Tree.Remove(PathNameB))
			{
				// Because B was the last child of ParentPathName, the parent should also get removed.
				CHECK_FALSE(Tree.Find(ParentPathName));

			}
		}
	}
}

} // UE

#endif // WITH_TESTS
