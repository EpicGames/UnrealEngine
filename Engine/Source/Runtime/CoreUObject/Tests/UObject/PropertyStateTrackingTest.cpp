// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyStateTrackingTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyStateTrackingTest)

#if WITH_TESTS && WITH_EDITORONLY_DATA

#include "Logging/LogScopedVerbosityOverride.h"
#include "Serialization/Formatters/BinaryArchiveFormatter.h"
#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/StructuredArchive.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/Package.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyPathNameTree.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/UObjectThreadContext.h"

namespace UE
{

TEST_CASE_NAMED(FPropertyValueFlagsTest, "CoreUObject::Serialization::PropertyValueFlags", "[CoreUObject][EngineFilter]")
{
	const auto SavePropertyValueInitializedFlags = [](FInitializedPropertyValueState& InitializedState, TArray<uint8>& OutFlags)
	{
		FMemoryWriter Ar(OutFlags, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		InitializedState.Serialize(StructuredAr.Open().EnterRecord());
	};

	const auto LoadPropertyValueInitializedFlags = [](FInitializedPropertyValueState& InitializedState, TArray<uint8>& OutFlags)
	{
		FMemoryReader Ar(OutFlags, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		InitializedState.Serialize(StructuredAr.Open().EnterRecord());
	};

	UTestInstanceDataObjectClass* BaseObject = NewObject<UTestInstanceDataObjectClass>();

	// Test BaseObject w/o IDOs
	{
		FInitializedPropertyValueState InitializedState(BaseObject);

		CHECK_FALSE(InitializedState.IsTracking());

		// Test serialization of empty flags.
		{
			TArray<uint8> FlagsData;
			SavePropertyValueInitializedFlags(InitializedState, FlagsData);
			LoadPropertyValueInitializedFlags(InitializedState, FlagsData);
			CHECK_FALSE(InitializedState.IsTracking());
		}

		CHECK(InitializedState.ActivateTracking());
		CHECK(InitializedState.IsTracking());

		const UClass* SearchClass = BaseObject->GetClass();
		FIntProperty* AProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("A"));
		FFloatProperty* BProperty = FindFProperty<FFloatProperty>(SearchClass, TEXT("B"));
		FInt64Property* CProperty = FindFProperty<FInt64Property>(SearchClass, TEXT("C"));
		FIntProperty* DProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("D"));
		FIntProperty* EProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("E"));
		REQUIRE(AProperty);
		REQUIRE(BProperty);
		REQUIRE(CProperty);
		REQUIRE(DProperty);
		REQUIRE(EProperty);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK_FALSE(InitializedState.IsSet(EProperty));

		InitializedState.Set(AProperty, 1);
		InitializedState.Set(AProperty, 3);
		InitializedState.Set(CProperty);
		InitializedState.Set(EProperty);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK(InitializedState.IsSet(EProperty));

		TArray<uint8> FlagsData;
		SavePropertyValueInitializedFlags(InitializedState, FlagsData);

		InitializedState.Reset();

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK_FALSE(InitializedState.IsSet(EProperty));

		LoadPropertyValueInitializedFlags(InitializedState, FlagsData);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK(InitializedState.IsSet(EProperty));

		UTestInstanceDataObjectClass* EmptyObject = NewObject<UTestInstanceDataObjectClass>();
		FInitializedPropertyValueState EmptyInitializedState(EmptyObject);

		LoadPropertyValueInitializedFlags(EmptyInitializedState, FlagsData);

		CHECK_FALSE(EmptyInitializedState.IsSet(AProperty, 0));
		CHECK(EmptyInitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(EmptyInitializedState.IsSet(AProperty, 2));
		CHECK(EmptyInitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(EmptyInitializedState.IsSet(BProperty));
		CHECK(EmptyInitializedState.IsSet(CProperty));
		CHECK_FALSE(EmptyInitializedState.IsSet(DProperty));
		CHECK(EmptyInitializedState.IsSet(EProperty));
	}

	// Create IDO for BaseObject

	UClass* TestClass = CreateInstanceDataObjectClass(nullptr, nullptr, BaseObject->GetClass(), BaseObject->GetOuter());

	FName TestObjectName = MakeUniqueObjectName(nullptr, TestClass, FName(WriteToString<128>(TestClass->GetFName(), TEXT("_Instance"))));
	UObject* TestObject = NewObject<UObject>(GetTransientPackage(), TestClass, TestObjectName);

	// Test BaseObject w/ IDOs
	{
		FInitializedPropertyValueState InitializedState(TestObject);

		InitializedState.Reset();

		CHECK(InitializedState.ActivateTracking());
		CHECK(InitializedState.IsTracking());

		const UClass* SearchClass = TestObject->GetClass();
		FIntProperty* AProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("A"));
		FFloatProperty* BProperty = FindFProperty<FFloatProperty>(SearchClass, TEXT("B"));
		FInt64Property* CProperty = FindFProperty<FInt64Property>(SearchClass, TEXT("C"));
		FIntProperty* DProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("D"));
		FIntProperty* EProperty = FindFProperty<FIntProperty>(SearchClass, TEXT("E"));
		REQUIRE(AProperty);
		REQUIRE(BProperty);
		REQUIRE(CProperty);
		REQUIRE(DProperty);
		REQUIRE(EProperty);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK_FALSE(InitializedState.IsSet(EProperty));

		InitializedState.Set(AProperty, 1);
		InitializedState.Set(AProperty, 3);
		InitializedState.Set(CProperty);
		InitializedState.Set(EProperty);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK(InitializedState.IsSet(EProperty));

		TArray<uint8> FlagsData;
		SavePropertyValueInitializedFlags(InitializedState, FlagsData);

		InitializedState.Reset();

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK_FALSE(InitializedState.IsSet(EProperty));

		LoadPropertyValueInitializedFlags(InitializedState, FlagsData);

		CHECK_FALSE(InitializedState.IsSet(AProperty, 0));
		CHECK(InitializedState.IsSet(AProperty, 1));
		CHECK_FALSE(InitializedState.IsSet(AProperty, 2));
		CHECK(InitializedState.IsSet(AProperty, 3));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));
		CHECK(InitializedState.IsSet(EProperty));
	}

	// Test BaseObject.Struct with IDOs
	{
		FStructProperty* StructProperty = FindFProperty<FStructProperty>(TestClass, TEXT("Struct"));
		REQUIRE(StructProperty);
		REQUIRE(StructProperty->Struct);
		void* StructData = StructProperty->ContainerPtrToValuePtr<void>(TestObject);

		FInitializedPropertyValueState InitializedState(StructProperty->Struct, StructData);

		InitializedState.Reset();

		FIntProperty* AProperty = FindFProperty<FIntProperty>(StructProperty->Struct, TEXT("A"));
		FIntProperty* BProperty = FindFProperty<FIntProperty>(StructProperty->Struct, TEXT("B"));
		FIntProperty* CProperty = FindFProperty<FIntProperty>(StructProperty->Struct, TEXT("C"));
		FIntProperty* DProperty = FindFProperty<FIntProperty>(StructProperty->Struct, TEXT("D"));
		REQUIRE(AProperty);
		REQUIRE(BProperty);
		REQUIRE(CProperty);
		REQUIRE(DProperty);

		CHECK_FALSE(InitializedState.IsSet(AProperty));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));

		InitializedState.Set(AProperty);
		InitializedState.Set(CProperty);
		CHECK(InitializedState.IsSet(AProperty));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));

		InitializedState.Clear(AProperty);
		InitializedState.Clear(CProperty);
		CHECK_FALSE(InitializedState.IsSet(AProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));

		InitializedState.Set(BProperty);
		InitializedState.Set(DProperty);

		TArray<uint8> FlagsData;
		SavePropertyValueInitializedFlags(InitializedState, FlagsData);

		InitializedState.Reset();
		CHECK_FALSE(InitializedState.IsSet(AProperty));
		CHECK_FALSE(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK_FALSE(InitializedState.IsSet(DProperty));

		LoadPropertyValueInitializedFlags(InitializedState, FlagsData);

		CHECK_FALSE(InitializedState.IsSet(AProperty));
		CHECK(InitializedState.IsSet(BProperty));
		CHECK_FALSE(InitializedState.IsSet(CProperty));
		CHECK(InitializedState.IsSet(DProperty));
	}
}

TEST_CASE_NAMED(FInstanceDataObjectEnumsTest, "CoreUObject::Serialization::InstanceDataObjectEnums", "[CoreUObject][EngineFilter]")
{
	UTestInstanceDataObjectClass* BaseObject = NewObject<UTestInstanceDataObjectClass>();

	FUnknownEnumNames UnknownEnumNames(BaseObject);
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectBird>(), {}, "TIDOB_Pigeon");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectGrain::Type>(), {}, "Rye");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFruit>(), {}, "Cherry");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectDirection>(), {}, "Up");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFullFlags>(), {}, "Flag3");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFullFlags>(), {}, "Flag8");
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFullFlags>(), {}, "Flag9");

	UClass* TestClass = CreateInstanceDataObjectClass(nullptr, &UnknownEnumNames, BaseObject->GetClass(), BaseObject->GetOuter());

	FName TestObjectName = MakeUniqueObjectName(nullptr, TestClass, FName(WriteToString<128>(TestClass->GetFName(), TEXT("_Instance"))));
	UObject* TestObject = NewObject<UObject>(GetTransientPackage(), TestClass, TestObjectName);

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(TestClass, TEXT("Struct"));
	REQUIRE(StructProperty);
	REQUIRE(StructProperty->Struct);

	FByteProperty* BirdProperty = FindFProperty<FByteProperty>(StructProperty->Struct, TEXT("Bird"));
	FByteProperty* GrainProperty = FindFProperty<FByteProperty>(StructProperty->Struct, TEXT("Grain"));
	FEnumProperty* FruitProperty = FindFProperty<FEnumProperty>(StructProperty->Struct, TEXT("Fruit"));
	FEnumProperty* DirectionProperty = FindFProperty<FEnumProperty>(StructProperty->Struct, TEXT("Direction"));
	FEnumProperty* FullFlagsProperty = FindFProperty<FEnumProperty>(StructProperty->Struct, TEXT("FullFlags"));
	REQUIRE(BirdProperty);
	REQUIRE(GrainProperty);
	REQUIRE(FruitProperty);
	REQUIRE(DirectionProperty);
	REQUIRE(FullFlagsProperty);

	CHECK(BirdProperty->Enum->GetIndexByName("TIDOB_Pigeon") != INDEX_NONE);
	CHECK(GrainProperty->Enum->GetIndexByName("ETestInstanceDataObjectGrain::Rye") != INDEX_NONE);
	CHECK(FruitProperty->GetEnum()->GetIndexByName("ETestInstanceDataObjectFruit::Cherry") != INDEX_NONE);
	CHECK(DirectionProperty->GetEnum()->GetIndexByName("ETestInstanceDataObjectDirection::Up") != INDEX_NONE);
	CHECK(FullFlagsProperty->GetEnum()->GetIndexByName("ETestInstanceDataObjectFullFlags::Flag3") != INDEX_NONE);
	CHECK(FullFlagsProperty->GetEnum()->GetIndexByName("ETestInstanceDataObjectFullFlags::Flag8") != INDEX_NONE);
	CHECK(FullFlagsProperty->GetEnum()->GetIndexByName("ETestInstanceDataObjectFullFlags::Flag9") != INDEX_NONE);
	CHECK(FullFlagsProperty->GetEnum()->GetMaxEnumValue() == 0b11'1111'1111);
}

TEST_CASE_NAMED(FTrackInitializedPropertiesTest, "CoreUObject::Serialization::TrackInitializedProperties", "[CoreUObject][EngineFilter]")
{
	UObject* TestObject = NewObject<UTestInstanceDataObjectClass>();
	UClass* TestClass = TestObject->GetClass();
	UObject* DefaultObject = TestClass->GetDefaultObject();
	FInitializedPropertyValueState InitializedState(TestObject);
	
	InitializedState.ActivateTracking();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(TestClass, TEXT("Struct"));
	REQUIRE(StructProperty);
	REQUIRE(TestClass);

	FIntProperty* AProperty = FindFProperty<FIntProperty>(TestClass, TEXT("A"));
	FFloatProperty* BProperty = FindFProperty<FFloatProperty>(TestClass, TEXT("B"));
	FInt64Property* CProperty = FindFProperty<FInt64Property>(TestClass, TEXT("C"));
	FIntProperty* DProperty = FindFProperty<FIntProperty>(TestClass, TEXT("D"));
	REQUIRE(AProperty);
	REQUIRE(BProperty);
	REQUIRE(CProperty);
	REQUIRE(DProperty);


	AProperty->SetPropertyValue_InContainer(TestObject, 1, 0);
	AProperty->SetPropertyValue_InContainer(TestObject, 2, 1);
	AProperty->SetPropertyValue_InContainer(TestObject, 3, 2);
	AProperty->SetPropertyValue_InContainer(TestObject, 4, 3);
	BProperty->SetPropertyValue_InContainer(TestObject, 5);
	CProperty->SetPropertyValue_InContainer(TestObject, 6);
	DProperty->SetPropertyValue_InContainer(TestObject, -1);

	InitializedState.Set(AProperty, 0);
	InitializedState.Set(AProperty, 2);
	InitializedState.Set(BProperty);
	InitializedState.Set(DProperty);

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	TGuardValue<bool> TrackInitializedPropertiesScope(SerializeContext->bTrackInitializedProperties, true);

	TArray<uint8> BinaryData;
	{
		FMemoryWriter Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		TestClass->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)TestObject, TestClass, (uint8*)DefaultObject);
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	TArray<uint8> JsonData;
	{
		FMemoryWriter Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveOutputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		TestClass->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)TestObject, TestClass, (uint8*)DefaultObject);
	}
#endif // WITH_TEXT_ARCHIVE_SUPPORT

	AProperty->SetPropertyValue_InContainer(TestObject, 7, 0);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 1);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 2);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 3);
	BProperty->SetPropertyValue_InContainer(TestObject, 7);
	CProperty->SetPropertyValue_InContainer(TestObject, 7);
	DProperty->SetPropertyValue_InContainer(TestObject, 7);

	InitializedState.Reset();

	{
		FMemoryReader Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		TestClass->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)TestObject, TestClass, (uint8*)DefaultObject);
	}

	CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 0) == 1);
	CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 1) == 7); // A[1] unchanged because it was not initialized
	CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 2) == 3);
	CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 3) == 7); // A[3] unchanged because it was not initialized
	CHECK(BProperty->GetPropertyValue_InContainer(TestObject) == 5.0f);
	CHECK(CProperty->GetPropertyValue_InContainer(TestObject) == 7); // C unchanged because it was not initialized
	CHECK(DProperty->GetPropertyValue_InContainer(TestObject) == 7); // D unchanged because it was serialized without its value

	CHECK(InitializedState.IsSet(AProperty, 0));
	CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
	CHECK(InitializedState.IsSet(AProperty, 2));
	CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
	CHECK(InitializedState.IsSet(BProperty));
	CHECK_FALSE(InitializedState.IsSet(CProperty));
	CHECK(InitializedState.IsSet(DProperty));

#if WITH_TEXT_ARCHIVE_SUPPORT
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 0);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 1);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 2);
	AProperty->SetPropertyValue_InContainer(TestObject, 7, 3);
	BProperty->SetPropertyValue_InContainer(TestObject, 7);
	CProperty->SetPropertyValue_InContainer(TestObject, 7);
	DProperty->SetPropertyValue_InContainer(TestObject, 7);

	InitializedState.Reset();

	{
		FMemoryReader Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveInputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		TestClass->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)TestObject, TestClass, (uint8*)DefaultObject);
	}

	CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 0) == 1);
	// Disabled because text serialization behaves unexpectedly in these cases
	//CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 1) == 7); // A[1] unchanged because it was not initialized
	//CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 2) == 3);
	//CHECK(AProperty->GetPropertyValue_InContainer(TestObject, 3) == 7); // A[3] unchanged because it was not initialized
	CHECK(BProperty->GetPropertyValue_InContainer(TestObject) == 5.0f);
	CHECK(CProperty->GetPropertyValue_InContainer(TestObject) == 7); // C unchanged because it was not initialized
	CHECK(DProperty->GetPropertyValue_InContainer(TestObject) == 7); // D unchanged because it was serialized without its value

	CHECK(InitializedState.IsSet(AProperty, 0));
	//CHECK_FALSE(InitializedState.IsSet(AProperty, 1));
	//CHECK(InitializedState.IsSet(AProperty, 2));
	//CHECK_FALSE(InitializedState.IsSet(AProperty, 3));
	CHECK(InitializedState.IsSet(BProperty));
	CHECK_FALSE(InitializedState.IsSet(CProperty));
	CHECK(InitializedState.IsSet(DProperty));
#endif // WITH_TEXT_ARCHIVE_SUPPORT
}

TEST_CASE_NAMED(FTrackUnknownPropertiesTest, "CoreUObject::Serialization::TrackUnknownProperties", "[CoreUObject][EngineFilter]")
{
	const auto MakePropertyTypeName = [](FName Name)
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(Name);
		return Builder.Build();
	};

	UObject* Owner = NewObject<UTestInstanceDataObjectClass>();

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	TGuardValue<UObject*> SerializedObjectScope(SerializeContext->SerializedObject, Owner);
	TGuardValue<bool> TrackSerializedPropertyPathScope(SerializeContext->bTrackSerializedPropertyPath, true);
	TGuardValue<bool> TrackUnknownPropertiesScope(SerializeContext->bTrackUnknownProperties, true);
	TGuardValue<bool> TrackImpersonatePropertiesScope(SerializeContext->bImpersonateProperties, true);
	FSerializedPropertyPathScope SerializedObjectPath(SerializeContext, {"Struct"});

	FTestInstanceDataObjectStructAlternate AltStructData;
	AltStructData.B = 2.5f;
	AltStructData.C = 3;
	AltStructData.D = 4;
	AltStructData.E = 5;
	AltStructData.Bird = TIDOB_Raven;
	AltStructData.Grain = ETestInstanceDataObjectGrainAlternate::Corn;
	AltStructData.Fruit = ETestInstanceDataObjectFruitAlternate::Orange;
	AltStructData.Direction = ETestInstanceDataObjectDirectionAlternate::North | ETestInstanceDataObjectDirectionAlternate::West;
	AltStructData.GrainTypeChange = ETestInstanceDataObjectGrainAlternate::Corn;
	AltStructData.FruitTypeChange = ETestInstanceDataObjectFruitAlternate::Orange;
	AltStructData.GrainTypeAndPropertyChange = ETestInstanceDataObjectGrainAlternateEnumClass::Corn;
	AltStructData.FruitTypeAndPropertyChange = ETestInstanceDataObjectFruitAlternateNamespace::Orange;
	AltStructData.Point.U = 1;
	AltStructData.Point.V = 2;
	AltStructData.Point.W = 3;

	TArray<uint8> BinaryData;
	{
		FMemoryWriter Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStructAlternate::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&AltStructData, nullptr, nullptr);
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	TArray<uint8> JsonData;
	{
		FMemoryWriter Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveOutputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStructAlternate::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&AltStructData, nullptr, nullptr);
	}
#endif // WITH_TEXT_ARCHIVE_SUPPORT

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogClass, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEnum, ELogVerbosity::Error);

	FUnknownPropertyTree TreeAccessor(Owner);
	CHECK(!TreeAccessor.Find());

	FTestInstanceDataObjectStruct StructData;

	{
		FMemoryReader Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStruct::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&StructData, nullptr, nullptr);
	}

	CHECK(StructData.A == -1);
	CHECK(StructData.B == 2);
	CHECK(StructData.C == 3);
	CHECK(StructData.D == 4);
	CHECK(StructData.Bird == TIDOB_Raven);
	CHECK(StructData.Grain == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.Fruit == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.Direction == (ETestInstanceDataObjectDirection::North | ETestInstanceDataObjectDirection::West));
	CHECK(StructData.GrainTypeChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeChange == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.GrainTypeAndPropertyChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeAndPropertyChange == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.Point.X == 0);
	CHECK(StructData.Point.Y == 0);
	CHECK(StructData.Point.Z == 0);
#if WITH_METADATA
	CHECK(StructData.Point.W == 3);
#endif

	TSharedPtr<FPropertyPathNameTree> Tree = TreeAccessor.FindOrCreate();
	REQUIRE(Tree);
	CHECK(TreeAccessor.Find() == Tree);
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"B", MakePropertyTypeName(NAME_FloatProperty)});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"C", MakePropertyTypeName(NAME_Int64Property)});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"E", MakePropertyTypeName(NAME_IntProperty)});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FProperty* PointProperty = FTestInstanceDataObjectStructAlternate::StaticStruct()->FindPropertyByName("Point");
		CHECKED_IF(PointProperty)
		{
			FSerializedPropertyPathScope Path(SerializeContext, {"Point", FPropertyTypeName(PointProperty)});
			CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
		#if WITH_METADATA
			{
				FSerializedPropertyPathScope SubPath(SerializeContext, {"U", MakePropertyTypeName(NAME_IntProperty)});
				CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
			}
			{
				FSerializedPropertyPathScope SubPath(SerializeContext, {"V", MakePropertyTypeName(NAME_IntProperty)});
				CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
			}
			{
				FSerializedPropertyPathScope SubPath(SerializeContext, {"W", MakePropertyTypeName(NAME_IntProperty)});
				CHECK_FALSE(Tree->Find(SerializeContext->SerializedPropertyPath));
			}
		#endif
		}
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"GrainTypeChange", FPropertyTypeName(FindFProperty<FProperty>(StaticStruct<FTestInstanceDataObjectStructAlternate>(), TEXT("GrainTypeChange")))});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"FruitTypeChange", FPropertyTypeName(FindFProperty<FProperty>(StaticStruct<FTestInstanceDataObjectStructAlternate>(), TEXT("FruitTypeChange")))});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"GrainTypeAndPropertyChange", FPropertyTypeName(FindFProperty<FProperty>(StaticStruct<FTestInstanceDataObjectStructAlternate>(), TEXT("GrainTypeAndPropertyChange")))});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	{
		FSerializedPropertyPathScope Path(SerializeContext, {"FruitTypeAndPropertyChange", FPropertyTypeName(FindFProperty<FProperty>(StaticStruct<FTestInstanceDataObjectStructAlternate>(), TEXT("FruitTypeAndPropertyChange")))});
		CHECK(Tree->Find(SerializeContext->SerializedPropertyPath));
	}
	TreeAccessor.Destroy();
	CHECK(!TreeAccessor.Find());

#if WITH_TEXT_ARCHIVE_SUPPORT
	StructData = {};

	{
		FMemoryReader Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveInputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStruct::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&StructData, nullptr, nullptr);
	}

	CHECK(StructData.A == -1);
	CHECK(StructData.B == 2);
	CHECK(StructData.C == 3);
	CHECK(StructData.D == 4);
	CHECK(StructData.Bird == TIDOB_Raven);
	CHECK(StructData.Grain == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.Fruit == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.Direction == (ETestInstanceDataObjectDirection::North | ETestInstanceDataObjectDirection::West));
	CHECK(StructData.GrainTypeChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeChange == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.GrainTypeAndPropertyChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeAndPropertyChange == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.Point.X == 0);
	CHECK(StructData.Point.Y == 0);
	CHECK(StructData.Point.Z == 0);
#if WITH_METADATA
	CHECK(StructData.Point.W == 3);
#endif

	// Testing of the unknown property tree is skipped because it is not supported by the text format.

	TreeAccessor.Destroy();
	CHECK(!TreeAccessor.Find());
#endif // WITH_TEXT_ARCHIVE_SUPPORT
}

TEST_CASE_NAMED(FTrackUnknownEnumNamesTest, "CoreUObject::Serialization::TrackUnknownEnumNames", "[CoreUObject][EngineFilter]")
{
	const auto MakePropertyTypeName = [](const UEnum* Enum)
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddPath(Enum);
		return Builder.Build();
	};

	const auto ParsePropertyTypeName = [](const TCHAR* Name) -> FPropertyTypeName
	{
		FPropertyTypeNameBuilder Builder;
		CHECK(Builder.TryParse(Name));
		return Builder.Build();
	};

	UObject* Owner = NewObject<UTestInstanceDataObjectClass>();

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	TGuardValue<UObject*> SerializedObjectScope(SerializeContext->SerializedObject, Owner);
	TGuardValue<bool> TrackSerializedPropertyPathScope(SerializeContext->bTrackSerializedPropertyPath, true);
	TGuardValue<bool> TrackUnknownPropertiesScope(SerializeContext->bTrackUnknownProperties, true);
	TGuardValue<bool> TrackUnknownEnumNamesScope(SerializeContext->bTrackUnknownEnumNames, true);
	TGuardValue<bool> TrackImpersonatePropertiesScope(SerializeContext->bImpersonateProperties, true);
	FSerializedPropertyPathScope SerializedObjectPath(SerializeContext, {"Struct"});

	FTestInstanceDataObjectStructAlternate AltStructData;
	AltStructData.Grain = ETestInstanceDataObjectGrainAlternate::Rye;
	AltStructData.Fruit = ETestInstanceDataObjectFruitAlternate::Cherry;
	AltStructData.Direction = ETestInstanceDataObjectDirectionAlternate::North | ETestInstanceDataObjectDirectionAlternate::West |
		ETestInstanceDataObjectDirectionAlternate::Up | ETestInstanceDataObjectDirectionAlternate::Down;
	AltStructData.GrainFromEnumClass = ETestInstanceDataObjectGrainAlternateEnumClass::Corn;
	AltStructData.FruitFromNamespace = ETestInstanceDataObjectFruitAlternateNamespace::Orange;
	AltStructData.GrainTypeChange = ETestInstanceDataObjectGrainAlternate::Corn;
	AltStructData.FruitTypeChange = ETestInstanceDataObjectFruitAlternate::Orange;
	AltStructData.DeletedGrain = ETestInstanceDataObjectGrainAlternate::Rice;
	AltStructData.DeletedFruit = ETestInstanceDataObjectFruitAlternate::Apple;
	AltStructData.DeletedDirection = ETestInstanceDataObjectDirectionAlternate::South | ETestInstanceDataObjectDirectionAlternate::Up;

	TArray<uint8> BinaryData;
	{
		FMemoryWriter Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStructAlternate::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&AltStructData, nullptr, nullptr);
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	TArray<uint8> JsonData;
	{
		FMemoryWriter Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveOutputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStructAlternate::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&AltStructData, nullptr, nullptr);
	}
#endif // WITH_TEXT_ARCHIVE_SUPPORT

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogClass, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEnum, ELogVerbosity::Error);

	const FPropertyTypeName GrainTypeName = MakePropertyTypeName(StaticEnum<ETestInstanceDataObjectGrain::Type>());
	const FPropertyTypeName FruitTypeName = MakePropertyTypeName(StaticEnum<ETestInstanceDataObjectFruit>());
	const FPropertyTypeName DirectionTypeName = MakePropertyTypeName(StaticEnum<ETestInstanceDataObjectDirection>());

	FUnknownEnumNames UnknownEnumNames(Owner);
	TArray<FName> Names{NAME_None};
	bool bHasFlags = false;

	FTestInstanceDataObjectStruct StructData;

	{
		FMemoryReader Ar(BinaryData, /*bIsPersistent*/ true);
		FBinaryArchiveFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStruct::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&StructData, nullptr, nullptr);
	}

	CHECK(StructData.Grain == (ETestInstanceDataObjectGrain::Type)((uint8)ETestInstanceDataObjectGrain::Wheat + 1));
	CHECK(StructData.Fruit == (ETestInstanceDataObjectFruit)((uint8)ETestInstanceDataObjectFruit::Orange + 1));
	CHECK(StructData.Direction == (ETestInstanceDataObjectDirection)MAX_uint16);
	CHECK(StructData.GrainFromEnumClass == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitFromNamespace == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.GrainTypeChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeChange == ETestInstanceDataObjectFruit::Orange);

#if WITH_METADATA
	UnknownEnumNames.Find(GrainTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Rye");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(FruitTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Cherry");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(DirectionTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 2)
	{
		CHECK(Names[0] == "Up");
		CHECK(Names[1] == "Down");
	}
	CHECK(bHasFlags);

	UnknownEnumNames.Find(ParsePropertyTypeName(TEXT("ETestInstanceDataObjectDeletedGrain(/Script/CoreUObject)")), Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Rice");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(ParsePropertyTypeName(TEXT("ETestInstanceDataObjectDeletedFruit(/Script/CoreUObject)")), Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Apple");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(ParsePropertyTypeName(TEXT("ETestInstanceDataObjectDeletedDirection(/Script/CoreUObject)")), Names, bHasFlags);
	CHECKED_IF(Names.Num() == 2)
	{
		CHECK(Names[0] == "Up");
		CHECK(Names[1] == "South");
	}
	CHECK(bHasFlags);
#endif // WITH_METADATA

	UnknownEnumNames.Destroy();

#if WITH_TEXT_ARCHIVE_SUPPORT
	StructData = {};

	{
		FMemoryReader Ar(JsonData, /*bIsPersistent*/ true);
		FJsonArchiveInputFormatter Formatter(Ar);
		FStructuredArchive StructuredAr(Formatter);
		FTestInstanceDataObjectStruct::StaticStruct()->SerializeTaggedProperties(StructuredAr.Open(), (uint8*)&StructData, nullptr, nullptr);
	}

	CHECK(StructData.Grain == (ETestInstanceDataObjectGrain::Type)((uint8)ETestInstanceDataObjectGrain::Wheat + 1));
	CHECK(StructData.Fruit == (ETestInstanceDataObjectFruit)((uint8)ETestInstanceDataObjectFruit::Orange + 1));
	CHECK(StructData.Direction == (ETestInstanceDataObjectDirection)MAX_uint16);
	CHECK(StructData.GrainFromEnumClass == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitFromNamespace == ETestInstanceDataObjectFruit::Orange);
	CHECK(StructData.GrainTypeChange == ETestInstanceDataObjectGrain::Corn);
	CHECK(StructData.FruitTypeChange == ETestInstanceDataObjectFruit::Orange);

#if WITH_METADATA
	UnknownEnumNames.Find(GrainTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Rye");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(FruitTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 1)
	{
		CHECK(Names[0] == "Cherry");
	}
	CHECK_FALSE(bHasFlags);

	UnknownEnumNames.Find(DirectionTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 2)
	{
		CHECK(Names[0] == "Up");
		CHECK(Names[1] == "Down");
	}
	CHECK(bHasFlags);

	// Testing of the unknown property tree is skipped because it is not supported by the text format.
#endif // WITH_METADATA

	UnknownEnumNames.Destroy();
#endif // WITH_TEXT_ARCHIVE_SUPPORT
}

TEST_CASE_NAMED(FUnknownEnumNamesTest, "CoreUObject::Serialization::UnknownEnumNames", "[CoreUObject][EngineFilter]")
{
	UObject* Owner = NewObject<UTestInstanceDataObjectClass>();

	TArray<FName> Names{NAME_None};
	bool bHasFlags = true;

	// Test a non-flags enum...

	FPropertyTypeName FruitTypeName = []
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddPath(StaticEnum<ETestInstanceDataObjectFruit>());
		return Builder.Build();
	}();

	FUnknownEnumNames UnknownEnumNames(Owner);

	UnknownEnumNames.Find(FruitTypeName, Names, bHasFlags);
	CHECK(Names.IsEmpty());
	CHECK_FALSE(bHasFlags);

	const FName NAME_Cherry = "Cherry";
	const FName NAME_Pear = "Pear";

	UnknownEnumNames.Add(nullptr, FruitTypeName, NAME_Pear);
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFruit>(), {}, NAME_Cherry);
	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectFruit>(), {}, NAME_Pear);
	UnknownEnumNames.Add(nullptr, FruitTypeName, NAME_Cherry);

	UnknownEnumNames.Find(FruitTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 2)
	{
		CHECK(Names[0] == NAME_Pear);
		CHECK(Names[1] == NAME_Cherry);
	}
	CHECK_FALSE(bHasFlags);

	// Test a flags enum by name only...

	FPropertyTypeName FlagsTypeName = []
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddPath(StaticEnum<ETestInstanceDataObjectDirection>());
		return Builder.Build();
	}();

	const FName NAME_South = "South";
	const FName NAME_Down = "Down";
	const FName NAME_Up = "Up";

	TStringBuilder<128> FlagsString;
	FlagsString.Join(MakeArrayView({NAME_Up, NAME_Down, NAME_South}), TEXTVIEW(" | "));

	UnknownEnumNames.Add(nullptr, FlagsTypeName, NAME_Down);
	UnknownEnumNames.Add(nullptr, FlagsTypeName, *FlagsString);

	UnknownEnumNames.Find(FlagsTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 3)
	{
		CHECK(Names[0] == NAME_Down);
		CHECK(Names[1] == NAME_Up);
		CHECK(Names[2] == NAME_South);
	}
	CHECK(bHasFlags);

	// Test resetting unknown enum names for an owner...

	UnknownEnumNames.Destroy();

	UnknownEnumNames.Find(FlagsTypeName, Names, bHasFlags);
	CHECK(Names.IsEmpty());
	CHECK_FALSE(bHasFlags);

	// Test a flags enum by enum...

	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectDirection>(), {}, NAME_Up);

	UnknownEnumNames.Find(FlagsTypeName, Names, bHasFlags);
	CHECK(Names.Num() == 1);
	CHECK(bHasFlags);

	UnknownEnumNames.Add(StaticEnum<ETestInstanceDataObjectDirection>(), FlagsTypeName, *FlagsString);

	UnknownEnumNames.Find(FlagsTypeName, Names, bHasFlags);
	CHECKED_IF(Names.Num() == 2)
	{
		CHECK(Names[0] == NAME_Up);
		CHECK(Names[1] == NAME_Down);
	}
	CHECK(bHasFlags);
}

} // UE

#endif // WITH_TESTS && WITH_EDITORONLY_DATA
