// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "PropertyStateTrackingTest.generated.h"

#if WITH_TESTS

UENUM()
enum ETestInstanceDataObjectBird : uint8
{
	TIDOB_None = 0,
	TIDOB_Cardinal,
	TIDOB_Crow,
	TIDOB_Eagle,
	TIDOB_Hawk,
	TIDOB_Owl,
	TIDOB_Raven,
};

UENUM()
namespace ETestInstanceDataObjectGrain
{
	enum Type : uint8
	{
		None = 0,
		Barley,
		Corn,
		Quinoa,
		Rice,
		Wheat,
	};
}

UENUM()
namespace ETestInstanceDataObjectGrainAlternate
{
	enum Type : uint8
	{
		None = 0,
		Corn,
		Rice,
		Rye,
		Wheat,
	};
}

UENUM()
enum class ETestInstanceDataObjectGrainAlternateEnumClass : uint8
{
	None = 0,
	Corn,
	Rice,
	Rye,
	Wheat,
};

static_assert((uint8)ETestInstanceDataObjectGrain::Corn != (uint8)ETestInstanceDataObjectGrainAlternate::Corn);
static_assert((uint8)ETestInstanceDataObjectGrain::Corn != (uint8)ETestInstanceDataObjectGrainAlternateEnumClass::Corn);

UENUM()
enum class ETestInstanceDataObjectFruit : uint8
{
	None = 0,
	Apple,
	Banana,
	Lemon,
	Orange,
};

UENUM()
enum class ETestInstanceDataObjectFruitAlternate : uint8
{
	None = 0,
	Apple,
	Cherry,
	Orange,
	Pear,
};

UENUM()
namespace ETestInstanceDataObjectFruitAlternateNamespace
{
	enum Type : uint8
	{
		None = 0,
		Apple,
		Cherry,
		Orange,
		Pear,
	};
}

static_assert((uint8)ETestInstanceDataObjectFruit::Orange != (uint8)ETestInstanceDataObjectFruitAlternate::Orange);
static_assert((uint8)ETestInstanceDataObjectFruit::Orange != (uint8)ETestInstanceDataObjectFruitAlternateNamespace::Orange);

UENUM(Flags)
enum class ETestInstanceDataObjectDirection : uint16
{
	None = 0,
	North = 1 << 0,
	East = 1 << 1,
	South = 1 << 2,
	West = 1 << 3,
};

ENUM_CLASS_FLAGS(ETestInstanceDataObjectDirection);

UENUM(Flags)
enum class ETestInstanceDataObjectDirectionAlternate : uint16
{
	None = 0,
	Up = 1 << 0,
	Down = 1 << 1,
	North = 1 << 2,
	East = 1 << 3,
	South = 1 << 4,
	West = 1 << 5,
};

ENUM_CLASS_FLAGS(ETestInstanceDataObjectDirectionAlternate);

UENUM(Flags)
enum class ETestInstanceDataObjectFullFlags : uint8
{
	None = 0,
	Flag0 = 1 << 0,
	Flag1 = 1 << 1,
	Flag2 = 1 << 2,
	// Flag3 skipped for testing insertion of an unknown flag in the middle.
	Flag4 = 1 << 4,
	Flag5 = 1 << 5,
	Flag6 = 1 << 6,
	Flag7 = 1 << 7,
};

ENUM_CLASS_FLAGS(ETestInstanceDataObjectFullFlags);

USTRUCT()
struct FTestInstanceDataObjectPoint
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 X = 0;

	UPROPERTY()
	int32 Y = 0;

	UPROPERTY()
	int32 Z = 0;

	UPROPERTY()
	int32 W = 0;
};

USTRUCT()
struct FTestInstanceDataObjectPointAlternate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 U = 0;

	UPROPERTY()
	int32 V = 0;

	UPROPERTY()
	int32 W = 0;
};

USTRUCT()
struct FTestInstanceDataObjectStruct
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 A = -1;

	UPROPERTY()
	int32 B = -1;

	UPROPERTY()
	int32 C = -1;

	UPROPERTY()
	int32 D = -1;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectBird> Bird = TIDOB_None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectGrain::Type> Grain = ETestInstanceDataObjectGrain::None;

	UPROPERTY()
	ETestInstanceDataObjectFruit Fruit = ETestInstanceDataObjectFruit::None;

	UPROPERTY()
	ETestInstanceDataObjectDirection Direction = ETestInstanceDataObjectDirection::None;

	UPROPERTY()
	ETestInstanceDataObjectFullFlags FullFlags = ETestInstanceDataObjectFullFlags::None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectGrain::Type> GrainFromEnumClass = ETestInstanceDataObjectGrain::None;

	UPROPERTY()
	ETestInstanceDataObjectFruit FruitFromNamespace = ETestInstanceDataObjectFruit::None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectGrain::Type> GrainTypeChange = ETestInstanceDataObjectGrain::None;

	UPROPERTY()
	ETestInstanceDataObjectFruit FruitTypeChange = ETestInstanceDataObjectFruit::None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectGrain::Type> GrainTypeAndPropertyChange = ETestInstanceDataObjectGrain::None;

	UPROPERTY()
	ETestInstanceDataObjectFruit FruitTypeAndPropertyChange = ETestInstanceDataObjectFruit::None;

	UPROPERTY()
	FTestInstanceDataObjectPoint Point;
};

USTRUCT()
struct FTestInstanceDataObjectStructAlternate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	float B = -1;

	UPROPERTY()
	int64 C = -1;

	UPROPERTY()
	int32 D = -1;

	UPROPERTY()
	int32 E = -1;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectBird> Bird = TIDOB_None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectGrain(/Script/CoreUObject)"))
	TEnumAsByte<ETestInstanceDataObjectGrainAlternate::Type> Grain = ETestInstanceDataObjectGrainAlternate::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectFruit(/Script/CoreUObject)"))
	ETestInstanceDataObjectFruitAlternate Fruit = ETestInstanceDataObjectFruitAlternate::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectDirection(/Script/CoreUObject)"))
	ETestInstanceDataObjectDirectionAlternate Direction = ETestInstanceDataObjectDirectionAlternate::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectGrain(/Script/CoreUObject)"))
	ETestInstanceDataObjectGrainAlternateEnumClass GrainFromEnumClass = ETestInstanceDataObjectGrainAlternateEnumClass::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectFruit(/Script/CoreUObject)"))
	TEnumAsByte<ETestInstanceDataObjectFruitAlternateNamespace::Type> FruitFromNamespace = ETestInstanceDataObjectFruitAlternateNamespace::None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectGrainAlternate::Type> GrainTypeChange = ETestInstanceDataObjectGrainAlternate::None;

	UPROPERTY()
	ETestInstanceDataObjectFruitAlternate FruitTypeChange = ETestInstanceDataObjectFruitAlternate::None;

	UPROPERTY()
	ETestInstanceDataObjectGrainAlternateEnumClass GrainTypeAndPropertyChange = ETestInstanceDataObjectGrainAlternateEnumClass::None;

	UPROPERTY()
	TEnumAsByte<ETestInstanceDataObjectFruitAlternateNamespace::Type> FruitTypeAndPropertyChange = ETestInstanceDataObjectFruitAlternateNamespace::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectDeletedGrain(/Script/CoreUObject)"))
	TEnumAsByte<ETestInstanceDataObjectGrainAlternate::Type> DeletedGrain = ETestInstanceDataObjectGrainAlternate::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectDeletedFruit(/Script/CoreUObject)"))
	ETestInstanceDataObjectFruitAlternate DeletedFruit = ETestInstanceDataObjectFruitAlternate::None;

	UPROPERTY(meta=(OriginalType="ETestInstanceDataObjectDeletedDirection(/Script/CoreUObject)"))
	ETestInstanceDataObjectDirectionAlternate DeletedDirection = ETestInstanceDataObjectDirectionAlternate::None;

	UPROPERTY(meta=(OriginalType="TestInstanceDataObjectPoint(/Script/CoreUObject)"))
	FTestInstanceDataObjectPointAlternate Point;
};

UCLASS()
class UTestInstanceDataObjectClass : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 A[4]{-1, -1, -1, -1};

	UPROPERTY()
	float B = -1;

	UPROPERTY()
	int64 C = -1;

	UPROPERTY()
	int32 D = -1;

	UPROPERTY()
	int32 E = -1;

	UPROPERTY()
	FTestInstanceDataObjectStruct Struct;
};

#endif // WITH_TESTS
