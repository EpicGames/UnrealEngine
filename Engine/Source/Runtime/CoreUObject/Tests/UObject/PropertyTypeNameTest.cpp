// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyTypeName.h"

#if WITH_TESTS

#include "Async/ParallelFor.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Formatters/BinaryArchiveFormatter.h"
#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/StructuredArchive.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{

namespace PropertyTypeNameTest
{

static FPropertyTypeName CreateInt()
{
	FPropertyTypeNameBuilder Builder;
	Builder.AddName(NAME_IntProperty);
	return Builder.Build();
}

static FPropertyTypeName CreateVectorArray()
{
	FPropertyTypeNameBuilder Builder;
	Builder.AddName(NAME_ArrayProperty);
	{
		Builder.BeginParameters();
		Builder.AddName(NAME_StructProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(NAME_Vector);
			Builder.EndParameters();
		}
		Builder.EndParameters();
	}
	return Builder.Build();
}

static FPropertyTypeName CreateEnumMap()
{
	FPropertyTypeNameBuilder Builder;
	Builder.AddName(NAME_MapProperty);
	{
		Builder.BeginParameters();
		Builder.AddName(NAME_EnumProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(TEXT("Key"));
			Builder.AddName(NAME_ByteProperty);
			Builder.EndParameters();
		}
		Builder.AddName(NAME_EnumProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(TEXT("Value"));
			Builder.AddName(NAME_ByteProperty);
			Builder.EndParameters();
		}
		Builder.EndParameters();
	}
	return Builder.Build();
}

} // PropertyTypeNameTest

TEST_CASE_NAMED(FPropertyTypeNameSmokeTest, "CoreUObject::PropertyTypeName::Smoke", "[Core][UObject][SmokeFilter]")
{
	SECTION("Empty")
	{
		FPropertyTypeName TypeName;
		CHECK(TypeName.IsEmpty());
		CHECK(TypeName.GetName().IsNone());
		CHECK(TypeName.GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).IsEmpty());
	}

	SECTION("Numeric")
	{
		FPropertyTypeName TypeName = PropertyTypeNameTest::CreateInt();
		CHECK(!TypeName.IsEmpty());
		CHECK(TypeName.GetName() == NAME_IntProperty);
		CHECK(TypeName.GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).IsEmpty());
	}

	SECTION("NumericArray")
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_ArrayProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(NAME_IntProperty);
			Builder.EndParameters();
		}

		FPropertyTypeName TypeName = Builder.Build();
		CHECK(!TypeName.IsEmpty());
		CHECK(TypeName.GetName() == NAME_ArrayProperty);
		CHECK(TypeName.GetParameterCount() == 1);
		CHECK(TypeName.GetParameter(0).GetName() == NAME_IntProperty);
		CHECK(TypeName.GetParameter(0).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).GetParameter(0).IsEmpty());
		CHECK(TypeName.GetParameter(1).IsEmpty());
	}

	SECTION("StructArray")
	{
		FPropertyTypeName TypeName = PropertyTypeNameTest::CreateVectorArray();
		CHECK(!TypeName.IsEmpty());
		CHECK(TypeName.GetName() == NAME_ArrayProperty);
		CHECK(TypeName.GetParameterCount() == 1);
		CHECK(TypeName.GetParameter(0).GetName() == NAME_StructProperty);
		CHECK(TypeName.GetParameter(0).GetParameterCount() == 1);
		CHECK(TypeName.GetParameter(0).GetParameter(0).GetName() == NAME_Vector);
		CHECK(TypeName.GetParameter(0).GetParameter(0).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).GetParameter(0).GetParameter(0).IsEmpty());
		CHECK(TypeName.GetParameter(0).GetParameter(1).IsEmpty());
		CHECK(TypeName.GetParameter(1).IsEmpty());
	}

	SECTION("NumericMap")
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_MapProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(NAME_Int32Property);
			Builder.AddName(NAME_UInt32Property);
			Builder.EndParameters();
		}

		FPropertyTypeName TypeName = Builder.Build();
		CHECK(!TypeName.IsEmpty());
		CHECK(TypeName.GetName() == NAME_MapProperty);
		CHECK(TypeName.GetParameterCount() == 2);
		CHECK(TypeName.GetParameter(0).GetName() == NAME_Int32Property);
		CHECK(TypeName.GetParameter(0).GetParameter(0).IsEmpty());
		CHECK(TypeName.GetParameter(1).GetName() == NAME_UInt32Property);
		CHECK(TypeName.GetParameter(1).GetParameter(0).IsEmpty());
		CHECK(TypeName.GetParameter(2).IsEmpty());
	}

	SECTION("EnumMap")
	{
		FPropertyTypeName TypeName = PropertyTypeNameTest::CreateEnumMap();
		CHECK(!TypeName.IsEmpty());
		CHECK(TypeName.GetName() == NAME_MapProperty);
		CHECK(TypeName.GetParameterCount() == 2);
		CHECK(TypeName.GetParameter(0).GetName() == NAME_EnumProperty);
		CHECK(TypeName.GetParameter(0).GetParameterName(0) == TEXT("Key"));
		CHECK(TypeName.GetParameter(0).GetParameterName(1) == NAME_ByteProperty);
		CHECK(TypeName.GetParameter(0).GetParameter(0).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).GetParameter(1).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(0).GetParameter(2).IsEmpty());
		CHECK(TypeName.GetParameter(1).GetName() == NAME_EnumProperty);
		CHECK(TypeName.GetParameter(1).GetParameterName(0) == TEXT("Value"));
		CHECK(TypeName.GetParameter(1).GetParameterName(1) == NAME_ByteProperty);
		CHECK(TypeName.GetParameter(1).GetParameter(0).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(1).GetParameter(1).GetParameterCount() == 0);
		CHECK(TypeName.GetParameter(1).GetParameter(2).IsEmpty());
		CHECK(TypeName.GetParameter(2).IsEmpty());
	}

	SECTION("IsStruct")
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_StructProperty);
		CHECK_FALSE(Builder.Build().IsStruct(NAME_Vector));
		CHECK_FALSE(FPropertyTypeName().IsStruct(NAME_Vector));
		CHECK(PropertyTypeNameTest::CreateVectorArray().GetParameter(0).IsStruct(NAME_Vector));
	}

	SECTION("IsEnum(EnumProperty)")
	{
		const FName EnumName = "Key";
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_EnumProperty);
		CHECK_FALSE(Builder.Build().IsEnum(EnumName));
		CHECK_FALSE(FPropertyTypeName().IsEnum(EnumName));
		CHECK(PropertyTypeNameTest::CreateEnumMap().GetParameter(0).IsEnum(EnumName));
	}

	SECTION("IsEnum(ByteProperty)")
	{
		const FName EnumName = "Key";
		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_ByteProperty);
		CHECK_FALSE(Builder.Build().IsEnum(EnumName));
		CHECK_FALSE(FPropertyTypeName().IsEnum(EnumName));
		Builder.BeginParameters();
		Builder.AddName(EnumName);
		Builder.EndParameters();
		CHECK(Builder.Build().IsEnum(EnumName));
	}

	SECTION("Equals+Less+GetTypeHash")
	{
		const FPropertyTypeName Int = PropertyTypeNameTest::CreateInt();
		const FPropertyTypeName VectorArray = PropertyTypeNameTest::CreateVectorArray();

		FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_MapProperty);
		Builder.BeginParameters();
		Builder.AddName(NAME_IntProperty);
		Builder.AddName(NAME_IntProperty);
		Builder.EndParameters();
		const FPropertyTypeName MapIntToInt = Builder.Build();

		Builder.Reset();
		Builder.AddName(NAME_MapProperty);
		Builder.BeginParameters();
		Builder.AddName(NAME_IntProperty);
		Builder.AddName(NAME_MapProperty);
		{
			Builder.BeginParameters();
			Builder.AddName(NAME_IntProperty);
			Builder.AddName(NAME_IntProperty);
			Builder.EndParameters();
		}
		Builder.EndParameters();
		const FPropertyTypeName MapIntToIntToInt = Builder.Build();

		Builder.Reset();
		Builder.AddName(NAME_MapProperty);
		Builder.BeginParameters();
		Builder.AddName(NAME_IntProperty);
		Builder.AddType(MapIntToInt);
		Builder.EndParameters();
		const FPropertyTypeName MapIntToIntToIntAlternative = Builder.Build();

		CHECK(FPropertyTypeName() == FPropertyTypeName());
		CHECK(GetTypeHash(FPropertyTypeName()) == GetTypeHash(FPropertyTypeName()));

		CHECK(Int == CopyTemp(Int));
		CHECK_FALSE(Int < CopyTemp(Int));
		CHECK(GetTypeHash(Int) == GetTypeHash(CopyTemp(Int)));

		CHECK_FALSE(Int == VectorArray);
		CHECK_FALSE(Int == FPropertyTypeName());
		CHECK_FALSE(VectorArray < VectorArray);
		CHECK_FALSE(Int < VectorArray);
		CHECK(VectorArray < Int);
		CHECK_FALSE(GetTypeHash(Int) == GetTypeHash(VectorArray));
		CHECK_FALSE(GetTypeHash(Int) == GetTypeHash(FPropertyTypeName()));

		CHECK(Int == MapIntToInt.GetParameter(0));
		CHECK_FALSE(Int == MapIntToInt);
		CHECK_FALSE(MapIntToInt == Int);
		CHECK_FALSE(MapIntToInt < MapIntToInt);
		CHECK(Int < MapIntToInt);
		CHECK_FALSE(MapIntToInt < Int);
		CHECK_FALSE(MapIntToInt < VectorArray);
		CHECK(VectorArray < MapIntToInt);
		CHECK(GetTypeHash(Int) == GetTypeHash(MapIntToInt.GetParameter(0)));
		CHECK_FALSE(GetTypeHash(Int) == GetTypeHash(MapIntToInt));
		CHECK_FALSE(GetTypeHash(MapIntToInt) == GetTypeHash(Int));

		CHECK(MapIntToInt == MapIntToIntToInt.GetParameter(1));
		CHECK_FALSE(MapIntToInt == MapIntToIntToInt);
		CHECK_FALSE(MapIntToIntToInt < MapIntToIntToInt);
		CHECK(MapIntToInt < MapIntToIntToInt);
		CHECK_FALSE(MapIntToIntToInt < MapIntToInt);
		CHECK_FALSE(MapIntToIntToInt < VectorArray);
		CHECK(VectorArray < MapIntToIntToInt);
		CHECK(GetTypeHash(MapIntToInt) == GetTypeHash(MapIntToIntToInt.GetParameter(1)));
		CHECK_FALSE(GetTypeHash(MapIntToInt) == GetTypeHash(MapIntToIntToInt));

		CHECK(MapIntToIntToInt == MapIntToIntToIntAlternative);
	}

	SECTION("Archive")
	{
		FPropertyTypeName Empty;
		FPropertyTypeName Int = PropertyTypeNameTest::CreateInt();
		FPropertyTypeName EnumMap = PropertyTypeNameTest::CreateEnumMap();
		FPropertyTypeName VectorArray = PropertyTypeNameTest::CreateVectorArray();

		TArray<uint8> PersistentData;
		{
			FMemoryWriter Ar(PersistentData, /*bIsPersistent*/ true);
			Ar << Empty << Int << EnumMap << VectorArray;
		}

		TArray<uint8> MemoryData;
		{
			FMemoryWriter Ar(MemoryData, /*bIsPersistent*/ false);
			Ar << Empty << Int << EnumMap << VectorArray;
		}

		TArray<uint8> BinaryData;
		{
			FMemoryWriter Ar(BinaryData, /*bIsPersistent*/ true);
			FBinaryArchiveFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			Record.EnterField(TEXT("Empty")) << Empty;
			Record.EnterField(TEXT("Int")) << Int;
			Record.EnterField(TEXT("EnumMap")) << EnumMap;
			Record.EnterField(TEXT("VectorArray")) << VectorArray;
		}

	#if WITH_TEXT_ARCHIVE_SUPPORT
		TArray<uint8> JsonData;
		{
			FMemoryWriter Ar(JsonData, /*bIsPersistent*/ true);
			FJsonArchiveOutputFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			Record.EnterField(TEXT("Empty")) << Empty;
			Record.EnterField(TEXT("Int")) << Int;
			Record.EnterField(TEXT("EnumMap")) << EnumMap;
			Record.EnterField(TEXT("VectorArray")) << VectorArray;
		}
	#endif

		FPropertyTypeName EmptyCopy = Int; // needs to be non-empty
		FPropertyTypeName IntCopy;
		FPropertyTypeName EnumMapCopy;
		FPropertyTypeName VectorArrayCopy;

		{
			FMemoryReader Ar(PersistentData, /*bIsPersistent*/ true);
			Ar << EmptyCopy << IntCopy << EnumMapCopy << VectorArrayCopy;
		}
		CHECK(Empty == EmptyCopy);
		CHECK(Int == IntCopy);
		CHECK(EnumMap == EnumMapCopy);
		CHECK(VectorArray == VectorArrayCopy);

		EmptyCopy = Int; // needs to be non-empty
		IntCopy.Reset();
		EnumMapCopy.Reset();
		VectorArrayCopy.Reset();

		{
			FMemoryReader Ar(MemoryData, /*bIsPersistent*/ false);
			Ar << EmptyCopy << IntCopy << EnumMapCopy << VectorArrayCopy;
		}
		CHECK(Empty == EmptyCopy);
		CHECK(Int == IntCopy);
		CHECK(EnumMap == EnumMapCopy);
		CHECK(VectorArray == VectorArrayCopy);

		EmptyCopy = Int; // needs to be non-empty
		IntCopy.Reset();
		EnumMapCopy.Reset();
		VectorArrayCopy.Reset();

		{
			FMemoryReader Ar(BinaryData, /*bIsPersistent*/ true);
			FBinaryArchiveFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			Record.EnterField(TEXT("Empty")) << EmptyCopy;
			Record.EnterField(TEXT("Int")) << IntCopy;
			Record.EnterField(TEXT("EnumMap")) << EnumMapCopy;
			Record.EnterField(TEXT("VectorArray")) << VectorArrayCopy;
		}
		CHECK(Empty == EmptyCopy);
		CHECK(Int == IntCopy);
		CHECK(EnumMap == EnumMapCopy);
		CHECK(VectorArray == VectorArrayCopy);

		EmptyCopy = Int; // needs to be non-empty
		IntCopy.Reset();
		EnumMapCopy.Reset();
		VectorArrayCopy.Reset();

	#if WITH_TEXT_ARCHIVE_SUPPORT
		{
			FMemoryReader Ar(JsonData, /*bIsPersistent*/ true);
			FJsonArchiveInputFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			Record.EnterField(TEXT("Empty")) << EmptyCopy;
			Record.EnterField(TEXT("Int")) << IntCopy;
			Record.EnterField(TEXT("EnumMap")) << EnumMapCopy;
			Record.EnterField(TEXT("VectorArray")) << VectorArrayCopy;
		}
		CHECK(Empty == EmptyCopy);
		CHECK(Int == IntCopy);
		CHECK(EnumMap == EnumMapCopy);
		CHECK(VectorArray == VectorArrayCopy);
	#endif
	}

	const auto Parse = [](FStringView String) -> FPropertyTypeName
	{
		FPropertyTypeNameBuilder Builder;
		CHECK(Builder.TryParse(String));
		return Builder.Build();
	};
	const auto TryParse = [](FStringView String) -> bool
	{
		FPropertyTypeNameBuilder Builder;
		return Builder.TryParse(String);
	};

	SECTION("String")
	{
		FPropertyTypeName Empty;
		FPropertyTypeName Int = PropertyTypeNameTest::CreateInt();
		FPropertyTypeName EnumMap = PropertyTypeNameTest::CreateEnumMap();
		FPropertyTypeName VectorArray = PropertyTypeNameTest::CreateVectorArray();

		TStringBuilder<128> EmptyString(InPlace, Empty);
		TStringBuilder<128> IntString(InPlace, Int);
		TStringBuilder<128> EnumMapString(InPlace, EnumMap);
		TStringBuilder<128> VectorArrayString(InPlace, VectorArray);

		CHECK(TEXTVIEW("None").Equals(EmptyString));
		CHECK(TEXTVIEW("IntProperty").Equals(IntString));
		CHECK(TEXTVIEW("MapProperty(EnumProperty(Key,ByteProperty),EnumProperty(Value,ByteProperty))").Equals(EnumMapString));
		CHECK(TEXTVIEW("ArrayProperty(StructProperty(Vector))").Equals(VectorArrayString));

		CHECK(Empty == Parse(EmptyString));
		CHECK(Int == Parse(IntString));
		CHECK(EnumMap == Parse(EnumMapString));
		CHECK(VectorArray == Parse(VectorArrayString));
	}

	SECTION("Parse")
	{
		// There are positive parsing tests in the String section above.
		CHECK(Parse(TEXTVIEW(" \t IntProperty \t ")) == PropertyTypeNameTest::CreateInt());
		CHECK(Parse(TEXTVIEW(" \t ArrayProperty ( StructProperty\t(\tVector\t)\t ) \t ")) == PropertyTypeNameTest::CreateVectorArray());

		CHECK_FALSE(TryParse(TEXTVIEW("")));
		CHECK_FALSE(TryParse(TEXTVIEW(",")));
		CHECK_FALSE(TryParse(TEXTVIEW("()")));
		CHECK_FALSE(TryParse(TEXTVIEW(",IntProperty")));
		CHECK_FALSE(TryParse(TEXTVIEW("IntProperty,")));
		CHECK_FALSE(TryParse(TEXTVIEW("IntProperty,IntProperty")));
		CHECK_FALSE(TryParse(TEXTVIEW("IntProperty)")));
		CHECK_FALSE(TryParse(TEXTVIEW("(IntProperty)")));
		CHECK_FALSE(TryParse(TEXTVIEW("(IntProperty")));
		CHECK_FALSE(TryParse(TEXTVIEW("ArrayProperty(IntProperty")));
		CHECK_FALSE(TryParse(TEXTVIEW("ArrayProperty(IntProperty))")));
		CHECK_FALSE(TryParse(TEXTVIEW("ArrayProperty((IntProperty)")));
		CHECK_FALSE(TryParse(TEXTVIEW("ArrayProperty(IntProperty)IntProperty")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty()")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty(,)")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty(IntProperty,)")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty(,IntProperty)")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty(IntProperty,,IntProperty)")));
		CHECK_FALSE(TryParse(TEXTVIEW("MapProperty(IntProperty)(IntProperty)")));
	}
}

TEST_CASE_NAMED(FPropertyTypeNameConcurrencyTest, "CoreUObject::PropertyTypeName::Concurrency", "[Core][UObject][EngineFilter]")
{
	ParallelFor(TEXT("PropertyTypeNameConcurrencyTest"), 64 * 1024, 1, [NAME_TestStruct = FName(ANSITEXTVIEW("TestStruct"))](int32 Index)
	{
		FPropertyTypeNameBuilder Builder;

		if (Index % 8 == 0)
		{
			Builder.AddName(NAME_OptionalProperty);
			Builder.BeginParameters();
		}
		if (Index % 4 == 0)
		{
			Builder.AddName(NAME_ArrayProperty);
			Builder.BeginParameters();
		}

		Builder.AddName(NAME_StructProperty);
		Builder.BeginParameters();
		Builder.AddName(FName(NAME_TestStruct, NAME_EXTERNAL_TO_INTERNAL(Index)));
		Builder.EndParameters();

		if (Index % 4 == 0)
		{
			Builder.EndParameters();
		}
		if (Index % 8 == 0)
		{
			Builder.EndParameters();
		}

		FPropertyTypeName Name = Builder.Build();
	});
}

} // UE

#endif // WITH_TESTS
