// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "SubobjectInstancingTest.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubobjectInstancingTest)

USubobjectInstancingTestOuterObject::USubobjectInstancingTestOuterObject(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{
	// Self reference - no instancing, should always reference this object
	// Since this is a native type, this must be marked 'Instanced' to work
	SelfRef = this;

	// Null reference - no instancing, should always be NULL after construction
	NullObject = nullptr;

	// Instanced subobject that is expected to be a unique ptr value after construction
	InnerObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObject"));

	// Object that exists within the scope of this object, but is otherwise not instanced
	// With one exception for native class types (see below), not expected to be a unique ptr value
	SharedObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("SharedObject"));

	// Object that exists outside the scope of this object; e.g. a simple reference (no instancing)
	ExternalObject = NewObject<USubobjectInstancingTestObject>(GetTransientPackage(), TEXT("ExternalObject"));

	// Internal reference - no instancing, but should reference the unique InnerObject constructed above
	// Since this is a native type that uses property-based instancing, this must be marked 'Instanced' to work
	InternalObject = InnerObject;

	// Instanced subobject that is not going to be serialized on save; note that this is a transient property
	TransientInnerObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObject"));

	// Subobject that may be instanced at edit time rather than at construction time; this is NOT considered a default subobject
	EditTimeInnerObject = nullptr;

	// Same as above, but marked transient; this is not expected to be serialized on save
	TransientEditTimeInnerObject = nullptr;

	// Same as above, but marked duplicatetransient; this is not expected to be serialized as part of duplicating the outer object
	DuplicateTransientEditTimeInnerObject = nullptr;

	// Subobject that does not inherit values from the CDO/template when instanced; 'transient' here does not affect serialization
	// Note: This only applies to C++ types; otherwise it behaves the same as 'InnerObject' above and 'bTransient' is not supported
	LocalOnlyInnerObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("LocalOnlyInnerObject"), /*bTransient =*/ true);

	// Property with instanced semantics inferred from the class type (DefaultToInstanced); property flags exclude CPF_PersistentInstance
	// Runtime behavior is otherwise expected to be the same as any other CPF_InstancedReference property (e.g. InnerObject above)
	InnerObjectFromType = CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromType"));

	// Instanced subobject that is constructed via NewObject() instead of CreateDefaultSubobject(), which is occasionally used instead
	// Property flags will exclude RF_DefaultSubObject, and runtime behavior is similar to LocalOnlyInnerObject above (i.e. no inheritance)
	InnerObjectUsingNew = NewObject<USubobjectInstancingTestObject>(this, TEXT("InnerObjectUsingNew"));

	// Property with transient semantics inferred from the class type (Transient); all non-archetype instances will be RF_Transient
	TransientInnerObjectFromType = CreateDefaultSubobject<USubobjectInstancingTransientTestObject>(TEXT("TransientInnerObjectFromType"));

	// Transient reference to an instanced subobject that is constructed via NewObject() instead of CreateDefaultSubobject() (not serialized)
	TransientInnerObjectUsingNew = NewObject<USubobjectInstancingTestObject>(this, TEXT("TransientInnerObjectUsingNew"));

	// Instanced subobject constructed via NewObject() that should not be serialized during duplication, but is non-transient during save
	DuplicateTransientInnerObjectUsingNew = NewObject<USubobjectInstancingTestObject>(this, TEXT("DuplicateTransientInnerObjectUsingNew"));

	// Instanced subobject constructed via NewObject() that should not be serialized during duplication outside of PIE, but is non-transient during save
	NonPIEDuplicateTransientInnerObjectUsingNew = NewObject<USubobjectInstancingTestObject>(this, TEXT("NonPIEDuplicateTransientInnerObjectUsingNew"));

	// Array container of references to instanced subobjects
	InnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectArrayElem_0")));
	InnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectArrayElem_1")));

	// Transient array container of references to instanced subobjects (CPF_Transient on the property; elements are not RF_Transient)
	TransientInnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectArrayElem_0")));
	TransientInnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectArrayElem_1")));

	// Array container of references to DefaultToInstanced-typed subobjects
	InnerObjectFromTypeArray.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeArrayElem_0")));
	InnerObjectFromTypeArray.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeArrayElem_1")));

	// Set container of references to instanced subobjects
	InnerObjectSet.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectSetElem_0")));
	InnerObjectSet.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectSetElem_1")));

	// Transient set container of references to instanced subobjects (CPF_Transient on the property; elements are not RF_Transient)
	TransientInnerObjectSet.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectSetElem_0")));
	TransientInnerObjectSet.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectSetElem_1")));

	// Set container of references to DefaultToInstanced-typed subobjects
	InnerObjectFromTypeSet.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeSetElem_0")));
	InnerObjectFromTypeSet.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeSetElem_1")));

	// Map container of pairs of references to instanced subobjects
	InnerObjectMap.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectMapKey_0")), CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectMapVal_0")));
	InnerObjectMap.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectMapKey_1")), CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectMapVal_1")));

	// Map container of pairs of references to DefaultToInstanced-typed subobjects
	InnerObjectFromTypeMap.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeMapKey_0")), CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeMapVal_0")));
	InnerObjectFromTypeMap.Add(CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeMapKey_1")), CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeMapVal_1")));

	// Transient map container of pairs of references to instanced subobjects (CPF_Transient on the property; elements are not RF_Transient)
	TransientInnerObjectMap.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectMapKey_0")), CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectMapVal_0")));
	TransientInnerObjectMap.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectMapKey_1")), CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("TransientInnerObjectMapVal_1")));

	// Struct container of references to instanced subobjects
	StructWithInnerObjects.InnerObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectForStruct"));
	StructWithInnerObjects.InnerObjectFromType = CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("InnerObjectFromTypeForStruct"));
	StructWithInnerObjects.InnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectForStructArrayElem_0")));
	StructWithInnerObjects.InnerObjectArray.Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObjectForStructArrayElem_1")));

	// Optional reference to instanced subobject; creating as optional to also test the DoNotCreate override
	// Note that this internally sets the 'bIsRequired' parameter to false, which could also be done here instead
	OptionalInnerObject = CreateOptionalDefaultSubobject<USubobjectInstancingTestObject>(TEXT("OptionalInnerObject"));

	// Optional transient reference to instanced subobject (CPF_Transient on the property; element is not RF_Transient)
	OptionalTransientInnerObject = CreateOptionalDefaultSubobject<USubobjectInstancingTestObject>(TEXT("OptionalTransientInnerObject"));

	// Optional reference to DefaultToInstanced subobject (with instanced property semantics inferred from ptr type)
	OptionalInnerObjectFromType = CreateDefaultSubobject<USubobjectInstancingDefaultToInstancedTestObject>(TEXT("OptionalInnerObjectFromType"));

	// Optional array of references to instanced subobjects
	OptionalInnerObjectArray.Emplace();
	OptionalInnerObjectArray->Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("OptionalInnerObjectArrayElem_0")));
	OptionalInnerObjectArray->Add(CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("OptionalInnerObjectArrayElem_1")));

	// Instanced subobject that contains a directly-nested instanced subobject (i.e. one level deep)
	InnerObjectWithDirectlyNestedObject = CreateDefaultSubobject<USubobjectInstancingTestDirectlyNestedObject>(TEXT("InnerObjectWithDirectlyNestedObject"));
	InnerObjectWithDirectlyNestedObject->OwnerObject = this;

	// Instanced subobject that contains an indirectly-nested instanced subobject (i.e. more than one level deep)
	InnerObjectWithIndirectlyNestedObject = CreateDefaultSubobject<USubobjectInstancingTestIndirectlyNestedObject>(TEXT("InnerObjectWithIndirectlyNestedObject"));
}

void USubobjectInstancingTestOuterObject::PostInitProperties()
{
	Super::PostInitProperties();

	// Subobject that's deferred from native construction, but still instanced as part of UObject initialization flow
	InnerObjectPostInit = NewObject<USubobjectInstancingTestObject>(this, TEXT("InnerObjectPostInit"));

	// Same as above, but assigned to a transient instanced property
	TransientInnerObjectPostInit = NewObject<USubobjectInstancingTestObject>(this, TEXT("TransientInnerObjectPostInit"));
}

USubobjectInstancingTestDirectlyNestedObject::USubobjectInstancingTestDirectlyNestedObject()
{
	// Nested reference to self
	SelfRef = this;

	// Nested instanced subobject that is expected to be a unique ptr value after construction
	InnerObject = CreateDefaultSubobject<USubobjectInstancingTestObject>(TEXT("InnerObject"));

	// Nested subobject that may be instanced at edit time rather than at construction time; this is NOT considered a default subobject
	EditTimeInnerObject = nullptr;
}

USubobjectInstancingTestIndirectlyNestedObject::USubobjectInstancingTestIndirectlyNestedObject()
{
	InnerObject = CreateDefaultSubobject<USubobjectInstancingTestDirectlyNestedObject>(TEXT("InnerObject"));
	InnerObject->OwnerObject = this;
}

// Derived outer type that is expected to instance its 'InnerObject' property at construction time using a subtype of the archetype instance's type
USubobjectInstancingTestDerivedOuterObjectWithTypeOverride::USubobjectInstancingTestDerivedOuterObjectWithTypeOverride(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer.SetDefaultSubobjectClass<USubobjectInstancingTestDerivedObject>("InnerObject"))
{
}

// Derived outer type that is not expected to instance the 'InnerObject' property at construction time - the value should be set to NULL after being default-initialized
USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride::USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer.DoNotCreateDefaultSubobject("OptionalInnerObject"))
{
}

UDynamicSubobjectInstancingTestClass::UDynamicSubobjectInstancingTestClass()
{
	bNeedsDynamicSubobjectInstancing = true;
}

namespace UE
{

namespace SubobjectInstancingTest::Private
{
	static const FName NonNativeSelfReferencePropertyName("NonNativeSelfRef");
	static const FName NonNativeInnerObjectPropertyName("NonNativeInnerObject");
	static const FName NonNativeOuterObjectPropertyName("NonNativeOwnerObject");
	static const FName NonNativeEditTimeInnerObjectPropertyName("NonNativeEditTimeInnerObject");
	static const FName NonNativeDirectlyNestedInnerObjectPropertyName("NonNativeDirectlyNestedInnerObject");
	static const FName NonNativeTransientInnerObjectPropertyName("NonNativeTransientInnerObject");
	static const FName NonNativeTransientEditTimeInnerObjectPropertyName("NonNativeTransientEditTimeInnerObject");
	static const FName NonNullableInnerObjectPropertyName("NonNullableInnerObject");
	static const FName NonNullableTransientInnerObjectPropertyName("NonNullableTransientInnerObject");
	static const FName NonNullableOptionalInnerObjectPropertyName("OptionalNonNullableInnerObject");

	template<typename ClassType>
	static UClass* NewTestClass(UClass* SuperClass)
	{
		if (!SuperClass)
		{
			SuperClass = USubobjectInstancingTestObject::StaticClass();
		}

		UClass* TestClass = NewObject<ClassType>(GetTransientPackage(), NAME_None, RF_Public | RF_Transient);

		// Simulate creation as a non-native (e.g. Blueprint) type
		TestClass->ClassFlags |= (SuperClass->ClassFlags & CLASS_Inherit) | CLASS_CompiledFromBlueprint;
		TestClass->ClassCastFlags |= SuperClass->ClassCastFlags;

		// Hint to editor/runtime that the class may contain at least one reference to an instanced subobject (activates certain code paths)
		TestClass->ClassFlags |= CLASS_HasInstancedReference;

		FField::FLinkedListBuilder PropertyListBuilder(&TestClass->ChildProperties);

		EPropertyFlags NonNativeObjectPropertyFlags = CPF_TObjectPtrWrapper;
		if (!TestClass->ShouldUseDynamicSubobjectInstancing())
		{
			NonNativeObjectPropertyFlags |= CPF_InstancedReference | CPF_PersistentInstance;
		}

		// Note: This flag was meant to be temporary in lieu of dynamic instancing, but allows non-native properties to work with self-referencing
		EPropertyFlags NonNativeSelfReferencePropertyFlags = NonNativeObjectPropertyFlags;
		if (!TestClass->ShouldUseDynamicSubobjectInstancing())
		{
			NonNativeSelfReferencePropertyFlags |= CPF_AllowSelfReference;
		}

		// Non-native object property to store a non-native reference to self - TObjectPtr<UObject>
		FObjectProperty* NonNativeSelfReferenceProperty = new FObjectProperty(TestClass, NonNativeSelfReferencePropertyName);
		NonNativeSelfReferenceProperty->SetPropertyFlags(NonNativeSelfReferencePropertyFlags);
		NonNativeSelfReferenceProperty->SetPropertyClass(UObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeSelfReferenceProperty);

		// Non-native object reference property to store a non-native instantiation by ctor - TObjectPtr<USubobjectInstancingTestObject>
		FObjectProperty* NonNativeInnerObjectProperty = new FObjectProperty(TestClass, NonNativeInnerObjectPropertyName);
		NonNativeInnerObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags);
		NonNativeInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeInnerObjectProperty);

		// Non-native object reference property to store a non-native instantiation of a directly-nested subobject - TObjectPtr<USubobjectInstancingTestDirectlyNestedObject>
		FObjectProperty* NonNativeDirectlyNestedInnerObjectProperty = new FObjectProperty(TestClass, NonNativeDirectlyNestedInnerObjectPropertyName);
		NonNativeDirectlyNestedInnerObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags);
		NonNativeDirectlyNestedInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestDirectlyNestedObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeDirectlyNestedInnerObjectProperty);

		// Non-native object property to store a non-native reference to an outer object - TObjectPtr<UObject>
		FObjectProperty* NonNativeOuterObjectProperty = new FObjectProperty(TestClass, NonNativeOuterObjectPropertyName);
		NonNativeOuterObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags);
		NonNativeOuterObjectProperty->SetPropertyClass(UObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeOuterObjectProperty);

		// Non-native object reference property to simulate deferred edit-time instantiation - TObjectPtr<USubobjectInstancingTestObject>
		FObjectProperty* NonNativeEditTimeInnerObjectProperty = new FObjectProperty(TestClass, NonNativeEditTimeInnerObjectPropertyName);
		NonNativeEditTimeInnerObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags);
		NonNativeEditTimeInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeEditTimeInnerObjectProperty);

		// Non-native object reference property to store a transient reference to a non-native instantiation - TObjectPtr<USubobjectInstancingTestObject>
		FObjectProperty* NonNativeTransientInnerObjectProperty = new FObjectProperty(TestClass, NonNativeTransientInnerObjectPropertyName);
		NonNativeTransientInnerObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags | CPF_Transient);
		NonNativeTransientInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeTransientInnerObjectProperty);

		// Non-native transient object reference property to simulate deferred edit-time instantiation - TObjectPtr<USubobjectInstancingTestObject>
		FObjectProperty* NonNativeTransientEditTimeInnerObjectProperty = new FObjectProperty(TestClass, NonNativeTransientEditTimeInnerObjectPropertyName);
		NonNativeTransientEditTimeInnerObjectProperty->SetPropertyFlags(NonNativeObjectPropertyFlags | CPF_Transient);
		NonNativeTransientEditTimeInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNativeTransientEditTimeInnerObjectProperty);

		// Non-nullable object reference property to store a non-native instantiation by ctor - TNonNullPtr<USubobjectInstancingTestObject>
		EPropertyFlags NonNullableObjectPropertyFlags = (NonNativeObjectPropertyFlags & ~CPF_TObjectPtrWrapper) | CPF_NonNullable;
		FObjectProperty* NonNullableInnerObjectProperty = new FObjectProperty(TestClass, NonNullableInnerObjectPropertyName);
		NonNullableInnerObjectProperty->SetPropertyFlags(NonNullableObjectPropertyFlags);
		NonNullableInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNullableInnerObjectProperty);

		// Non-nullable transient reference property to store a non-native instantiation by ctor - TNonNullPtr<USubobjectInstancingTestObject>
		FObjectProperty* NonNullableTransientInnerObjectProperty = new FObjectProperty(TestClass, NonNullableTransientInnerObjectPropertyName);
		NonNullableTransientInnerObjectProperty->SetPropertyFlags(NonNullableObjectPropertyFlags | CPF_Transient);
		NonNullableTransientInnerObjectProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		PropertyListBuilder.AppendNoTerminate(*NonNullableTransientInnerObjectProperty);

		// Non-nullable optional object reference property to store an optional non-null instantiation by ctor - TOptional<TNonNullPtr<USubobjectinstancingTestObject>>
		// Note: TOptional<TNonNullPtr<T>> is not currently supported as a native property, so this is being used to validate intrusive state for non-nullable references.
		FOptionalProperty* NonNullableOptionalInnerObjectProperty = new FOptionalProperty(TestClass, NonNullableOptionalInnerObjectPropertyName);
		FObjectProperty* NonNullableOptionalInnerObjectValueProperty = new FObjectProperty(NonNullableOptionalInnerObjectProperty, NonNullableOptionalInnerObjectPropertyName);
		NonNullableOptionalInnerObjectProperty->SetPropertyFlags(CPF_ContainsInstancedReference);
		NonNullableOptionalInnerObjectValueProperty->SetPropertyFlags(NonNullableObjectPropertyFlags);
		NonNullableOptionalInnerObjectValueProperty->SetPropertyClass(USubobjectInstancingTestObject::StaticClass());
		NonNullableOptionalInnerObjectProperty->SetValueProperty(NonNullableOptionalInnerObjectValueProperty);
		PropertyListBuilder.AppendNoTerminate(*NonNullableOptionalInnerObjectProperty);

		TestClass->SetSuperStruct(SuperClass);
		TestClass->Bind();
		TestClass->StaticLink(/*bRelinkExistingProperties =*/ true);
		TestClass->AssembleReferenceTokenStream();

		UObject* TestCDO = TestClass->GetDefaultObject(/*bCreateIfNeeded  =*/ true);
		TestClass->PostLoadDefaultObject(TestCDO);

		// Simulate a non-native class constructor assignment of 'self'
		NonNativeSelfReferenceProperty->SetObjectPropertyValue_InContainer(TestCDO, TestCDO);

		// Simulate a non-native class constructor instantiation of a default subobject, which will be "captured" by the CDO and used to initialize all new objects of this type
		// Note: Non-native default subobjects should not include the RF_DefaultSubObject flag - that flag is used to identify subobjects constructed using the CreateDefaultSubobject() API
		static const FName NonNativeInnerObjectName("NonNativeInnerObject");
		UObject* NonNativeInnerDefaultSubobject = NewObject<USubobjectInstancingTestObject>(TestCDO, NonNativeInnerObjectName, RF_ArchetypeObject | TestCDO->GetMaskedFlags(RF_PropagateToSubObjects));
		NonNativeInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, NonNativeInnerDefaultSubobject);

		// Same as above, but typed to USubobjectInstancingTestDirectlyNestedObject to allow access to its nested instanced properties
		static const FName NonNativeDirectlyNestedInnerObjectName("NonNativeDirectlyNestedInnerObject");
		USubobjectInstancingTestDirectlyNestedObject* NonNativeDirectlyNestedInnerDefaultSubobject = NewObject<USubobjectInstancingTestDirectlyNestedObject>(TestCDO, NonNativeDirectlyNestedInnerObjectName, RF_ArchetypeObject | TestCDO->GetMaskedFlags(RF_PropagateToSubObjects));
		NonNativeDirectlyNestedInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, NonNativeDirectlyNestedInnerDefaultSubobject);

		// Same as above, but assigned to a non-native transient reference property
		static const FName NonNativeTransientInnerObjectName("NonNativeTransientInnerObject");
		UObject* NonNativeTransientInnerDefaultSubobject = NewObject<USubobjectInstancingTestObject>(TestCDO, NonNativeTransientInnerObjectName, RF_ArchetypeObject | TestCDO->GetMaskedFlags(RF_PropagateToSubObjects));
		NonNativeTransientInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, NonNativeTransientInnerDefaultSubobject);

		// Simulate a non-native class constructor initialization of the outer object property's default value
		NonNativeOuterObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, nullptr);

		// Simulate a non-native class constructor initialization of the edit-time reference property's default value (non-transient case)
		NonNativeEditTimeInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, nullptr);

		// Simulate a non-native class constructor initialization of the transient edit-time reference property's default value (same as above)
		NonNativeTransientEditTimeInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, nullptr);

		// Simulate a non-native class constructor initialization of a non-nullable subobject reference, also "captured" by the CDO and used to initialize all new objects of this type
		static const FName NonNullableInnerObjectName("NonNullableInnerObject");
		UObject* NonNullableInnerDefaultSubobject = NewObject<USubobjectInstancingTestObject>(TestCDO, NonNullableInnerObjectName, RF_ArchetypeObject | TestCDO->GetMaskedFlags(RF_PropagateToSubObjects));
		NonNullableInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, NonNullableInnerDefaultSubobject);

		// Same as above, but assigned to a non-nullable transient reference property
		static const FName NonNullableTransientInnerObjectName("NonNullableTransientInnerObject");
		UObject* NonNullableTransientInnerDefaultSubobject = NewObject<USubobjectInstancingTestObject>(TestCDO, NonNullableTransientInnerObjectName, RF_ArchetypeObject | TestCDO->GetMaskedFlags(RF_PropagateToSubObjects));
		NonNullableTransientInnerObjectProperty->SetObjectPropertyValue_InContainer(TestCDO, NonNullableTransientInnerDefaultSubobject);

		return TestClass;
	}

	void RunSubobjectInstancingTests_Base(UClass* InstancingTestClass)
	{
		SECTION("Null subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->NullObject == nullptr);
		}

		SECTION("Null subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->NullObject == nullptr);
		}

		SECTION("Default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObject->TestValue == 100);

			CHECK(OuterObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->InnerObject != NewOuterObject->InnerObject);
		}

		SECTION("Default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

			REQUIRE(Template->InnerObject != nullptr);
			CHECK(Template->InnerObject->IsInOuter(Template));

			Template->InnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObject->TestValue == 200);

			CHECK(OuterObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObject != NewOuterObject->InnerObject);
		}

		SECTION("Default subobject from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->InnerObjectFromType != nullptr);
			CHECK(OuterObject->InnerObjectFromType->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObjectFromType->TestValue == 100);

			CHECK(OuterObject->InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectFromType->IsTemplate());
			CHECK(OuterObject->InnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->InnerObjectFromType != NewOuterObject->InnerObjectFromType);
		}

		SECTION("Default subobject from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->InnerObjectFromType->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObjectFromType != nullptr);
			CHECK(OuterObject->InnerObjectFromType->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObjectFromType->TestValue == 200);

			CHECK(OuterObject->InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectFromType->IsTemplate());
			CHECK(OuterObject->InnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObjectFromType != NewOuterObject->InnerObjectFromType);
		}

		SECTION("Shared default subobject initialized from CDO")
		{
			const USubobjectInstancingTestOuterObject* CDO = CastChecked<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// Since this reference is initialized by the native ctor, it will have constructed a new instance for the outer object. And since the default data object
			// is the CDO in this case, InitProperties() won't copy the default value to this property when the outer object is constructed. As a result, this value
			// will remain set to reference a unique instance, so the expected outcome is as if this property were marked as instanced. It's likely that this behavior
			// diverged from the template case when the fast path was added to InitProperties(), as it relies on the property always being initialized from default data.
			REQUIRE(OuterObject->SharedObject != nullptr);
			CHECK(OuterObject->SharedObject->IsInOuter(OuterObject));

			// Expected result is that we are not mutating the archetype object that's owned by the CDO in this case.
			OuterObject->SharedObject->TestValue = 300;
			CHECK(OuterObject->SharedObject->TestValue != CDO->SharedObject->TestValue);

			CHECK_FALSE(OuterObject->SharedObject->HasAllFlags(RF_ArchetypeObject));
			CHECK(OuterObject->SharedObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->SharedObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->SharedObject->IsTemplate());
			CHECK(OuterObject->SharedObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->SharedObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->SharedObject != NewOuterObject->SharedObject);
		}

		SECTION("Shared default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->SharedObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->SharedObject != nullptr);

			// Types using dynamic instancing do not support referencing back to the source archetype.
			if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
			{
				// Expected result should match an instanced reference in this case.
				CHECK(OuterObject->SharedObject->IsInOuter(OuterObject));
			}
			else    // legacy path
			{
				// This reference is first initialized by the native ctor, which constructs a new instance for the outer object. But since the default data object is
				// not the CDO in this case, InitProperties() will copy the template's reference to this property when the outer object is constructed. Because this
				// property is not also marked as instanced, it will not be included in subobject instancing, and should equate to the template's value as a result.
				CHECK(OuterObject->SharedObject->IsInOuter(Template));

				// Expected result is that we are mutating the archetype object that's owned by the template in this case.
				OuterObject->SharedObject->TestValue = 300;
				CHECK(OuterObject->SharedObject->TestValue == Template->SharedObject->TestValue);

				CHECK(OuterObject->SharedObject->HasAllFlags(RF_ArchetypeObject));
				CHECK(OuterObject->SharedObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(OuterObject->SharedObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK(OuterObject->SharedObject->IsTemplate());
				CHECK(OuterObject->SharedObject->IsDefaultSubobject());
				CHECK(OuterObject->SharedObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				CHECK(OuterObject->SharedObject == NewOuterObject->SharedObject);
			}
		}

		SECTION("Reference to external object initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->ExternalObject != nullptr);
			CHECK_FALSE(OuterObject->ExternalObject->IsInOuter(OuterObject));
		}

		SECTION("Reference to external object initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->ExternalObject != nullptr);
			CHECK_FALSE(OuterObject->ExternalObject->IsInOuter(Template));
			CHECK_FALSE(OuterObject->ExternalObject->IsInOuter(OuterObject));

			CHECK(OuterObject->ExternalObject == Template->ExternalObject);
		}

		SECTION("Transient default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->TransientInnerObject != nullptr);
			CHECK(OuterObject->TransientInnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->TransientInnerObject->TestValue == 100);

			CHECK(OuterObject->TransientInnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			// This must otherwise be explicitly set at construction/instancing time, ensuring here that it's not implied
			CHECK_FALSE(OuterObject->TransientInnerObject->HasAnyFlags(RF_Transient));

			CHECK_FALSE(OuterObject->TransientInnerObject->IsTemplate());
			CHECK(OuterObject->TransientInnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->TransientInnerObject != NewOuterObject->TransientInnerObject);
		}

		SECTION("Transient default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->TransientInnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->TransientInnerObject != nullptr);
			CHECK(OuterObject->TransientInnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->TransientInnerObject->TestValue == 200);

			CHECK(OuterObject->TransientInnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			// This must otherwise be explicitly set at construction/instancing time, ensuring here that it's not implied
			CHECK_FALSE(OuterObject->TransientInnerObject->HasAnyFlags(RF_Transient));

			CHECK_FALSE(OuterObject->TransientInnerObject->IsTemplate());
			CHECK(OuterObject->TransientInnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->TransientInnerObject != NewOuterObject->TransientInnerObject);
		}

		SECTION("Local-only default subobject initialized from template with transient data")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

			REQUIRE(Template->LocalOnlyInnerObject != nullptr);
			CHECK(Template->LocalOnlyInnerObject->TestValue == 100);

			Template->LocalOnlyInnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->LocalOnlyInnerObject != nullptr);
			CHECK(OuterObject->LocalOnlyInnerObject->IsInOuter(OuterObject));

			if (InstancingTestClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				// BP/non-native types do not support the transient option to CreateDefaultSubobject(). The reference will just be reconstructed at instancing time using the template's data.
				// This is because we currently skip the code that looks for an existing instance on the outer object before construction (see FObjectInstancingGraph::GetInstancedSubobject()).
				CHECK(OuterObject->LocalOnlyInnerObject->TestValue == 200);
			}
			else
			{
				// For native types, because we added the transient argument in the constructor, instance data is not expected to propagate through the template. It should keep the default value.
				CHECK(OuterObject->LocalOnlyInnerObject->TestValue == 100);
			}

			CHECK_FALSE(OuterObject->LocalOnlyInnerObject->IsTemplate());
			CHECK(OuterObject->LocalOnlyInnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->LocalOnlyInnerObject->IsTemplateForSubobjects());
		}

		SECTION("Native property initialized from CDO using NewObject()")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->InnerObjectUsingNew != nullptr);
			CHECK(OuterObject->InnerObjectUsingNew->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObjectUsingNew->TestValue == 100);

			CHECK_FALSE(OuterObject->InnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->InnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->InnerObjectUsingNew != NewOuterObject->InnerObjectUsingNew);
		}

		SECTION("Native property initialized from template using NewObject()")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->InnerObjectUsingNew->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObjectUsingNew != nullptr);
			CHECK(OuterObject->InnerObjectUsingNew->IsInOuter(OuterObject));

			if (InstancingTestClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				// BP/non-native types behave differently and will not currently search for an existing subobject at instancing time
				// before reconstructing the reference using the template object - see FObjectInstancingGraph::GetInstancedSubobject().
				// @todo - Maybe we can fix this at some point, but this needs to remain backwards-compatible with existing projects.
				CHECK(OuterObject->InnerObjectUsingNew->TestValue == 200);
			}
			else
			{
				// Note: Because we did not construct using CreateDefaultSubobject(), instance data will not propagate through the template.
				// Using NewObject() is effectively the same result as passing TRUE for the transient argument to CreateDefaultSubobject().
				CHECK(OuterObject->InnerObjectUsingNew->TestValue == 100);
			}

			CHECK_FALSE(OuterObject->InnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->InnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObjectUsingNew != NewOuterObject->InnerObjectUsingNew);
		}

		SECTION("Transient native property initialized from CDO using NewObject()")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// Transient instanced properties are excluded from default value initialization - see FObjectInitializer::InitProperties().
			// Note that for BP/non-native types, this behavior differs from the non-transient case, where the reference is reinstanced.
			REQUIRE(OuterObject->TransientInnerObjectUsingNew != nullptr);
			CHECK(OuterObject->TransientInnerObjectUsingNew->IsInOuter(OuterObject));

			CHECK(OuterObject->TransientInnerObjectUsingNew->TestValue == 100);

			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->TransientInnerObjectUsingNew != NewOuterObject->TransientInnerObjectUsingNew);
		}

		SECTION("Transient native property initialized from template using NewObject()")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->TransientInnerObjectUsingNew->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->TransientInnerObjectUsingNew != nullptr);
			CHECK(OuterObject->TransientInnerObjectUsingNew->IsInOuter(OuterObject));

			// Note: Because we did not construct using CreateDefaultSubobject(), instance data does not propagate through the template.
			// As with the non-transient case, using NewObject() is effectively the same result as passing TRUE for the transient argument
			// to CreateDefaultSubobject().
			// 
			// However, unlike the non-transient case, this is true even for BP/non-native types, because transient instanced properties
			// are explicitly excluded from default/template initialization - see FObjectInitializer::InitProperties(). This means we will
			// not run through the same path in FObjectInstancingGraph::GetInstancedSubobject() as we otherwise do in the non-transient case.
			CHECK(OuterObject->TransientInnerObjectUsingNew->TestValue == 100);

			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->TransientInnerObjectUsingNew != NewOuterObject->TransientInnerObjectUsingNew);
		}

		SECTION("DuplicateTransient native property initialized from CDO using NewObject()")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// CPF_DuplicateTransient suppresses default value initialization in FObjectInitializer::InitProperties()
			// in the same way as CPF_Transient. The ctor-created instance is used directly and instance data does not propagate from the CDO.
			REQUIRE(OuterObject->DuplicateTransientInnerObjectUsingNew != nullptr);
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->IsInOuter(OuterObject));

			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->TestValue == 100);

			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew != NewOuterObject->DuplicateTransientInnerObjectUsingNew);
		}

		SECTION("DuplicateTransient native property initialized from template using NewObject()")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->DuplicateTransientInnerObjectUsingNew->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->DuplicateTransientInnerObjectUsingNew != nullptr);
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->IsInOuter(OuterObject));

			// CPF_DuplicateTransient suppresses the InitProperties template copy path in the same way as CPF_Transient, so instance
			// data does not propagate from the template regardless of whether the type is native or non-native.
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->TestValue == 100);

			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->IsTemplate());
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->DuplicateTransientInnerObjectUsingNew->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->DuplicateTransientInnerObjectUsingNew != NewOuterObject->DuplicateTransientInnerObjectUsingNew);
		}

		SECTION("Native property initialized to a transient type instantiation from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->TransientInnerObjectFromType != nullptr);
			CHECK(OuterObject->TransientInnerObjectFromType->IsInOuter(OuterObject));

			CHECK(OuterObject->TransientInnerObjectFromType->TestValue == 100);

			// Transient class types impose RF_Transient on all non-archetype instantiations (see StaticAllocateObject).
			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(RF_Transient));

			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectFromType->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->TransientInnerObjectFromType != NewOuterObject->TransientInnerObjectFromType);
		}

		SECTION("Native property initialized to a transient type instantiation from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->TransientInnerObjectFromType->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->TransientInnerObjectFromType != nullptr);
			CHECK(OuterObject->TransientInnerObjectFromType->IsInOuter(OuterObject));

			// The property itself is not transient, so the template is used as the basis for property initialization.
			CHECK(OuterObject->TransientInnerObjectFromType->TestValue == 200);

			// Transient class types impose RF_Transient on all non-archetype instantiations (see StaticAllocateObject).
			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(RF_Transient));
			CHECK_FALSE(Template->TransientInnerObjectFromType->HasAnyFlags(RF_Transient));

			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectFromType->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->TransientInnerObjectFromType != NewOuterObject->TransientInnerObjectFromType);
		}
	}

	void RunSubobjectInstancingTests_Array(UClass* InstancingTestClass)
	{
		SECTION("Array of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->InnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Array of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->InnerObjectArray)
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->InnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Array of default subobjects from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObjectFromType : OuterObject->InnerObjectFromTypeArray)
			{
				REQUIRE(InnerObjectFromType != nullptr);
				CHECK(InnerObjectFromType->IsInOuter(OuterObject));

				CHECK(InnerObjectFromType->TestValue == 100);

				CHECK(InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromType->IsTemplate());
				CHECK(InnerObjectFromType->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromType->IsTemplateForSubobjects());
			}
		}

		SECTION("Array of default subobjects from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObjectFromType : Template->InnerObjectFromTypeArray)
			{
				InnerObjectFromType->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObjectFromType : OuterObject->InnerObjectFromTypeArray)
			{
				REQUIRE(InnerObjectFromType != nullptr);
				CHECK(InnerObjectFromType->IsInOuter(OuterObject));

				CHECK(InnerObjectFromType->TestValue == 200);

				CHECK(InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromType->IsTemplate());
				CHECK(InnerObjectFromType->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromType->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient array of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// CPF_Transient on the array property excludes it from InitProperties default value copy. Elements are
			// re-constructed from the native constructor on each new instance, not inherited from the CDO.
			CHECK_FALSE(OuterObject->TransientInnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->TransientInnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				// CPF_Transient is on the array property, not the element instances; RF_Transient is not imposed.
				CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient array of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->TransientInnerObjectArray)
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			// CPF_Transient on the array property excludes it from InitProperties template copy. The container
			// elements are re-constructed by the native constructor via CreateDefaultSubobject, which initializes
			// each new element from its named counterpart in the template — so TestValue is still inherited.
			CHECK_FALSE(OuterObject->TransientInnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->TransientInnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}
	}

	void RunSubobjectInstancingTests_Set(UClass* InstancingTestClass)
	{
		SECTION("Set of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->InnerObjectSet)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Set of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->InnerObjectSet)
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->InnerObjectSet)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Set of default subobjects from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObjectFromType : OuterObject->InnerObjectFromTypeSet)
			{
				REQUIRE(InnerObjectFromType != nullptr);
				CHECK(InnerObjectFromType->IsInOuter(OuterObject));

				CHECK(InnerObjectFromType->TestValue == 100);

				CHECK(InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromType->IsTemplate());
				CHECK(InnerObjectFromType->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromType->IsTemplateForSubobjects());
			}
		}

		SECTION("Set of default subobjects from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObjectFromType : Template->InnerObjectFromTypeSet)
			{
				InnerObjectFromType->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObjectFromType : OuterObject->InnerObjectFromTypeSet)
			{
				REQUIRE(InnerObjectFromType != nullptr);
				CHECK(InnerObjectFromType->IsInOuter(OuterObject));

				CHECK(InnerObjectFromType->TestValue == 200);

				CHECK(InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromType->IsTemplate());
				CHECK(InnerObjectFromType->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromType->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient set of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// CPF_Transient on the set property excludes it from InitProperties default value copy. Elements are
			// re-constructed from the native constructor on each new instance, not inherited from the CDO.
			CHECK_FALSE(OuterObject->TransientInnerObjectSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->TransientInnerObjectSet)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				// CPF_Transient is on the set property, not the element instances; RF_Transient is not imposed.
				CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient set of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->TransientInnerObjectSet)
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			// CPF_Transient on the set property excludes it from InitProperties template copy. The container
			// elements are re-constructed by the native constructor via CreateDefaultSubobject, which initializes
			// each new element from its named counterpart in the template — so TestValue is still inherited.
			CHECK_FALSE(OuterObject->TransientInnerObjectSet.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->TransientInnerObjectSet)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}
	}

	void RunSubobjectInstancingTests_Map(UClass* InstancingTestClass)
	{
		SECTION("Map of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : OuterObject->InnerObjectMap)
			{
				REQUIRE(InnerObjectPair.Key != nullptr);
				CHECK(InnerObjectPair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Key->TestValue == 100);

				CHECK(InnerObjectPair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectPair.Key->IsTemplate());
				CHECK(InnerObjectPair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectPair.Value != nullptr);
				CHECK(InnerObjectPair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Value->TestValue == 100);

				CHECK(InnerObjectPair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectPair.Value->IsTemplate());
				CHECK(InnerObjectPair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Value->IsTemplateForSubobjects());
			}
		}

		SECTION("Map of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : Template->InnerObjectMap)
			{
				InnerObjectPair.Key->TestValue = 200;
				InnerObjectPair.Value->TestValue = 300;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : OuterObject->InnerObjectMap)
			{
				REQUIRE(InnerObjectPair.Key != nullptr);
				CHECK(InnerObjectPair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Key->TestValue == 200);

				CHECK(InnerObjectPair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectPair.Key->IsTemplate());
				CHECK(InnerObjectPair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectPair.Value != nullptr);
				CHECK(InnerObjectPair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Value->TestValue == 300);

				CHECK(InnerObjectPair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectPair.Value->IsTemplate());
				CHECK(InnerObjectPair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Value->IsTemplateForSubobjects());
			}
		}

		SECTION("Map of default subobjects from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectFromTypePair : OuterObject->InnerObjectFromTypeMap)
			{
				REQUIRE(InnerObjectFromTypePair.Key != nullptr);
				CHECK(InnerObjectFromTypePair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectFromTypePair.Key->TestValue == 100);

				CHECK(InnerObjectFromTypePair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromTypePair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromTypePair.Key->IsTemplate());
				CHECK(InnerObjectFromTypePair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromTypePair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectFromTypePair.Value != nullptr);
				CHECK(InnerObjectFromTypePair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectFromTypePair.Value->TestValue == 100);

				CHECK(InnerObjectFromTypePair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromTypePair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromTypePair.Value->IsTemplate());
				CHECK(InnerObjectFromTypePair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromTypePair.Value->IsTemplateForSubobjects());
			}
		}

		SECTION("Map of default subobjects from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectFromTypePair : Template->InnerObjectFromTypeMap)
			{
				InnerObjectFromTypePair.Key->TestValue = 200;
				InnerObjectFromTypePair.Value->TestValue = 300;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->InnerObjectFromTypeMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectFromTypePair : OuterObject->InnerObjectFromTypeMap)
			{
				REQUIRE(InnerObjectFromTypePair.Key != nullptr);
				CHECK(InnerObjectFromTypePair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectFromTypePair.Key->TestValue == 200);

				CHECK(InnerObjectFromTypePair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromTypePair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromTypePair.Key->IsTemplate());
				CHECK(InnerObjectFromTypePair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromTypePair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectFromTypePair.Value != nullptr);
				CHECK(InnerObjectFromTypePair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectFromTypePair.Value->TestValue == 300);

				CHECK(InnerObjectFromTypePair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectFromTypePair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObjectFromTypePair.Value->IsTemplate());
				CHECK(InnerObjectFromTypePair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectFromTypePair.Value->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient map of default subobjects initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// CPF_Transient on the map property excludes it from InitProperties default value copy. Elements are
			// re-constructed from the native constructor on each new instance, not inherited from the CDO.
			CHECK_FALSE(OuterObject->TransientInnerObjectMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : OuterObject->TransientInnerObjectMap)
			{
				REQUIRE(InnerObjectPair.Key != nullptr);
				CHECK(InnerObjectPair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Key->TestValue == 100);

				CHECK(InnerObjectPair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				// CPF_Transient is on the map property, not the element instances; RF_Transient is not imposed.
				CHECK_FALSE(InnerObjectPair.Key->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObjectPair.Key->IsTemplate());
				CHECK(InnerObjectPair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectPair.Value != nullptr);
				CHECK(InnerObjectPair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Value->TestValue == 100);

				CHECK(InnerObjectPair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				CHECK_FALSE(InnerObjectPair.Value->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObjectPair.Value->IsTemplate());
				CHECK(InnerObjectPair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Value->IsTemplateForSubobjects());
			}
		}

		SECTION("Transient map of default subobjects initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : Template->TransientInnerObjectMap)
			{
				InnerObjectPair.Key->TestValue = 200;
				InnerObjectPair.Value->TestValue = 300;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			// CPF_Transient on the map property excludes it from InitProperties template copy. The container
			// elements are re-constructed by the native constructor via CreateDefaultSubobject, which initializes
			// each new element from its named counterpart in the template — so TestValue is still inherited.
			CHECK_FALSE(OuterObject->TransientInnerObjectMap.IsEmpty());
			for (TPair<USubobjectInstancingTestObject*, USubobjectInstancingTestObject*> InnerObjectPair : OuterObject->TransientInnerObjectMap)
			{
				REQUIRE(InnerObjectPair.Key != nullptr);
				CHECK(InnerObjectPair.Key->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Key->TestValue == 200);

				CHECK(InnerObjectPair.Key->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Key->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				CHECK_FALSE(InnerObjectPair.Key->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObjectPair.Key->IsTemplate());
				CHECK(InnerObjectPair.Key->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Key->IsTemplateForSubobjects());

				REQUIRE(InnerObjectPair.Value != nullptr);
				CHECK(InnerObjectPair.Value->IsInOuter(OuterObject));

				CHECK(InnerObjectPair.Value->TestValue == 300);

				CHECK(InnerObjectPair.Value->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObjectPair.Value->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
				CHECK_FALSE(InnerObjectPair.Value->HasAllFlags(RF_Transient));

				CHECK_FALSE(InnerObjectPair.Value->IsTemplate());
				CHECK(InnerObjectPair.Value->IsDefaultSubobject());
				CHECK_FALSE(InnerObjectPair.Value->IsTemplateForSubobjects());
			}
		}
	}

	void RunSubobjectInstancingTests_Struct(UClass* InstancingTestClass)
	{
		SECTION("Struct member with default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->StructWithInnerObjects.InnerObject != nullptr);
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->StructWithInnerObjects.InnerObject->TestValue == 100);

			CHECK(OuterObject->StructWithInnerObjects.InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObject->IsTemplate());
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->StructWithInnerObjects.InnerObject != NewOuterObject->StructWithInnerObjects.InnerObject);
		}

		SECTION("Struct member with default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->StructWithInnerObjects.InnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->StructWithInnerObjects.InnerObject != nullptr);
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->IsInOuter(OuterObject));

			CHECK(OuterObject->StructWithInnerObjects.InnerObject->TestValue == 200);

			CHECK(OuterObject->StructWithInnerObjects.InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObject->IsTemplate());
			CHECK(OuterObject->StructWithInnerObjects.InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->StructWithInnerObjects.InnerObject != NewOuterObject->StructWithInnerObjects.InnerObject);
		}

		SECTION("Struct member with default subobject from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->StructWithInnerObjects.InnerObjectFromType != nullptr);
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsInOuter(OuterObject));

			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->TestValue == 100);

			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsTemplate());
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType != NewOuterObject->StructWithInnerObjects.InnerObjectFromType);
		}

		SECTION("Struct member with default subobject from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->StructWithInnerObjects.InnerObjectFromType->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->StructWithInnerObjects.InnerObjectFromType != nullptr);
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsInOuter(OuterObject));

			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->TestValue == 200);

			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsTemplate());
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectFromType->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->StructWithInnerObjects.InnerObjectFromType != NewOuterObject->StructWithInnerObjects.InnerObjectFromType);
		}

		SECTION("Struct member with default subobject array initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->StructWithInnerObjects.InnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Struct member with default subobject array initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->StructWithInnerObjects.InnerObjectArray)
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->StructWithInnerObjects.InnerObjectArray.IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->StructWithInnerObjects.InnerObjectArray)
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}
	}

	void RunSubobjectInstancingTests_Optional(UClass* InstancingTestClass)
	{
		SECTION("Optional default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->OptionalInnerObject.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalInnerObject.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 100);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalInnerObject.GetValue());
		}

		SECTION("Optional default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->OptionalInnerObject.GetValue()->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->OptionalInnerObject.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalInnerObject.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 200);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalInnerObject.GetValue());
		}

		SECTION("Optional default subobject from type initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->OptionalInnerObjectFromType.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalInnerObjectFromType.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 100);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalInnerObjectFromType.GetValue());
		}

		SECTION("Optional default subobject from type initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->OptionalInnerObjectFromType.GetValue()->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->OptionalInnerObjectFromType.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalInnerObjectFromType.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 200);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalInnerObjectFromType.GetValue());
		}

		SECTION("Optional default subobject array initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->OptionalInnerObjectArray.IsSet());

			CHECK_FALSE(OuterObject->OptionalInnerObjectArray->IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->OptionalInnerObjectArray.GetValue())
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 100);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Optional default subobject array initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			for (USubobjectInstancingTestObject* InnerObject : Template->OptionalInnerObjectArray.GetValue())
			{
				InnerObject->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->OptionalInnerObjectArray.IsSet());

			CHECK_FALSE(OuterObject->OptionalInnerObjectArray->IsEmpty());
			for (USubobjectInstancingTestObject* InnerObject : OuterObject->OptionalInnerObjectArray.GetValue())
			{
				REQUIRE(InnerObject != nullptr);
				CHECK(InnerObject->IsInOuter(OuterObject));

				CHECK(InnerObject->TestValue == 200);

				CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(InnerObject->IsTemplate());
				CHECK(InnerObject->IsDefaultSubobject());
				CHECK_FALSE(InnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Unset optional default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OuterObject->OptionalNullableInnerObjectUnset.IsSet());
			CHECK(OuterObject->OptionalNullableInnerObjectUnset.GetPtrOrNull() == nullptr);
		}

		SECTION("Unset optional default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

			CHECK_FALSE(Template->OptionalNullableInnerObjectUnset.IsSet());

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK_FALSE(OuterObject->OptionalNullableInnerObjectUnset.IsSet());

			Template->OptionalNullableInnerObjectUnset = nullptr;

			CHECK(Template->OptionalNullableInnerObjectUnset.IsSet());
			CHECK(Template->OptionalNullableInnerObjectUnset.GetValue().Get() == nullptr);

			OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->OptionalNullableInnerObjectUnset.IsSet());
			CHECK(OuterObject->OptionalNullableInnerObjectUnset.GetValue().Get() == nullptr);
		}

		SECTION("Optional transient default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// CPF_Transient on the optional property excludes it from InitProperties default value copy. The element
			// is re-constructed from the native constructor on each new instance, not inherited from the CDO.
			CHECK(OuterObject->OptionalTransientInnerObject.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalTransientInnerObject.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 100);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
			// CPF_Transient is on the optional property, not the element instance; RF_Transient is not imposed.
			CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalTransientInnerObject.GetValue());
		}

		SECTION("Optional transient default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->OptionalTransientInnerObject.GetValue()->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			// CPF_Transient on the optional property excludes it from InitProperties template copy. The element is
			// re-constructed by the native constructor via CreateDefaultSubobject, which initializes it from its
			// named counterpart in the template — so TestValue is still inherited.
			CHECK(OuterObject->OptionalTransientInnerObject.IsSet());

			USubobjectInstancingTestObject* InnerObject = OuterObject->OptionalTransientInnerObject.GetValue();
			REQUIRE(InnerObject != nullptr);
			CHECK(InnerObject->IsInOuter(OuterObject));

			CHECK(InnerObject->TestValue == 200);

			CHECK(InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(InnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));
			CHECK_FALSE(InnerObject->HasAllFlags(RF_Transient));

			CHECK_FALSE(InnerObject->IsTemplate());
			CHECK(InnerObject->IsDefaultSubobject());
			CHECK_FALSE(InnerObject->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(InnerObject != NewOuterObject->OptionalTransientInnerObject.GetValue());
		}
	}

	void RunSubobjectInstancingTests_Nested(UClass* InstancingTestClass)
	{
		SECTION("Nested default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsInOuter(OuterObject->InnerObjectWithDirectlyNestedObject));

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue == 100);

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->HasAllFlags(OuterObject->InnerObjectWithDirectlyNestedObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->SelfRef == OuterObject->InnerObjectWithDirectlyNestedObject);
			if (InstancingTestClass->IsNative())
			{
				CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->OwnerObject == OuterObject);
			}
			else
			{
				// @todo - Not currently supported for native instanced properties on non-native outer types, but check the expected result to ensure backwards-compatibility.
				//CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->OwnerObject == OuterObject);
				CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->OwnerObject == InstancingTestClass->GetDefaultObject());
			}

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject != NewOuterObject->InnerObjectWithDirectlyNestedObject->InnerObject);
		}

		SECTION("Nested default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsInOuter(OuterObject->InnerObjectWithDirectlyNestedObject));

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue == 200);

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->HasAllFlags(OuterObject->InnerObjectWithDirectlyNestedObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->SelfRef == OuterObject->InnerObjectWithDirectlyNestedObject);

			// @todo - Not currently supported for native instanced properties on non-native outer types, but check the expected result to ensure backwards-compatibility.
			// Note that native and non-native outer types currently result in different outputs.
			//CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->OwnerObject == OuterObject);
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->OwnerObject == (InstancingTestClass->IsNative() ? Template : InstancingTestClass->GetDefaultObject()));

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObjectWithDirectlyNestedObject->InnerObject != NewOuterObject->InnerObjectWithDirectlyNestedObject->InnerObject);
		}

		SECTION("Deeply-nested default subobject initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			REQUIRE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsInOuter(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject));

			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue == 100);

			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->HasAllFlags(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplateForSubobjects());

			if (InstancingTestClass->IsNative())
			{
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject);
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == OuterObject->InnerObjectWithIndirectlyNestedObject);
			}
			else
			{
				// @todo - Not currently supported for nested default subobjects instanced for non-native outer types, but check the expected result to ensure backwards-compatibility.
				//CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject);
				//CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == OuterObject->InnerObjectWithIndirectlyNestedObject);
				USubobjectInstancingTestOuterObject* CDO = CastChecked<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef);
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == CDO->InnerObjectWithIndirectlyNestedObject);
			}

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != NewOuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject);
		}

		SECTION("Deeply-nested default subobject initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != nullptr);
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsInOuter(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject));

			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue == 200);

			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->HasAllFlags(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplate());
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplateForSubobjects());

			if (InstancingTestClass->IsNative())
			{
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject);

				// @todo - Not currently supported for nested default subobjects instanced for native outer types when using the 'Instanced' flag, but check the expected result
				// to ensure backwards-compatibility.
				//CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == OuterObject->InnerObjectWithIndirectlyNestedObject);
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == Template->InnerObjectWithIndirectlyNestedObject);
			}
			else
			{
				// @todo - These cases are not currently supported for nested default subobjects constructed for non-native outer types when using the 'Instanced' flag, but check
				// the expected result to ensure backwards-compatibility.
				//CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject);
				//CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == OuterObject->InnerObjectWithIndirectlyNestedObject);
				USubobjectInstancingTestOuterObject* CDO = CastChecked<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef == Template->InnerObjectWithIndirectlyNestedObject->InnerObject->SelfRef);
				CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->OwnerObject == CDO->InnerObjectWithIndirectlyNestedObject);
			}

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != NewOuterObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject);
		}

		SECTION("Nested instanced subobject preserves transient flag")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native outer types.
			}
			else
			{
				FObjectProperty* NonNativeDirectlyNestedInnerObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(NonNativeDirectlyNestedInnerObjectPropertyName));

				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());
				REQUIRE(CDO != nullptr);

				// Default-constructed archetype
				USubobjectInstancingTestDirectlyNestedObject* NonNativeDirectlyNestedInnerObjectTemplate = CastChecked<USubobjectInstancingTestDirectlyNestedObject>(NonNativeDirectlyNestedInnerObjectProperty->GetObjectPropertyValue_InContainer(CDO));
				
				REQUIRE(NonNativeDirectlyNestedInnerObjectTemplate != nullptr);
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->IsTemplate());
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->IsDefaultSubobject());
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->IsTemplateForSubobjects());

				// Default value of the nested archetype before it's edited 
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject == nullptr);

				// Simulate instantiating a nested transient subobject archetype at edit time
				NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(NonNativeDirectlyNestedInnerObjectTemplate, TEXT("EditTimeInnerObject"), NonNativeDirectlyNestedInnerObjectTemplate->GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transient);

				REQUIRE(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject != nullptr);
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject->IsTemplate());
				CHECK_FALSE(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject->IsDefaultSubobject());
				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject->IsTemplateForSubobjects());

				CHECK(NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject->HasAllFlags(RF_Transient));

				// Construct a new instance and verify RF_Transient is preserved on the nested instanced copy.
				USubobjectInstancingTestOuterObject* OuterInstance = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				REQUIRE(OuterInstance != nullptr);

				USubobjectInstancingTestDirectlyNestedObject* NonNativeDirectlyNestedInnerObject = CastChecked<USubobjectInstancingTestDirectlyNestedObject>(NonNativeDirectlyNestedInnerObjectProperty->GetObjectPropertyValue_InContainer(OuterInstance));

				REQUIRE(NonNativeDirectlyNestedInnerObject != nullptr);
				CHECK(NonNativeDirectlyNestedInnerObject != NonNativeDirectlyNestedInnerObjectTemplate);
				CHECK(NonNativeDirectlyNestedInnerObject->IsInOuter(OuterInstance));

				// Instanced value of the nested subobject
				REQUIRE(NonNativeDirectlyNestedInnerObject->EditTimeInnerObject != nullptr);
				CHECK(NonNativeDirectlyNestedInnerObject->EditTimeInnerObject != NonNativeDirectlyNestedInnerObjectTemplate->EditTimeInnerObject);
				CHECK(NonNativeDirectlyNestedInnerObject->EditTimeInnerObject->IsInOuter(NonNativeDirectlyNestedInnerObject));

				CHECK(NonNativeDirectlyNestedInnerObject->EditTimeInnerObject->HasAllFlags(RF_Transient));
			}
		}

		SECTION("Default subobject with nested dynamically-instanced subobject")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native outer types. Native types do not currently support dynamic subobject instancing.
			}
			else
			{
				// Create a dynamically-instanced test class to use for the outer default subobject that we'll simulate being instanced at edit time below. It will contain an inner
				// object property that needs to be instanced dynamically on new instances of the owning object (which itself may not use dynamic instancing). This is a valid use case.
				UClass* TestClass = FSubobjectInstancingTestUtils::CreateDynamicallyInstancedTestClass(USubobjectInstancingTestDirectlyNestedObject::StaticClass());

				// Create a template for the outer object that may or may not contain dynamic reference properties.
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

				// Simulate instantiating a subobject archetype at edit time with a type that may contain a dynamic reference property. This should use dynamic instancing to instance it when its outer object is constructed.
				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(NonNativeEditTimeInnerObjectPropertyName));
				USubobjectInstancingTestDirectlyNestedObject* NonNativeEditTimeInnerObjectDefaultValue = NewObject<USubobjectInstancingTestDirectlyNestedObject>(Template, TestClass, TEXT("NonNativeEditTimeInnerObject"), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				ObjectProperty->SetObjectPropertyValue_InContainer(Template, NonNativeEditTimeInnerObjectDefaultValue);

				// Modify non-native properties that are expected to be mirrored to the nested subobject instance.
				NonNativeEditTimeInnerObjectDefaultValue->TestValue = 200;
				CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeOuterObjectPropertyName))->SetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObjectDefaultValue, Template);
				CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeSelfReferencePropertyName))->SetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObjectDefaultValue, NonNativeEditTimeInnerObjectDefaultValue);
				
				// Modify the default child object's state to simulate an override value on its nested subobject. This value should be copied to the child object of the nested subobject instance.
				USubobjectInstancingTestObject* NestedNonNativeInnerObjectDefaultValue = CastChecked<USubobjectInstancingTestObject>(CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeInnerObjectPropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObjectDefaultValue));
				NestedNonNativeInnerObjectDefaultValue->TestValue = 200;

				// Also modify the default child object's state to simulate an override value on its nested non-native transient-referenced subobject. This value should also be copied after instancing.
				USubobjectInstancingTestObject* NestedNonNativeTransientInnerObjectDefaultValue = CastChecked<USubobjectInstancingTestObject>(CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeTransientInnerObjectPropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObjectDefaultValue));
				NestedNonNativeTransientInnerObjectDefaultValue->TestValue = 200;

				// Construct a new instance of the outer object type using the template for initialization.
				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				USubobjectInstancingTestDirectlyNestedObject* NonNativeEditTimeInnerObject = CastChecked<USubobjectInstancingTestDirectlyNestedObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

				REQUIRE(NonNativeEditTimeInnerObject != nullptr);
				CHECK(NonNativeEditTimeInnerObject->TestValue == 200);
				CHECK(NonNativeEditTimeInnerObject->IsInOuter(OuterObject));

				// Deeply-nested reference to a dynamically-instanced subobject, expected to be duplicated as part of constructing the outermost root object initialized using the template.
				UObject* NonNativeEditTimeInnerObject_NonNativeInnerObject = CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeInnerObjectPropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObject);
				{
					REQUIRE(NonNativeEditTimeInnerObject_NonNativeInnerObject != nullptr);
					CHECK(NonNativeEditTimeInnerObject_NonNativeInnerObject->IsInOuter(NonNativeEditTimeInnerObject));

					USubobjectInstancingTestObject* NestedNonNativeInnerObjectValue = Cast<USubobjectInstancingTestObject>(NonNativeEditTimeInnerObject_NonNativeInnerObject);
					{
						REQUIRE(NestedNonNativeInnerObjectValue != nullptr);

						CHECK(NestedNonNativeInnerObjectValue != NestedNonNativeInnerObjectDefaultValue);
						CHECK(NestedNonNativeInnerObjectValue->IsInOuter(NonNativeEditTimeInnerObject));
						CHECK(NestedNonNativeInnerObjectValue->TestValue == NestedNonNativeInnerObjectDefaultValue->TestValue);
					}
				}

				// Deeply-nested transient reference to a dynamically-instanced subobject, expected to be duplicated as part of constructing the outermost root object initialized using the template.
				// Note: Unlike the case above, this would not be expected to be serialized as an export on save. It should, however, be initialized the same as the non-transient-referenced subobject.
				UObject* NonNativeEditTimeInnerObject_NonNativeTransientInnerObject = CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeTransientInnerObjectPropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObject);
				{
					REQUIRE(NonNativeEditTimeInnerObject_NonNativeTransientInnerObject != nullptr);
					CHECK(NonNativeEditTimeInnerObject_NonNativeTransientInnerObject->IsInOuter(NonNativeEditTimeInnerObject));
					
					USubobjectInstancingTestObject* NestedNonNativeTransientInnerObjectValue = Cast<USubobjectInstancingTestObject>(NonNativeEditTimeInnerObject_NonNativeTransientInnerObject);
					{
						REQUIRE(NestedNonNativeTransientInnerObjectValue != nullptr);
						
						CHECK(NestedNonNativeTransientInnerObjectValue != NestedNonNativeTransientInnerObjectDefaultValue);
						CHECK(NestedNonNativeTransientInnerObjectValue->IsInOuter(NonNativeEditTimeInnerObject));
						CHECK(NestedNonNativeTransientInnerObjectValue->TestValue == NestedNonNativeTransientInnerObjectDefaultValue->TestValue);
					}
				}

				// Nested reference to the outermost instance; ensures that we can self-reference nodes that exist above the immediate parent node in the object graph for nested subobject types that use dynamic instancing.
				UObject* NonNativeEditTimeInnerObject_NonNativeOuterObject = CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeOuterObjectPropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObject);
				{
					CHECK(NonNativeEditTimeInnerObject_NonNativeOuterObject == OuterObject);
				}

				// Nested reference to self; ensures that we can self-reference the immediate parent node in the object graph for nested subobject types that use dynamic instancing.
				UObject* NonNativeEditTimeInnerObject_NonNativeSelfReference = CastFieldChecked<FObjectProperty>(TestClass->FindPropertyByName(Private::NonNativeSelfReferencePropertyName))->GetObjectPropertyValue_InContainer(NonNativeEditTimeInnerObject);
				{
					CHECK(NonNativeEditTimeInnerObject_NonNativeSelfReference == NonNativeEditTimeInnerObject);
				}

				CHECK_FALSE(NonNativeEditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(NonNativeEditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplate());
				CHECK_FALSE(NonNativeEditTimeInnerObject->IsDefaultSubobject());
				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				CHECK(NonNativeEditTimeInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
			}
		}
	}

	void RunSubobjectInstancingTests_SelfReferencing(UClass* InstancingTestClass)
	{
		SECTION("Reference to native self property initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->SelfRef == OuterObject);
		}

		SECTION("Reference to native self property initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			// @todo - Self-referencing is not currently supported for native instanced properties when instanced from a template, but check the expected result to ensure backwards-compatibility.
			//CHECK(OuterObject->SelfRef == OuterObject);
			CHECK(OuterObject->SelfRef == Template);
		}

		SECTION("Reference to non-native self property initialized from CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(NonNativeSelfReferencePropertyName));
				UObject* NonNativeSelfRef = ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

				// This is initialized on the CDO, but in the non-native case, the property is linked into the PCL chain. So unlike the native case, InitProperties() will copy it to the new
				// instance, and FObjectInstancingGraph::GetInstancedSubobject() is expected to resolve the source value during subobject instancing. This currently has limited support.
				// 
				// @todo (UE-219797) - When NOT using dynamic instancing, GetInstancedSubobject() currently requires the 'CPF_AllowSelfReference' flag to work. This flag is considered to
				// be deprecated in favor of dynamic instancing and will eventually be removed. It can only be set in code on non-native properties at this time (there is no UHT support).
				if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing()
					|| ObjectProperty->HasAnyPropertyFlags(CPF_AllowSelfReference))
				{
					CHECK(NonNativeSelfRef == OuterObject);
				}
				else
				{
					// Unsupported cases (currently types w/o dynamic reference support)
					CHECK(NonNativeSelfRef == InstancingTestClass->GetDefaultObject());
				}
			}
		}

		SECTION("Reference to non-native self property initialized from template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeSelfReferencePropertyName));
				UObject* NonNativeSelfRef = ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject);

				// @todo (UE-219797) - See note in previous test above for the CDO case. The 'CPF_AllowSelfReference' flag is considered to be deprecated and will eventually be removed.
				if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing()
					|| ObjectProperty->HasAnyPropertyFlags(CPF_AllowSelfReference))
				{
					CHECK(NonNativeSelfRef == OuterObject);
				}
				else
				{
					// Unsupported cases (currently types w/o dynamic reference support)
					CHECK(NonNativeSelfRef == Template);
				}
			}
		}

		SECTION("Reference to inner object initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK(OuterObject->InternalObject == OuterObject->InnerObject);
		}

		SECTION("Reference to inner object initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			CHECK(OuterObject->InternalObject == OuterObject->InnerObject);
		}
	}

	void RunSubobjectInstancingTests_Deferred(UClass* InstancingTestClass)
	{
		SECTION("Simulate native property instanced at edit time on non-native CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				// Default value before it's edited (as will be initialized by the native super class ctor).
				CHECK(CDO->EditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native CDO.
				CDO->EditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(CDO, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, EditTimeInnerObject), CDO->GetMaskedFlags(RF_PropagateToSubObjects));
				CDO->EditTimeInnerObject->TestValue = 200;

				CHECK(CDO->EditTimeInnerObject->IsTemplate());
				// Since we instanced the subobject on the CDO, the value is "captured" in the same manner as if it were instanced at construction time, which makes it a default subobject by definition.
				// Given the current (legacy) implementation, this will be an expected result.
				CHECK(CDO->EditTimeInnerObject->IsDefaultSubobject());
				CHECK(CDO->EditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

				// Since the subobject archetype was not instanced at construction time, it's expected to remain set to NULL. When the CDO is used as the default object for initialization, as an optimization,
				// the native object initializer will not include properties inherited from the native super class hierarchy, because it expects they have already been initialized by the native ctor. For this
				// reason, users cannot override a native instanced reference member's default value. This test is validating that any such override will not actually be used by subobject template instancing.
				CHECK(OuterObject->EditTimeInnerObject == nullptr);

				// Clean up the subobject on the CDO so it doesn't persist through remaining tests.
				CDO->EditTimeInnerObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				CDO->EditTimeInnerObject = nullptr;
			}
		}

		SECTION("Simulate native property instanced at edit time on non-native template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
				CHECK(Template->EditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native template.
				Template->EditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(Template, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, EditTimeInnerObject), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				Template->EditTimeInnerObject->TestValue = 200;

				CHECK(Template->EditTimeInnerObject->IsTemplate());
				CHECK_FALSE(Template->EditTimeInnerObject->IsDefaultSubobject());
				// Since this is not considered a default subobject nor was it captured by a CDO, it's expected that this call should fail w/ the default flags argument on the internal IsTemplate() call.
				CHECK_FALSE(Template->EditTimeInnerObject->IsTemplateForSubobjects());
				// However, we can configure the API to tell us whether the subobject is also an archetype that will be used for instancing at construction time if the outer template is used for initialization.
				CHECK(Template->EditTimeInnerObject->IsTemplateForSubobjects(RF_ArchetypeObject));

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

				CHECK(OuterObject->EditTimeInnerObject != nullptr);
				CHECK(OuterObject->EditTimeInnerObject->IsInOuter(OuterObject));

				CHECK(OuterObject->EditTimeInnerObject->TestValue == 200);

				CHECK_FALSE(OuterObject->EditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(OuterObject->EditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(OuterObject->EditTimeInnerObject->IsTemplate());
				CHECK_FALSE(OuterObject->EditTimeInnerObject->IsDefaultSubobject());
				CHECK_FALSE(OuterObject->EditTimeInnerObject->IsTemplateForSubobjects());
			}
		}

		SECTION("Simulate native transient property instanced at edit time on non-native CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				// Default value before it's edited (as will be initialized by the native super class ctor).
				CHECK(CDO->TransientEditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native CDO.
				CDO->TransientEditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(CDO, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientEditTimeInnerObject), CDO->GetMaskedFlags(RF_PropagateToSubObjects));
				CDO->TransientEditTimeInnerObject->TestValue = 200;

				CHECK(CDO->TransientEditTimeInnerObject->IsTemplate());
				CHECK(CDO->TransientEditTimeInnerObject->IsDefaultSubobject());
				CHECK(CDO->TransientEditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

				// Since the subobject archetype was not instanced at construction time, it's expected to remain set to NULL. When the CDO is used as the default object for initialization, as an optimization,
				// the native object initializer will not include properties inherited from the native super class hierarchy, because it expects they have already been initialized by the native ctor. For this
				// reason, users cannot override a native instanced reference member's default value. This test is validating that any such override will not actually be used by subobject template instancing.
				CHECK(OuterObject->TransientEditTimeInnerObject == nullptr);

				// Clean up the subobject on the CDO so it doesn't persist through remaining tests.
				CDO->TransientEditTimeInnerObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				CDO->TransientEditTimeInnerObject = nullptr;
			}
		}

		SECTION("Simulate native transient property instanced at edit time on non-native template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
				CHECK(Template->TransientEditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native template.
				Template->TransientEditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(Template, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, TransientEditTimeInnerObject), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				Template->TransientEditTimeInnerObject->TestValue = 200;

				CHECK(Template->TransientEditTimeInnerObject->IsTemplate());
				CHECK_FALSE(Template->TransientEditTimeInnerObject->IsDefaultSubobject());
				// Since this is not considered a default subobject nor was it captured by a CDO, it's expected that this call should fail w/ the default flags argument on the internal IsTemplate() call.
				CHECK_FALSE(Template->TransientEditTimeInnerObject->IsTemplateForSubobjects());
				// However, we can configure the API to tell us whether the subobject is also an archetype that will be used for instancing at construction time if the outer template is used for initialization.
				CHECK(Template->TransientEditTimeInnerObject->IsTemplateForSubobjects(RF_ArchetypeObject));

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

				// As with the TransientInnerObjectUsingNew case, transient instanced properties are explicitly excluded from default/template initialization - see FObjectInitializer::InitProperties().
				// This means we will not run through the same path in FObjectInstancingGraph::GetInstancedSubobject() as we otherwise do in the non-transient case where the property is initialized from template.
				CHECK(OuterObject->TransientEditTimeInnerObject == nullptr);
			}
		}

		SECTION("Simulate native DuplicateTransient property instanced at edit time on non-native CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				// Default value before it's edited (as will be initialized by the native super class ctor).
				CHECK(CDO->DuplicateTransientEditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native CDO.
				CDO->DuplicateTransientEditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(CDO, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, DuplicateTransientEditTimeInnerObject), CDO->GetMaskedFlags(RF_PropagateToSubObjects));
				CDO->DuplicateTransientEditTimeInnerObject->TestValue = 200;

				CHECK(CDO->DuplicateTransientEditTimeInnerObject->IsTemplate());
				CHECK(CDO->DuplicateTransientEditTimeInnerObject->IsDefaultSubobject());
				CHECK(CDO->DuplicateTransientEditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

				// Since the subobject archetype was not instanced at construction time, it's expected to remain set to NULL. When the CDO is used as the default object for initialization, as an optimization,
				// the native object initializer will not include properties inherited from the native super class hierarchy, because it expects they have already been initialized by the native ctor.
				// Additionally, CPF_DuplicateTransient suppresses the InitProperties copy path in the same way as CPF_Transient.
				CHECK(OuterObject->DuplicateTransientEditTimeInnerObject == nullptr);

				// Clean up the subobject on the CDO so it doesn't persist through remaining tests.
				CDO->DuplicateTransientEditTimeInnerObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				CDO->DuplicateTransientEditTimeInnerObject = nullptr;
			}
		}

		SECTION("Simulate native DuplicateTransient property instanced at edit time on non-native template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
				CHECK(Template->DuplicateTransientEditTimeInnerObject == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native template.
				Template->DuplicateTransientEditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(Template, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, DuplicateTransientEditTimeInnerObject), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				Template->DuplicateTransientEditTimeInnerObject->TestValue = 200;

				CHECK(Template->DuplicateTransientEditTimeInnerObject->IsTemplate());
				CHECK_FALSE(Template->DuplicateTransientEditTimeInnerObject->IsDefaultSubobject());
				CHECK_FALSE(Template->DuplicateTransientEditTimeInnerObject->IsTemplateForSubobjects());
				CHECK(Template->DuplicateTransientEditTimeInnerObject->IsTemplateForSubobjects(RF_ArchetypeObject));

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

				// CPF_DuplicateTransient suppresses the InitProperties template copy path in the same way as CPF_Transient, so the
				// template value is not propagated.
				CHECK(OuterObject->DuplicateTransientEditTimeInnerObject == nullptr);
			}
		}

		SECTION("Simulate non-native property instanced at edit time on non-native CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native types.
			}
			else
			{
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeEditTimeInnerObjectPropertyName));
				USubobjectInstancingTestObject* NonNativeEditTimeInnerObjectDefaultValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(CDO));

				// Default value before it's edited (as will have been initialized by non-native construction).
				CHECK(NonNativeEditTimeInnerObjectDefaultValue == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native CDO.
				NonNativeEditTimeInnerObjectDefaultValue = NewObject<USubobjectInstancingTestObject>(CDO, TEXT("NonNativeEditTimeInnerObject"), CDO->GetMaskedFlags(RF_PropagateToSubObjects));
				NonNativeEditTimeInnerObjectDefaultValue->TestValue = 200;
				ObjectProperty->SetObjectPropertyValue_InContainer(CDO, NonNativeEditTimeInnerObjectDefaultValue);

				CHECK(NonNativeEditTimeInnerObjectDefaultValue->IsTemplate());
				// Since we instanced the subobject on the CDO, the value is "captured" in the same manner as if it were instanced at construction time, which makes it a default subobject by definition.
				// Given the current (legacy) implementation, this will be an expected result.
				CHECK(NonNativeEditTimeInnerObjectDefaultValue->IsDefaultSubobject());
				CHECK(NonNativeEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				USubobjectInstancingTestObject* NonNativeEditTimeInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

				REQUIRE(NonNativeEditTimeInnerObject != nullptr);
				CHECK(NonNativeEditTimeInnerObject->IsInOuter(OuterObject));

				CHECK(NonNativeEditTimeInnerObject->TestValue == 200);

				CHECK_FALSE(NonNativeEditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(NonNativeEditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplate());
				CHECK(NonNativeEditTimeInnerObject->IsDefaultSubobject());
				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				CHECK(NonNativeEditTimeInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));

				// Clean up the subobject on the CDO so it doesn't persist through remaining tests.
				ObjectProperty->SetObjectPropertyValue_InContainer(CDO, nullptr);
				NonNativeEditTimeInnerObjectDefaultValue->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		}

		SECTION("Simulate non-native property instanced at edit time on non-native template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native iterations - the native class won't include the non-native property.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeEditTimeInnerObjectPropertyName));
				USubobjectInstancingTestObject* NonNativeEditTimeInnerObjectDefaultValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(Template));

				// Default value before it's edited (as will have been initialized by non-native construction).
				CHECK(NonNativeEditTimeInnerObjectDefaultValue == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native template.
				NonNativeEditTimeInnerObjectDefaultValue = NewObject<USubobjectInstancingTestObject>(Template, TEXT("NonNativeEditTimeInnerObject"), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				NonNativeEditTimeInnerObjectDefaultValue->TestValue = 200;
				ObjectProperty->SetObjectPropertyValue_InContainer(Template, NonNativeEditTimeInnerObjectDefaultValue);

				CHECK(NonNativeEditTimeInnerObjectDefaultValue->IsTemplate());
				// Since we instanced the subobject on the template and not the CDO, this is not considered to be a default subobject.
				CHECK_FALSE(NonNativeEditTimeInnerObjectDefaultValue->IsDefaultSubobject());
				// It is also not considered to be a template for a CDO or default subobject, but rather a standalone archetype subobject.
				CHECK_FALSE(NonNativeEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects());
				CHECK(NonNativeEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects(RF_ArchetypeObject));

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				USubobjectInstancingTestObject* NonNativeEditTimeInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

				REQUIRE(NonNativeEditTimeInnerObject != nullptr);
				CHECK(NonNativeEditTimeInnerObject->IsInOuter(OuterObject));

				CHECK(NonNativeEditTimeInnerObject->TestValue == 200);

				CHECK_FALSE(NonNativeEditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
				CHECK(NonNativeEditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplate());
				CHECK_FALSE(NonNativeEditTimeInnerObject->IsDefaultSubobject());
				CHECK_FALSE(NonNativeEditTimeInnerObject->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				CHECK(NonNativeEditTimeInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
			}
		}

		SECTION("Simulate non-native transient property instanced at edit time on non-native CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native iterations - the native class won't include the non-native property.
			}
			else
			{
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeTransientEditTimeInnerObjectPropertyName));
				USubobjectInstancingTestObject* NonNativeTransientEditTimeInnerObjectDefaultValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(CDO));

				// Default value before it's edited (it will have been zero-initialized).
				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native CDO.
				NonNativeTransientEditTimeInnerObjectDefaultValue = NewObject<USubobjectInstancingTestObject>(CDO, TEXT("NonNativeTransientEditTimeInnerObject"), CDO->GetMaskedFlags(RF_PropagateToSubObjects));
				NonNativeTransientEditTimeInnerObjectDefaultValue->TestValue = 200;
				ObjectProperty->SetObjectPropertyValue_InContainer(CDO, NonNativeTransientEditTimeInnerObjectDefaultValue);

				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue->IsTemplate());
				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue->IsDefaultSubobject());
				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				USubobjectInstancingTestObject* NonNativeTransientEditTimeInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));
				{
					// Note: Only non-native types with dynamic referencing enabled can reflect a transient property's value through the CDO to new instances of the class, in order for
					// them to be dynamically instanced as part of UObject construction. This copies the deferred override from the CDO which effectively acts as the non-native ctor.
					// The reason it works in this case is because the property does not require 'Instanced', which decouples it from having to first be initialized by a native super.
					if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
					{
						REQUIRE(NonNativeTransientEditTimeInnerObject != nullptr);
						CHECK(NonNativeTransientEditTimeInnerObject->IsInOuter(OuterObject));

						CHECK(NonNativeTransientEditTimeInnerObject->TestValue == 200);

						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
						CHECK(NonNativeTransientEditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->IsTemplate());
						CHECK(NonNativeTransientEditTimeInnerObject->IsDefaultSubobject());
						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->IsTemplateForSubobjects());

						USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
						CHECK(NonNativeTransientEditTimeInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
					}
					else
					{
						// As with the native case (see TransientEditTimeInnerObject), the non-native property is also tagged as 'Instanced', which means it is expected to have first been
						// default-initialized by the native class constructor. Since the native side cannot access this property except via reflection, it is generally not default-
						// initialized, and is left memset() to zero (NULL). While users can set the value on the CDO at edit time, the initializer will not propagate it to new instances
						// of the class, and it cannot be instanced as a result. For this reason, overriding non-native transient references at edit time is not supported in this case.
						CHECK(NonNativeTransientEditTimeInnerObject == nullptr);
					}
				}

				// Clean up the subobject on the CDO so it doesn't persist through remaining tests.
				ObjectProperty->SetObjectPropertyValue_InContainer(CDO, nullptr);
				NonNativeTransientEditTimeInnerObjectDefaultValue->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		}

		SECTION("Simulate non-native transient property instanced at edit time on non-native template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native iterations - the native class won't include the non-native property.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);

				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeTransientEditTimeInnerObjectPropertyName));
				USubobjectInstancingTestObject* NonNativeTransientEditTimeInnerObjectDefaultValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(Template));

				// Default value before it's edited (it will have been zero-initialized).
				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue == nullptr);

				// Simulate a deferred edit-time instancing event along with a value override on the non-native template.
				NonNativeTransientEditTimeInnerObjectDefaultValue = NewObject<USubobjectInstancingTestObject>(Template, TEXT("NonNativeTransientEditTimeInnerObject"), Template->GetMaskedFlags(RF_PropagateToSubObjects));
				NonNativeTransientEditTimeInnerObjectDefaultValue->TestValue = 200;
				ObjectProperty->SetObjectPropertyValue_InContainer(Template, NonNativeTransientEditTimeInnerObjectDefaultValue);

				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue->IsTemplate());
				// As with the non-transient case, since we instanced the subobject on the template and not the CDO, this is not considered to be a default subobject.
				CHECK_FALSE(NonNativeTransientEditTimeInnerObjectDefaultValue->IsDefaultSubobject());
				// And, like the non-transient case, it is also not considered to be a template for a CDO or default subobject, but rather a standalone archetype subobject.
				CHECK_FALSE(NonNativeTransientEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects());
				CHECK(NonNativeTransientEditTimeInnerObjectDefaultValue->IsTemplateForSubobjects(RF_ArchetypeObject));

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				USubobjectInstancingTestObject* NonNativeTransientEditTimeInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));
				{
					// As with the CDO case, transient references that are not default-constructed in native code can only be instanced from the archetype's value in the dynamic referencing case.
					if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
					{
						REQUIRE(NonNativeTransientEditTimeInnerObject != nullptr);
						CHECK(NonNativeTransientEditTimeInnerObject->IsInOuter(OuterObject));

						CHECK(NonNativeTransientEditTimeInnerObject->TestValue == 200);

						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->HasAllFlags(RF_DefaultSubObject));
						CHECK(NonNativeTransientEditTimeInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->IsTemplate());
						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->IsDefaultSubobject());
						CHECK_FALSE(NonNativeTransientEditTimeInnerObject->IsTemplateForSubobjects());

						USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
						CHECK(NonNativeTransientEditTimeInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
					}
					else
					{
						// As with the CDO test, the property is tagged as 'Instanced' in this case, and transient 'Instanced' properties must be constructed in native code.
						// Since this is a non-native property, the native super type will not default-construct the archetype for this value, and it is left NULL as a result.
						CHECK(NonNativeTransientEditTimeInnerObject == nullptr);
					}
				}
			}
		}

		SECTION("Non-native transient subobject initialized from CDO")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native iterations - the native class won't include the non-native property.
			}
			else
			{
				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeTransientInnerObjectPropertyName));
				USubobjectInstancingTestOuterObject* CDO = Cast<USubobjectInstancingTestOuterObject>(InstancingTestClass->GetDefaultObject());

				// The CDO's value was assigned at class creation time as an RF_ArchetypeObject subobject, simulating a non-native constructor assignment.
				USubobjectInstancingTestObject* NonNativeTransientInnerObjectDefaultValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(CDO));
				REQUIRE(NonNativeTransientInnerObjectDefaultValue != nullptr);
				CHECK(NonNativeTransientInnerObjectDefaultValue->IsTemplate());
				CHECK(NonNativeTransientInnerObjectDefaultValue->IsDefaultSubobject());
				CHECK(NonNativeTransientInnerObjectDefaultValue->IsTemplateForSubobjects());

				NonNativeTransientInnerObjectDefaultValue->TestValue = 200;

				USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				USubobjectInstancingTestObject* NonNativeTransientInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));
				{
					if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
					{
						// Dynamic instancing propagates the CDO's transient non-native value to new instances.
						REQUIRE(NonNativeTransientInnerObject != nullptr);
						CHECK(NonNativeTransientInnerObject->IsInOuter(OuterObject));

						CHECK(NonNativeTransientInnerObject->TestValue == 200);

						CHECK_FALSE(NonNativeTransientInnerObject->HasAllFlags(RF_DefaultSubObject));
						CHECK(NonNativeTransientInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

						CHECK_FALSE(NonNativeTransientInnerObject->IsTemplate());
						CHECK(NonNativeTransientInnerObject->IsDefaultSubobject());
						CHECK_FALSE(NonNativeTransientInnerObject->IsTemplateForSubobjects());

						USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
						CHECK(NonNativeTransientInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
					}
					else
					{
						// As with the native transient edit-time case, the property requires 'Instanced' (CPF_InstancedReference), which means it must first be initialized by native code.
						// Since the native super type cannot access this non-native property, it is never default-constructed, and the transient InitProperties copy path is suppressed.
						CHECK(NonNativeTransientInnerObject == nullptr);
					}
				}

				// Restore CDO subobject state so it doesn't affect subsequent tests.
				NonNativeTransientInnerObjectDefaultValue->TestValue = 100;
			}
		}

		SECTION("Non-native transient subobject initialized from template")
		{
			if (InstancingTestClass->IsNative())
			{
				// Skip: This test only applies to non-native iterations - the native class won't include the non-native property.
			}
			else
			{
				USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
				FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(Private::NonNativeTransientInnerObjectPropertyName));
				USubobjectInstancingTestObject* NonNativeTransientInnerObjectTemplateValue = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(Template));

				if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
				{
					// When the template was constructed from the CDO, dynamic instancing created a copy of the CDO's transient value on the template.
					REQUIRE(NonNativeTransientInnerObjectTemplateValue != nullptr);
					CHECK(NonNativeTransientInnerObjectTemplateValue->IsDefaultSubobject());
					CHECK(NonNativeTransientInnerObjectTemplateValue->IsTemplateForSubobjects(RF_ArchetypeObject));

					NonNativeTransientInnerObjectTemplateValue->TestValue = 200;

					USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
					USubobjectInstancingTestObject* NonNativeTransientInnerObject = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

					REQUIRE(NonNativeTransientInnerObject != nullptr);
					CHECK(NonNativeTransientInnerObject->IsInOuter(OuterObject));

					CHECK(NonNativeTransientInnerObject->TestValue == 200);

					CHECK_FALSE(NonNativeTransientInnerObject->HasAllFlags(RF_DefaultSubObject));
					CHECK(NonNativeTransientInnerObject->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

					CHECK_FALSE(NonNativeTransientInnerObject->IsTemplate());
					CHECK(NonNativeTransientInnerObject->IsDefaultSubobject());
					CHECK_FALSE(NonNativeTransientInnerObject->IsTemplateForSubobjects());

					USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
					CHECK(NonNativeTransientInnerObject != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
				}
				else
				{
					// CPF_Transient suppresses the InitProperties template copy for instanced references, so the template value is nullptr
					// and no instancing occurs.
					CHECK(NonNativeTransientInnerObjectTemplateValue == nullptr);

					USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
					CHECK(Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject)) == nullptr);
				}
			}
		}

		SECTION("Native property initialized from template with deferred construction using NewObject()")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->InnerObjectPostInit->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->InnerObjectPostInit != nullptr);
			CHECK(OuterObject->InnerObjectPostInit->IsInOuter(OuterObject));

			CHECK(OuterObject->InnerObjectPostInit->TestValue == 200);

			CHECK_FALSE(OuterObject->InnerObjectPostInit->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->InnerObjectPostInit->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->InnerObjectPostInit->IsTemplate());
			CHECK(OuterObject->InnerObjectPostInit->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->InnerObjectPostInit->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->InnerObjectPostInit != NewOuterObject->InnerObjectPostInit);
		}

		SECTION("Native transient property initialized from CDO with deferred construction using NewObject()")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			// Transient instanced properties are excluded from default value initialization - see FObjectInitializer::InitProperties().
			// Default value initialization is deferred to the native PostInitProperties() call, so should be non-NULL post-construction.
			REQUIRE(OuterObject->TransientInnerObjectPostInit != nullptr);
			CHECK(OuterObject->TransientInnerObjectPostInit->IsInOuter(OuterObject));

			CHECK(OuterObject->TransientInnerObjectPostInit->TestValue == 100);

			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectPostInit->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectPostInit->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(OuterObject->TransientInnerObjectPostInit != NewOuterObject->TransientInnerObjectPostInit);
		}

		SECTION("Native transient property initialized from template with deferred construction using NewObject()")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			Template->TransientInnerObjectPostInit->TestValue = 200;

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);

			REQUIRE(OuterObject->TransientInnerObjectPostInit != nullptr);
			CHECK(OuterObject->TransientInnerObjectPostInit->IsInOuter(OuterObject));

			// As with the TransientInnerObjectUsingNew case, transient instanced properties are excluded from default value initialization.
			// This means it is also effectively the same result as passing TRUE for the transient argument to CreateDefaultSubobject().
			CHECK(OuterObject->TransientInnerObjectPostInit->TestValue == 100);

			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->HasAllFlags(RF_DefaultSubObject));
			CHECK(OuterObject->TransientInnerObjectPostInit->HasAllFlags(OuterObject->GetMaskedFlags(RF_PropagateToSubObjects)));

			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->IsTemplate());
			CHECK(OuterObject->TransientInnerObjectPostInit->IsDefaultSubobject());
			CHECK_FALSE(OuterObject->TransientInnerObjectPostInit->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			CHECK(OuterObject->TransientInnerObjectPostInit != NewOuterObject->TransientInnerObjectPostInit);
		}
	}

	void RunSubobjectInstancingTests_NonNullable(UClass* InstancingTestClass)
	{
		// Non-nullable tests are only applicable to non-native properties right now.
		if (InstancingTestClass->IsNative())
		{
			return;
		}

		const FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(NonNullableInnerObjectPropertyName));
		const FObjectProperty* TransientProperty = CastFieldChecked<FObjectProperty>(InstancingTestClass->FindPropertyByName(NonNullableTransientInnerObjectPropertyName));
		const FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(InstancingTestClass->FindPropertyByName(NonNullableOptionalInnerObjectPropertyName));

		SECTION("Non-nullable subobject reference property semantics")
		{
			CHECK(ObjectProperty->HasIntrusiveUnsetOptionalState());
		}

		SECTION("Non-nullable subobject reference initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

			REQUIRE(ObjectReference != nullptr);
			CHECK(ObjectReference->IsIn(OuterObject));
			CHECK(ObjectReference->TestValue == 100);

			CHECK_FALSE(ObjectReference->IsTemplate());
			CHECK(ObjectReference->IsDefaultSubobject());
			CHECK_FALSE(ObjectReference->IsTemplateForSubobjects());

			USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			CHECK(ObjectReference != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
		}

		SECTION("Non-nullable subobject reference initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			{
				USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(Template));

				REQUIRE(ObjectReference != nullptr);
				CHECK(ObjectReference->IsIn(Template));
				CHECK(ObjectReference->TestValue == 100);

				ObjectReference->TestValue = 200;
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			{
				USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(ObjectProperty->GetObjectPropertyValue_InContainer(OuterObject));

				REQUIRE(ObjectReference != nullptr);
				CHECK(ObjectReference->IsIn(OuterObject));
				CHECK(ObjectReference->TestValue == 200);

				CHECK_FALSE(ObjectReference->IsTemplate());
				CHECK(ObjectReference->IsDefaultSubobject());
				CHECK_FALSE(ObjectReference->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
				CHECK(ObjectReference != ObjectProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
			}
		}

		SECTION("Non-nullable transient subobject reference initialized from CDO")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
			USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(TransientProperty->GetObjectPropertyValue_InContainer(OuterObject));

			if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
			{
				REQUIRE(ObjectReference != nullptr);
				CHECK(ObjectReference->IsIn(OuterObject));
				CHECK(ObjectReference->TestValue == 100);

				CHECK_FALSE(ObjectReference->IsTemplate());
				CHECK(ObjectReference->IsDefaultSubobject());
				CHECK_FALSE(ObjectReference->IsTemplateForSubobjects());

				USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);
				CHECK(ObjectReference != TransientProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
			}
			else
			{
				// Note: Transient non-nullable reference types are not supported unless dynamic instancing is enabled. These are default-initialized to NULL,
				// and are then skipped as part of default value initialization when CPF_InstancedReference is also set (see FObjectInitializer::InitProperties).
				// Note: This assumes non-nullable reference types are only declared on non-native class types. This test is not currently valid for native types.
				CHECK(ObjectReference == nullptr);
			}
		}

		SECTION("Non-nullable transient subobject reference initialized from template")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			{
				USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(TransientProperty->GetObjectPropertyValue_InContainer(Template));
				
				if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
				{
					REQUIRE(ObjectReference != nullptr);
					CHECK(ObjectReference->IsIn(Template));
					CHECK(ObjectReference->TestValue == 100);

					ObjectReference->TestValue = 200;
				}
				else
				{
					// Note: As with the CDO case, transient non-nullable reference types are not supported unless dynamic instancing is enabled.
					CHECK(ObjectReference == nullptr);
				}
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			{
				USubobjectInstancingTestObject* ObjectReference = Cast<USubobjectInstancingTestObject>(TransientProperty->GetObjectPropertyValue_InContainer(OuterObject));

				if (InstancingTestClass->ShouldUseDynamicSubobjectInstancing())
				{
					REQUIRE(ObjectReference != nullptr);
					CHECK(ObjectReference->IsIn(OuterObject));
					CHECK(ObjectReference->TestValue == 200);

					CHECK_FALSE(ObjectReference->IsTemplate());
					CHECK(ObjectReference->IsDefaultSubobject());
					CHECK_FALSE(ObjectReference->IsTemplateForSubobjects());

					USubobjectInstancingTestOuterObject* NewOuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
					CHECK(ObjectReference != TransientProperty->GetObjectPropertyValue_InContainer(NewOuterObject));
				}
				else
				{
					// Note: As with the CDO case, transient non-nullable reference types are not supported unless dynamic instancing is enabled.
					CHECK(ObjectReference == nullptr);
				}
			}
		}

		SECTION("Non-nullable optional subobject reference property semantics")
		{
			CHECK_FALSE(OptionalProperty->HasIntrusiveUnsetOptionalState());

			FProperty* ValueProperty = OptionalProperty->GetValueProperty();
			{
				REQUIRE(ValueProperty != nullptr);
				CHECK(ValueProperty->HasIntrusiveUnsetOptionalState());
			}
		}

		SECTION("Non-nullable optional subobject reference initialized from CDO - unset state")
		{
			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			CHECK_FALSE(OptionalProperty->IsSet(OptionalProperty->ContainerPtrToValuePtr<void>(OuterObject)));
		}

		SECTION("Non-nullable optional subobject reference initialized from template - unset state")
		{
			USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_ArchetypeObject);
			{
				CHECK_FALSE(OptionalProperty->IsSet(OptionalProperty->ContainerPtrToValuePtr<void>(Template)));
			}

			USubobjectInstancingTestOuterObject* OuterObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass, NAME_None, RF_NoFlags, Template);
			{
				CHECK_FALSE(OptionalProperty->IsSet(OptionalProperty->ContainerPtrToValuePtr<void>(OuterObject)));
			}
		}
	}

	void RunSubobjectInstancingTests_Other(UClass* InstancingTestClass)
	{
		SECTION("StaticDuplicateObject() instancing")
		{
			USubobjectInstancingTestOuterObject* SrcObject = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), InstancingTestClass);

			SrcObject->InnerObject->TestValue = 200;
			SrcObject->SharedObject->TestValue = 200;
			SrcObject->TransientInnerObject->TestValue = 200;
			SrcObject->LocalOnlyInnerObject->TestValue = 200;
			SrcObject->InnerObjectFromType->TestValue = 200;
			SrcObject->InnerObjectUsingNew->TestValue = 200;
			SrcObject->TransientInnerObjectUsingNew->TestValue = 200;
			SrcObject->DuplicateTransientInnerObjectUsingNew->TestValue = 200;
			SrcObject->InnerObjectPostInit->TestValue = 200;
			SrcObject->TransientInnerObjectPostInit->TestValue = 200;
			SrcObject->StructWithInnerObjects.InnerObject->TestValue = 200;
			SrcObject->OptionalInnerObject.GetValue()->TestValue = 200;
			for (USubobjectInstancingTestObject* InnerObject : SrcObject->InnerObjectArray)
			{
				InnerObject->TestValue = 200;
			}

			SrcObject->InnerObjectWithDirectlyNestedObject->TestValue = 200;
			SrcObject->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue = 200;

			SrcObject->InnerObjectWithIndirectlyNestedObject->TestValue = 200;
			SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject->TestValue = 200;
			SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue = 200;

			// Simulate a deferred edit-time instancing event along with a value override for editinline tests
			SrcObject->DuplicateTransientEditTimeInnerObject = NewObject<USubobjectInstancingTestObject>(SrcObject, GET_MEMBER_NAME_CHECKED(USubobjectInstancingTestOuterObject, DuplicateTransientEditTimeInnerObject), SrcObject->GetMaskedFlags(RF_PropagateToSubObjects));
			SrcObject->DuplicateTransientEditTimeInnerObject->TestValue = 200;

			// Note: Using PIE mode for duplication as low level test runners do not configure UPS at boot time and normal mode requires it.
			FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(SrcObject, GetTransientPackage());
			Params.PortFlags = PPF_DuplicateForPIE;

			USubobjectInstancingTestOuterObject* DstObject = CastChecked<USubobjectInstancingTestOuterObject>(StaticDuplicateObjectEx(Params));

			CHECK(DstObject->SelfRef == DstObject);
			CHECK(DstObject->NullObject == nullptr);

			CHECK(DstObject->InnerObject != nullptr);
			CHECK(DstObject->InnerObject->IsInOuter(DstObject));
			CHECK(SrcObject->InnerObject != DstObject->InnerObject);
			CHECK(SrcObject->InnerObject->TestValue == DstObject->InnerObject->TestValue);

			CHECK(DstObject->SharedObject != nullptr);
			CHECK(DstObject->SharedObject->IsInOuter(DstObject));
			CHECK(SrcObject->SharedObject != DstObject->SharedObject);
			CHECK(SrcObject->SharedObject->TestValue == DstObject->SharedObject->TestValue);

			CHECK(DstObject->ExternalObject != nullptr);
			CHECK_FALSE(DstObject->ExternalObject->IsInOuter(DstObject));
			CHECK(SrcObject->ExternalObject == DstObject->ExternalObject);

			CHECK(DstObject->TransientInnerObject != nullptr);
			CHECK(DstObject->TransientInnerObject->IsInOuter(DstObject));
			CHECK(DstObject->TransientInnerObject != SrcObject->TransientInnerObject);
			CHECK(DstObject->TransientInnerObject->TestValue == SrcObject->TransientInnerObject->TestValue);

			CHECK(DstObject->DuplicateTransientInnerObjectUsingNew != nullptr);
			CHECK(DstObject->DuplicateTransientInnerObjectUsingNew->IsInOuter(DstObject));
			CHECK(DstObject->DuplicateTransientInnerObjectUsingNew != SrcObject->DuplicateTransientInnerObjectUsingNew);
			// Note: DuplicateTransient will not prevent a default subobject from being serialized after UE-22874
			// The expected result here would be that TestValue does not receive the override value from the source
			// A workaround that's widely in use is to construct non-default only and not rely on subobject instancing
			// @todo - This is likely a bug that needs fixing, but it will break backcompat - this ensures that for now
			CHECK(SrcObject->DuplicateTransientInnerObjectUsingNew->IsDefaultSubobject());
			CHECK(DstObject->DuplicateTransientInnerObjectUsingNew->TestValue == SrcObject->DuplicateTransientInnerObjectUsingNew->TestValue);

			// By contrast, DuplicateTransient correctly affects a property that has no default-constructed archetype
			// The expected result is the duplicatetransient reference will not be duplicated and thus is not serialized
			CHECK(SrcObject->DuplicateTransientEditTimeInnerObject != nullptr);
			CHECK_FALSE(SrcObject->DuplicateTransientEditTimeInnerObject->IsDefaultSubobject());
			CHECK_FALSE(DstObject->DuplicateTransientEditTimeInnerObject != nullptr);

			CHECK(DstObject->LocalOnlyInnerObject != nullptr);
			CHECK(DstObject->LocalOnlyInnerObject->IsInOuter(DstObject));
			CHECK(DstObject->LocalOnlyInnerObject != SrcObject->LocalOnlyInnerObject);
			CHECK(DstObject->LocalOnlyInnerObject->TestValue == SrcObject->LocalOnlyInnerObject->TestValue);

			CHECK(DstObject->InnerObjectFromType != nullptr);
			CHECK(DstObject->InnerObjectFromType->IsInOuter(DstObject));
			CHECK(DstObject->InnerObjectFromType != SrcObject->LocalOnlyInnerObject);
			CHECK(DstObject->InnerObjectFromType->TestValue == SrcObject->InnerObjectFromType->TestValue);

			CHECK(DstObject->InnerObjectUsingNew != nullptr);
			CHECK(DstObject->InnerObjectUsingNew->IsInOuter(DstObject));
			CHECK(DstObject->InnerObjectUsingNew != SrcObject->InnerObjectUsingNew);
			CHECK(DstObject->InnerObjectUsingNew->TestValue == SrcObject->InnerObjectUsingNew->TestValue);

			CHECK(DstObject->TransientInnerObjectFromType != nullptr);
			CHECK(DstObject->TransientInnerObjectFromType->IsInOuter(DstObject));
			CHECK(DstObject->TransientInnerObjectFromType != SrcObject->TransientInnerObjectFromType);
			CHECK(DstObject->TransientInnerObjectFromType->TestValue == SrcObject->TransientInnerObjectFromType->TestValue);

			CHECK(DstObject->TransientInnerObjectUsingNew != nullptr);
			CHECK(DstObject->TransientInnerObjectUsingNew->IsInOuter(DstObject));
			CHECK(DstObject->TransientInnerObjectUsingNew != SrcObject->TransientInnerObjectUsingNew);
			CHECK(DstObject->TransientInnerObjectUsingNew->TestValue == SrcObject->TransientInnerObjectUsingNew->TestValue);

			CHECK(DstObject->InnerObjectPostInit != nullptr);
			CHECK(DstObject->InnerObjectPostInit->IsInOuter(DstObject));
			CHECK(DstObject->InnerObjectPostInit != SrcObject->InnerObjectPostInit);
			CHECK(DstObject->InnerObjectPostInit->TestValue == SrcObject->InnerObjectPostInit->TestValue);

			CHECK(DstObject->TransientInnerObjectPostInit != nullptr);
			CHECK(DstObject->TransientInnerObjectPostInit->IsInOuter(DstObject));
			CHECK(DstObject->TransientInnerObjectPostInit != SrcObject->TransientInnerObjectPostInit);
			CHECK(DstObject->TransientInnerObjectPostInit->TestValue == SrcObject->TransientInnerObjectPostInit->TestValue);

			CHECK(DstObject->StructWithInnerObjects.InnerObject != nullptr);
			CHECK(DstObject->StructWithInnerObjects.InnerObject->IsInOuter(DstObject));
			CHECK(DstObject->StructWithInnerObjects.InnerObject != SrcObject->StructWithInnerObjects.InnerObject);
			CHECK(DstObject->StructWithInnerObjects.InnerObject->TestValue == SrcObject->StructWithInnerObjects.InnerObject->TestValue);

			CHECK(DstObject->OptionalInnerObject.IsSet());
			CHECK(DstObject->OptionalInnerObject.GetValue() != nullptr);
			CHECK(DstObject->OptionalInnerObject.GetValue()->IsInOuter(DstObject));
			CHECK(DstObject->OptionalInnerObject.GetValue() != SrcObject->OptionalInnerObject.GetValue());
			CHECK(DstObject->OptionalInnerObject.GetValue()->TestValue == SrcObject->OptionalInnerObject.GetValue()->TestValue);

			CHECK_FALSE(DstObject->InnerObjectArray.IsEmpty());
			CHECK(DstObject->InnerObjectArray.Num() == SrcObject->InnerObjectArray.Num());
			for (int32 ArrayIdx = 0; ArrayIdx < DstObject->InnerObjectArray.Num(); ++ArrayIdx)
			{
				CHECK(DstObject->InnerObjectArray[ArrayIdx] != nullptr);
				CHECK(DstObject->InnerObjectArray[ArrayIdx]->IsInOuter(DstObject));
				CHECK(DstObject->InnerObjectArray[ArrayIdx] != SrcObject->InnerObjectArray[ArrayIdx]);
				CHECK(DstObject->InnerObjectArray[ArrayIdx]->TestValue == SrcObject->InnerObjectArray[ArrayIdx]->TestValue);
			}

			CHECK(DstObject->InnerObjectWithDirectlyNestedObject != nullptr);
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->IsInOuter(DstObject));
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject != SrcObject->InnerObjectWithDirectlyNestedObject);
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->TestValue == SrcObject->InnerObjectWithDirectlyNestedObject->TestValue);

			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->InnerObject != nullptr);
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->InnerObject->IsInOuter(DstObject->InnerObjectWithDirectlyNestedObject));
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->InnerObject != SrcObject->InnerObjectWithDirectlyNestedObject->InnerObject);
			CHECK(DstObject->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue == SrcObject->InnerObjectWithDirectlyNestedObject->InnerObject->TestValue);

			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject != nullptr);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->IsInOuter(DstObject));
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject != SrcObject->InnerObjectWithIndirectlyNestedObject);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->TestValue == SrcObject->InnerObjectWithIndirectlyNestedObject->TestValue);

			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject != nullptr);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->IsInOuter(DstObject->InnerObjectWithIndirectlyNestedObject));
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject != SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->TestValue == SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject->TestValue);

			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != nullptr);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsInOuter(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject));
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject);
			CHECK(DstObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue == SrcObject->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->TestValue);
		}
	}

	void RunSubobjectInstancingTests(UClass* InstancingTestClass)
	{
		RunSubobjectInstancingTests_Base(InstancingTestClass);
		RunSubobjectInstancingTests_Array(InstancingTestClass);
		RunSubobjectInstancingTests_Set(InstancingTestClass);
		RunSubobjectInstancingTests_Map(InstancingTestClass);
		RunSubobjectInstancingTests_Optional(InstancingTestClass);
		RunSubobjectInstancingTests_Struct(InstancingTestClass);
		RunSubobjectInstancingTests_Nested(InstancingTestClass);
		RunSubobjectInstancingTests_Deferred(InstancingTestClass);
		RunSubobjectInstancingTests_SelfReferencing(InstancingTestClass);
		RunSubobjectInstancingTests_NonNullable(InstancingTestClass);
		RunSubobjectInstancingTests_Other(InstancingTestClass);
	}
}

const FName FSubobjectInstancingTestUtils::GetNonNativeInnerObjectPropertyName()
{
	return SubobjectInstancingTest::Private::NonNativeInnerObjectPropertyName;
}

const FName FSubobjectInstancingTestUtils::GetNonNativeOuterObjectPropertyName()
{
	return SubobjectInstancingTest::Private::NonNativeOuterObjectPropertyName;
}

const FName FSubobjectInstancingTestUtils::GetNonNativeSelfReferencePropertyName()
{
	return SubobjectInstancingTest::Private::NonNativeSelfReferencePropertyName;
}

const FName FSubobjectInstancingTestUtils::GetNonNativeEditTimeInnerObjectPropertyName()
{
	return SubobjectInstancingTest::Private::NonNativeEditTimeInnerObjectPropertyName;
}

UClass* FSubobjectInstancingTestUtils::CreateNonNativeInstancingTestClass(UClass* SuperClass)
{
	return SubobjectInstancingTest::Private::NewTestClass<UClass>(SuperClass);
}

UClass* FSubobjectInstancingTestUtils::CreateDynamicallyInstancedTestClass(UClass* SuperClass)
{
	return SubobjectInstancingTest::Private::NewTestClass<UDynamicSubobjectInstancingTestClass>(SuperClass);
}

TEST_CASE_NAMED(FNativeDefaultSubobjectsTest, "CoreUObject::NativeDefaultSubobjects", "[CoreUObject][EngineFilter]")
{
	SECTION("Class default subobjects")
	{
		const USubobjectInstancingTestOuterObject* CDO = GetDefault<USubobjectInstancingTestOuterObject>();

		REQUIRE(CDO->InnerObject != nullptr);
		CHECK(CDO->InnerObject->IsInOuter(CDO));

		CHECK(CDO->InnerObject->HasAllFlags(RF_ArchetypeObject));
		CHECK(CDO->InnerObject->HasAllFlags(RF_DefaultSubObject));
		CHECK(CDO->InnerObject->HasAllFlags(CDO->GetMaskedFlags(RF_PropagateToSubObjects)));

		CHECK(CDO->InnerObject->IsTemplate());
		CHECK(CDO->InnerObject->IsDefaultSubobject());
		CHECK(CDO->InnerObject->IsTemplateForSubobjects());

		REQUIRE(CDO->InnerObjectWithDirectlyNestedObject != nullptr);
		REQUIRE(CDO->InnerObjectWithDirectlyNestedObject->InnerObject != nullptr);
		CHECK(CDO->InnerObjectWithDirectlyNestedObject->InnerObject->IsInOuter(CDO->InnerObjectWithDirectlyNestedObject));

		CHECK(CDO->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplate());
		CHECK(CDO->InnerObjectWithDirectlyNestedObject->InnerObject->IsDefaultSubobject());
		CHECK(CDO->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

		REQUIRE(CDO->InnerObjectWithIndirectlyNestedObject != nullptr);
		REQUIRE(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject != nullptr);
		REQUIRE(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != nullptr);
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->IsInOuter(CDO->InnerObjectWithIndirectlyNestedObject));
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsInOuter(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject));

		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->IsTemplate());
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->IsDefaultSubobject());
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplate());
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsDefaultSubobject());
		CHECK(CDO->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplateForSubobjects());
	}

	SECTION("Template default subobjects")
	{
		USubobjectInstancingTestOuterObject* Template = NewObject<USubobjectInstancingTestOuterObject>(GetTransientPackage(), NAME_None, RF_ArchetypeObject);

		REQUIRE(Template->InnerObject != nullptr);
		CHECK(Template->InnerObject->IsInOuter(Template));

		CHECK(Template->InnerObject->HasAllFlags(RF_ArchetypeObject));
		CHECK(Template->InnerObject->HasAllFlags(RF_DefaultSubObject));
		CHECK(Template->InnerObject->HasAllFlags(Template->GetMaskedFlags(RF_PropagateToSubObjects)));

		CHECK(Template->InnerObject->IsTemplate());
		CHECK(Template->InnerObject->IsDefaultSubobject());
		CHECK(Template->InnerObject->IsTemplateForSubobjects());

		REQUIRE(Template->InnerObjectWithDirectlyNestedObject != nullptr);
		REQUIRE(Template->InnerObjectWithDirectlyNestedObject->InnerObject != nullptr);
		CHECK(Template->InnerObjectWithDirectlyNestedObject->InnerObject->IsInOuter(Template->InnerObjectWithDirectlyNestedObject));

		CHECK(Template->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplate());
		CHECK(Template->InnerObjectWithDirectlyNestedObject->InnerObject->IsDefaultSubobject());
		CHECK(Template->InnerObjectWithDirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

		REQUIRE(Template->InnerObjectWithIndirectlyNestedObject != nullptr);
		REQUIRE(Template->InnerObjectWithIndirectlyNestedObject->InnerObject != nullptr);
		REQUIRE(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject != nullptr);
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->IsInOuter(Template->InnerObjectWithIndirectlyNestedObject));
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsInOuter(Template->InnerObjectWithIndirectlyNestedObject->InnerObject));

		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->IsTemplate());
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->IsDefaultSubobject());
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->IsTemplateForSubobjects());

		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplate());
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsDefaultSubobject());
		CHECK(Template->InnerObjectWithIndirectlyNestedObject->InnerObject->InnerObject->IsTemplateForSubobjects());
	}

	SECTION("Default subobject constructed using SetDefaultSubobjectClass() override")
	{
		USubobjectInstancingTestDerivedOuterObjectWithTypeOverride* OuterObject = NewObject<USubobjectInstancingTestDerivedOuterObjectWithTypeOverride>(GetTransientPackage());

		REQUIRE(OuterObject->InnerObject != nullptr);
		CHECK(OuterObject->InnerObject->IsA<USubobjectInstancingTestDerivedObject>());
		CHECK(OuterObject->InnerObject->GetArchetype()->IsA<USubobjectInstancingTestObject>());
	}

	SECTION("Optional default subobject excluded from construction using DoNotCreateDefaultSubobject() override")
	{
		USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride* OuterObject = NewObject<USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride>(GetTransientPackage());

		REQUIRE(OuterObject->OptionalInnerObject.IsSet());
		CHECK(OuterObject->OptionalInnerObject.GetValue() == nullptr);
	}
}

TEST_CASE_NAMED(FNativeInstancedSubobjectsTest, "CoreUObject::NativeInstancedSubobjects", "[CoreUObject][EngineFilter]")
{
	UClass* NativeInstancingTestClass = USubobjectInstancingTestOuterObject::StaticClass();

	REQUIRE(NativeInstancingTestClass != nullptr);
	SubobjectInstancingTest::Private::RunSubobjectInstancingTests(NativeInstancingTestClass);
}

TEST_CASE_NAMED(FNonNativeInstancedSubobjectsTest, "CoreUObject::NonNativeInstancedSubobjects", "[CoreUObject][EngineFilter]")
{
	UClass* NativeInstancingTestClass = USubobjectInstancingTestOuterObject::StaticClass();
	UClass* NonNativeInstancingTestClass = FSubobjectInstancingTestUtils::CreateNonNativeInstancingTestClass(NativeInstancingTestClass);

	REQUIRE(NonNativeInstancingTestClass != nullptr);
	SubobjectInstancingTest::Private::RunSubobjectInstancingTests(NonNativeInstancingTestClass);
}

TEST_CASE_NAMED(FDynamicSubobjectInstancingTest, "CoreUObject::DynamicSubobjectInstancing", "[CoreUObject][EngineFilter]")
{
	UClass* NativeInstancingTestClass = USubobjectInstancingTestOuterObject::StaticClass();
	UClass* DynamicallyInstancedTestClass = FSubobjectInstancingTestUtils::CreateDynamicallyInstancedTestClass(NativeInstancingTestClass);

	REQUIRE(DynamicallyInstancedTestClass != nullptr);
	CHECK(DynamicallyInstancedTestClass->ShouldUseDynamicSubobjectInstancing());
	SubobjectInstancingTest::Private::RunSubobjectInstancingTests(DynamicallyInstancedTestClass);
}

}

#endif // WITH_TESTS
