// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TESTS

#include "UObject/Class.h"
#include "UObject/Object.h"
#include "SubobjectInstancingTest.generated.h"

namespace UE
{

struct FSubobjectInstancingTestUtils
{
	static const FName GetNonNativeInnerObjectPropertyName();
	static const FName GetNonNativeOuterObjectPropertyName();
	static const FName GetNonNativeSelfReferencePropertyName();
	static const FName GetNonNativeEditTimeInnerObjectPropertyName();

	static UClass* CreateNonNativeInstancingTestClass(UClass* SuperClass = nullptr);
	static UClass* CreateDynamicallyInstancedTestClass(UClass* SuperClass = nullptr);
};

}

UCLASS()
class USubobjectInstancingTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 TestValue = 100;
};

UCLASS()
class USubobjectInstancingTestDerivedObject : public USubobjectInstancingTestObject
{
	GENERATED_BODY()
};

UCLASS()
class USubobjectInstancingTestDirectlyNestedObject : public USubobjectInstancingTestObject
{
	GENERATED_BODY()

public:
	USubobjectInstancingTestDirectlyNestedObject();

	UPROPERTY(Instanced)
	TObjectPtr<UObject> SelfRef;

	UPROPERTY(Instanced)
	TObjectPtr<UObject> OwnerObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InnerObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> EditTimeInnerObject;
};

UCLASS()
class USubobjectInstancingTestIndirectlyNestedObject : public USubobjectInstancingTestObject
{
	GENERATED_BODY()

public:
	USubobjectInstancingTestIndirectlyNestedObject();

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestDirectlyNestedObject> InnerObject;
};

UCLASS(Transient)
class USubobjectInstancingTransientTestObject : public USubobjectInstancingTestObject
{
	GENERATED_BODY()
};

UCLASS(DefaultToInstanced)
class USubobjectInstancingDefaultToInstancedTestObject : public USubobjectInstancingTestObject
{
	GENERATED_BODY()
};

USTRUCT()
struct FSubobjectInstancingTestStructType
{
	GENERATED_BODY()

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InnerObject;

	UPROPERTY()
	TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject> InnerObjectFromType;

	UPROPERTY(Instanced)
	TArray<TObjectPtr<USubobjectInstancingTestObject>> InnerObjectArray;
};

UCLASS()
class USubobjectInstancingTestOuterObject : public UObject
{
    GENERATED_BODY()

public:
	USubobjectInstancingTestOuterObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void PostInitProperties() override;

	UPROPERTY(Instanced)
	TObjectPtr<UObject> SelfRef;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> NullObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InnerObject;

	UPROPERTY()
	TObjectPtr<USubobjectInstancingTestObject> SharedObject;

	UPROPERTY()
	TObjectPtr<USubobjectInstancingTestObject> ExternalObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InternalObject;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> TransientInnerObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> EditTimeInnerObject;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> TransientEditTimeInnerObject;

	UPROPERTY(DuplicateTransient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> DuplicateTransientEditTimeInnerObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> LocalOnlyInnerObject;

	UPROPERTY()
	TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject> InnerObjectFromType;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTransientTestObject> TransientInnerObjectFromType;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InnerObjectUsingNew;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> TransientInnerObjectUsingNew;

	UPROPERTY(DuplicateTransient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> DuplicateTransientInnerObjectUsingNew;

	UPROPERTY(NonPIEDuplicateTransient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> NonPIEDuplicateTransientInnerObjectUsingNew;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestObject> InnerObjectPostInit;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<USubobjectInstancingTestObject> TransientInnerObjectPostInit;

	UPROPERTY(Instanced)
	TArray<TObjectPtr<USubobjectInstancingTestObject>> InnerObjectArray;

	UPROPERTY()
	TArray<TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject>> InnerObjectFromTypeArray;

	UPROPERTY(Instanced, Transient)
	TArray<TObjectPtr<USubobjectInstancingTestObject>> TransientInnerObjectArray;

	UPROPERTY(Instanced)
	TSet<TObjectPtr<USubobjectInstancingTestObject>> InnerObjectSet;

	UPROPERTY()
	TSet<TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject>> InnerObjectFromTypeSet;

	UPROPERTY(Instanced, Transient)
	TSet<TObjectPtr<USubobjectInstancingTestObject>> TransientInnerObjectSet;

	UPROPERTY(Instanced)
	TMap<TObjectPtr<USubobjectInstancingTestObject>, TObjectPtr<USubobjectInstancingTestObject>> InnerObjectMap;

	UPROPERTY()
	TMap<TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject>, TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject>> InnerObjectFromTypeMap;

	UPROPERTY(Instanced, Transient)
	TMap<TObjectPtr<USubobjectInstancingTestObject>, TObjectPtr<USubobjectInstancingTestObject>> TransientInnerObjectMap;

	UPROPERTY()
	FSubobjectInstancingTestStructType StructWithInnerObjects;

	UPROPERTY(Instanced)
	TOptional<TObjectPtr<USubobjectInstancingTestObject>> OptionalInnerObject;

	UPROPERTY()
	TOptional<TObjectPtr<USubobjectInstancingDefaultToInstancedTestObject>> OptionalInnerObjectFromType;

	UPROPERTY(Instanced, Transient)
	TOptional<TObjectPtr<USubobjectInstancingTestObject>> OptionalTransientInnerObject;

	UPROPERTY(Instanced)
	TOptional<TArray<TObjectPtr<USubobjectInstancingTestObject>>>  OptionalInnerObjectArray;

	UPROPERTY(Instanced)
	TOptional<TObjectPtr<USubobjectInstancingTestObject>> OptionalNullableInnerObjectUnset;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestDirectlyNestedObject> InnerObjectWithDirectlyNestedObject;

	UPROPERTY(Instanced)
	TObjectPtr<USubobjectInstancingTestIndirectlyNestedObject> InnerObjectWithIndirectlyNestedObject;
};

UCLASS()
class USubobjectInstancingTestDerivedOuterObjectWithTypeOverride : public USubobjectInstancingTestOuterObject
{
	GENERATED_BODY()

public:
	USubobjectInstancingTestDerivedOuterObjectWithTypeOverride(const FObjectInitializer& ObjectInitializer);
};

UCLASS()
class USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride : public USubobjectInstancingTestOuterObject
{
	GENERATED_BODY()

public:
	USubobjectInstancingTestDerivedOuterObjectWithDoNotCreateOverride(const FObjectInitializer& ObjectInitializer);
};

UCLASS()
class UDynamicSubobjectInstancingTestClass : public UClass
{
	GENERATED_BODY()

public:
	UDynamicSubobjectInstancingTestClass();
};

#endif	// WITH_TESTS
