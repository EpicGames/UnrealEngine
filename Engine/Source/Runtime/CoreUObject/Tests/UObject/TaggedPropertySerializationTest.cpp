// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedPropertySerializationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TaggedPropertySerializationTest)

#if WITH_TESTS

#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/StructuredArchive.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FTaggedPropertySerializationTest, "CoreUObject::Serialization::TaggedPropertySerialization", "[Core][UObject][EngineFilter]")
{
	SECTION("Convert TArray<bool> to TArray<int32>")
	{
		TArray<uint8> Data;
		{
			FTestTaggedPropertySerializationBoolArray Struct;
			Struct.Array = { true, false, true, false };

			FMemoryWriter Ar(Data, /*bIsPersistent*/ true);
			FTestTaggedPropertySerializationBoolArray::StaticStruct()->SerializeItem(Ar, &Struct, nullptr);
		}
		{
			FTestTaggedPropertySerializationInt32Array Struct;

			FMemoryReader Ar(Data, /*bIsPersistent*/ true);
			FTestTaggedPropertySerializationInt32Array::StaticStruct()->SerializeItem(Ar, &Struct, nullptr);

			REQUIRE(Struct.Array.Num() == 4);
			CHECK(Struct.Array[0] == 1);
			CHECK(Struct.Array[1] == 0);
			CHECK(Struct.Array[2] == 1);
			CHECK(Struct.Array[3] == 0);
		}
	#if WITH_TEXT_ARCHIVE_SUPPORT
		TArray<uint8> JsonData;
		{
			FTestTaggedPropertySerializationBoolArray Struct;
			Struct.Array = { true, false, true, false };

			FMemoryWriter Ar(JsonData, /*bIsPersistent*/ true);
			FJsonArchiveOutputFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			FStructuredArchiveSlot Slot = Record.EnterField(TEXT("Value"));
			FTestTaggedPropertySerializationBoolArray::StaticStruct()->SerializeItem(Slot, &Struct, nullptr);
		}
		{
			FTestTaggedPropertySerializationInt32Array Struct;

			FMemoryReader Ar(JsonData, /*bIsPersistent*/ true);
			FJsonArchiveInputFormatter Formatter(Ar);
			FStructuredArchive StructuredAr(Formatter);
			FStructuredArchiveRecord Record = StructuredAr.Open().EnterRecord();
			FStructuredArchiveSlot Slot = Record.EnterField(TEXT("Value"));
			FTestTaggedPropertySerializationInt32Array::StaticStruct()->SerializeItem(Slot, &Struct, nullptr);

			REQUIRE(Struct.Array.Num() == 4);
			CHECK(Struct.Array[0] == 1);
			CHECK(Struct.Array[1] == 0);
			CHECK(Struct.Array[2] == 1);
			CHECK(Struct.Array[3] == 0);
		}
	#endif
	}
}

} // UE

#endif // WITH_TESTS
