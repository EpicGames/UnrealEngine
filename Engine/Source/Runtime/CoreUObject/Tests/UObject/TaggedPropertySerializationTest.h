// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "UObject/Class.h"

#include "TaggedPropertySerializationTest.generated.h"

#if WITH_TESTS

USTRUCT()
struct FTestTaggedPropertySerializationBoolArray
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<bool> Array;
};

USTRUCT()
struct FTestTaggedPropertySerializationInt32Array
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<int32> Array;
};

#endif // WITH_TESTS
