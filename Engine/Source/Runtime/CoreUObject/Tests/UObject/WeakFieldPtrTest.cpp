// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyTypeName.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/UnrealType.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(TWeakFieldPtrSmokeTest, "CoreUObject::TWeakFieldPtr::Smoke", "[Core][UObject][SmokeFilter]")
{
	SECTION("Check that TWeakFieldPtr can hold 'const FProperty'")
	{
		const FProperty* PropPtr = nullptr;
		TWeakFieldPtr<const FProperty> ConstPtr1;
		TWeakFieldPtr<const FProperty> ConstPtr2;

		ConstPtr1 = PropPtr;
		ConstPtr1 = ConstPtr2;

		// validate that the static_asserts inside these operators allow const FProperty as T
		CHECK(ConstPtr1 == ConstPtr2);
		CHECK(!(ConstPtr1 != ConstPtr2));
		CHECK(ConstPtr1 == PropPtr);
		CHECK(!(ConstPtr1 != PropPtr));
	}

	SECTION("Check that TWeakFieldPtr of different types can be compared in both directions")
	{
		FProperty* TestProperty = nullptr;

		TWeakFieldPtr<FProperty> WeakProperty = TestProperty;
		TWeakFieldPtr<FField> WeakField = TestProperty;

		CHECK(WeakProperty == WeakField);
		CHECK(WeakField == WeakProperty);
	}

	SECTION("Check that TWeakFieldPtr can be compared against raw pointers")
	{
		FProperty* TestProperty = nullptr;
		FProperty* ConstTestProperty = nullptr;

		FProperty* TestField = nullptr;
		FProperty* ConstTestField = nullptr;

		TWeakFieldPtr<FProperty> WeakProperty = TestProperty;
		TWeakFieldPtr<const FProperty> ConstWeakProperty = ConstTestProperty;

		CHECK(WeakProperty == TestProperty);
		CHECK_FALSE(WeakProperty != TestProperty);
		CHECK(WeakProperty == ConstTestProperty);
		CHECK_FALSE(WeakProperty != ConstTestProperty);

		CHECK(WeakProperty == TestField);
		CHECK_FALSE(WeakProperty != TestField);
		CHECK(WeakProperty == ConstTestField);
		CHECK_FALSE(WeakProperty != ConstTestField);

		CHECK(ConstWeakProperty == TestProperty);
		CHECK_FALSE(ConstWeakProperty != TestProperty);
		CHECK(ConstWeakProperty == ConstTestProperty);
		CHECK_FALSE(ConstWeakProperty != ConstTestProperty);

		CHECK(ConstWeakProperty == TestField);
		CHECK_FALSE(ConstWeakProperty != TestField);
		CHECK(ConstWeakProperty == ConstTestField);
		CHECK_FALSE(ConstWeakProperty != ConstTestField);
	}
}

} // UE

#endif // WITH_TESTS
