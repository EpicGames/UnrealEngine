// Copyright Epic Games, Inc. All Rights Reserved.

#include "WeakObjectPtrTest.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#endif

UWeakObjectPtrTest::UWeakObjectPtrTest()
{
	InputRefWeakPtrDelegate.BindDynamic(this, &UWeakObjectPtrTest::InputRefWeakPtr);
	InputValWeakPtrDelegate.BindDynamic(this, &UWeakObjectPtrTest::InputValWeakPtr);
	InputRefWeakPtrMulticastDelegate.AddDynamic(this, &UWeakObjectPtrTest::InputRefWeakPtr);
	InputValWeakPtrMulticastDelegate.AddDynamic(this, &UWeakObjectPtrTest::InputValWeakPtr);
	InOutWeakPtrDelegate.BindDynamic(this, &UWeakObjectPtrTest::InOutWeakPtr);
	InOutWeakPtrMulticastDelegate.AddDynamic(this, &UWeakObjectPtrTest::InOutWeakPtr);
	ReturnWeakPtrDelegate.BindDynamic(this, &UWeakObjectPtrTest::ReturnWeakPtr);
}

void UWeakObjectPtrTest::InputRefWeakPtr(const TWeakObjectPtr<UObject>& InWeakPtr)
{
#if WITH_LOW_LEVEL_TESTS
	CHECK(InWeakPtr == this);
#endif
}

void UWeakObjectPtrTest::InputValWeakPtr(TWeakObjectPtr<UObject> InWeakPtr)
{
#if WITH_LOW_LEVEL_TESTS
	CHECK(InWeakPtr == this);
#endif
}

void UWeakObjectPtrTest::InOutWeakPtr(TWeakObjectPtr<UObject>& InOutWeakPtr)
{
#if WITH_LOW_LEVEL_TESTS
	CHECK(InOutWeakPtr == nullptr);
#endif
	InOutWeakPtr = this;
}

TWeakObjectPtr<UObject> UWeakObjectPtrTest::ReturnWeakPtr()
{
	return this;
}

#if WITH_LOW_LEVEL_TESTS

void UWeakObjectPtrTest::RunTests()
{
	// Check that a weakobjptr can be passed through a dynamic delegate
	TWeakObjectPtr<UObject> InRefPtr = this;
	InputRefWeakPtrDelegate.Execute(InRefPtr);
	TWeakObjectPtr<UObject> InValPtr = this;
	InputValWeakPtrDelegate.Execute(InValPtr);

	// Check that a weakobjptr can be passed through a dynamic multicast delegate
	TWeakObjectPtr<UObject> InRefPtrMulticast = this;
	InputRefWeakPtrMulticastDelegate.Broadcast(InRefPtrMulticast);
	TWeakObjectPtr<UObject> InValPtrMulticast = this;
	InputValWeakPtrMulticastDelegate.Broadcast(InValPtrMulticast);

	// Check that a weakobjptr can be passed in and out of a dynamic delegate
	TWeakObjectPtr<UObject> InOutPtr;
	InOutWeakPtrDelegate.Execute(InOutPtr);
	CHECK(InOutPtr == this);

	// Check that a weakobjptr can be passed in and out of a multicast dynamic delegate
	TWeakObjectPtr<UObject> InOutPtrMulticast;
	InOutWeakPtrMulticastDelegate.Broadcast(InOutPtrMulticast);
	CHECK(InOutPtrMulticast == this);

	// Check that a weakobjptr can be returned by a dynamic delegate
	TWeakObjectPtr<UObject> OutPtr = ReturnWeakPtrDelegate.Execute();
	CHECK(OutPtr == this);
}

TEST_CASE("UE::CoreUObject::WeakObjectPtrTest", "[CoreUObject][WeakObjectPtrTest]")
{
	UWeakObjectPtrTest* TestObj = NewObject<UWeakObjectPtrTest>();
	TestObj->RunTests();
}

#endif // WITH_LOW_LEVEL_TESTS
