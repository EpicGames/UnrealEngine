// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "WeakObjectPtrTest.generated.h"

DECLARE_DYNAMIC_DELEGATE_OneParam(FInputRefWeakPtrDelegate, const TWeakObjectPtr<UObject>&, WeakPtr);
DECLARE_DYNAMIC_DELEGATE_OneParam(FInputValWeakPtrDelegate, TWeakObjectPtr<UObject>, WeakPtr);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInputRefWeakPtrMulticastDelegate, const TWeakObjectPtr<UObject>&, WeakPtr);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInputValWeakPtrMulticastDelegate, TWeakObjectPtr<UObject>, WeakPtr);
DECLARE_DYNAMIC_DELEGATE_OneParam(FInOutWeakPtrDelegate, TWeakObjectPtr<UObject>&, WeakPtr);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInOutWeakPtrMulticastDelegate, TWeakObjectPtr<UObject>&, WeakPtr);
DECLARE_DYNAMIC_DELEGATE_RetVal(TWeakObjectPtr<UObject>, FReturnWeakPtrDelegate);

UCLASS()
class UWeakObjectPtrTest : public UObject
{
	GENERATED_BODY()
public:

	UWeakObjectPtrTest();

	UPROPERTY()
	FInputRefWeakPtrDelegate InputRefWeakPtrDelegate;

	UPROPERTY()
	FInputValWeakPtrDelegate InputValWeakPtrDelegate;

	UPROPERTY()
	FInputRefWeakPtrMulticastDelegate InputRefWeakPtrMulticastDelegate;

	UPROPERTY()
	FInputValWeakPtrMulticastDelegate InputValWeakPtrMulticastDelegate;

	UPROPERTY()
	FInOutWeakPtrDelegate InOutWeakPtrDelegate;

	UPROPERTY()
	FInOutWeakPtrMulticastDelegate InOutWeakPtrMulticastDelegate;

	UPROPERTY()
	FReturnWeakPtrDelegate ReturnWeakPtrDelegate;

	UFUNCTION()
	void InputRefWeakPtr(const TWeakObjectPtr<UObject>& InWeakPtr);

	UFUNCTION()
	void InputValWeakPtr(TWeakObjectPtr<UObject> InWeakPtr);

	UFUNCTION()
	void InOutWeakPtr(TWeakObjectPtr<UObject>& InOutWeakPtr);

	UFUNCTION()
	TWeakObjectPtr<UObject> ReturnWeakPtr();

	void RunTests();
};
